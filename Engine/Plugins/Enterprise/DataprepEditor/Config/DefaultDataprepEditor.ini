[CoreRedirects]

+ClassRedirects=(OldName="/Script/DataprepEditor.K2Node_DataprepProducer", NewName="/Script/DataprepCore.K2Node_DataprepProducer")
+ClassRedirects=(OldName="DataprepRemoveObjectsOperation", NewName="/Script/DataprepLibraries.DataprepDeleteObjectsOperation")
+PropertyRedirects=(OldName="DataprepSetMaterialOperation.MaterialSubstitute",NewName="DataprepSetMaterialOperation.Material")
+PropertyRedirects=(OldName="DataprepSetMeshOperation.MeshSubstitute",NewName="DataprepSetMeshOperation.StaticMesh")
+EnumRedirects=(OldName="EDataprepFloatMatchType",ValueChanges=(("LessThen","LessThan"),("GreatherThen","GreatherThan")))
