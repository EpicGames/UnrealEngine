{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Dataprep Editor",
	"Description": "A tool to simplify creation and execution of data preparation pipelines from within the Unreal Editor.",
	"Category": "Dataprep",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataprepEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataprepCore",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataprepLibraries",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataprepEditorScriptingUtilities",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithContent",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "VariantManagerContent",
			"Enabled": true
		},
		{
			"Name": "Interchange",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"IsExperimentalVersion": false
}