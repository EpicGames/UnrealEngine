// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataprepCore : ModuleRules
	{
		public DataprepCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"BlueprintGraph",
					"Core",
					"CoreUObject",
					"DatasmithContent",
					"EditorFramework",
					"EditorScriptingUtilities",
					"EditorStyle",
					"EditorWidgets",
					"Engine",
					"GraphEditor",
					"InputCore",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"KismetCompiler",
					"LevelSequence",
					"MeshDescription",
					"MessageLog",
					"PropertyEditor",
					"RenderCore",
					"RHI",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"UnrealEd",
					"VariantManagerContent",
				}
			);
		}
	}
}
