// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/K2Node_DataprepActionCore.h"

// Dataprep includes
#include "DataprepActionAsset.h"

void UK2Node_DataprepActionCore::CreateDataprepActionAsset()
{
	if ( !DataprepActionAsset )
	{
		DataprepActionAsset = NewObject<UDataprepActionAsset>( this, UDataprepActionAsset::StaticClass(), NAME_None, RF_Transactional );
		DataprepActionAsset->SetLabel( *GetNodeTitle(ENodeTitleType::EditableTitle).ToString() );
	}
}

void UK2Node_DataprepActionCore::DuplicateExistingDataprepActionAsset(const UDataprepActionAsset& DataprepAsset)
{
	if ( !DataprepActionAsset )
	{
		DataprepActionAsset = DuplicateObject<UDataprepActionAsset>( &DataprepAsset, this, NAME_None );
		DataprepActionAsset->SetFlags( RF_Transactional );
		DataprepActionAsset->SetLabel(*GetNodeTitle(ENodeTitleType::EditableTitle).ToString());
	}
}
