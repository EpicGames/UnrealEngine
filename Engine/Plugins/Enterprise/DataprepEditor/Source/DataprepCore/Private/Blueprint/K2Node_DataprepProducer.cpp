// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DataprepProducer.h"

#include "EdGraphSchema_K2.h"
#include "SGraphNode.h"


void UK2Node_DataprepProducer::AllocateDefaultPins()
{
	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	Super::AllocateDefaultPins();
}

FText UK2Node_DataprepProducer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("UK2Node_DataprepProducer", "NodeTitle", "Start");
}

FText UK2Node_DataprepProducer::GetTooltipText() const
{
	return NSLOCTEXT("UK2Node_DataprepProducer", "NodeTooltip", "Hold onto all the action assets associated to a Blueprint based Dataprep asset");
}

#if WITH_EDITOR
TSharedPtr<SGraphNode> UK2Node_DataprepProducer::CreateVisualWidget()
{
	typedef TTuple< UClass*, FText, FText > DataprepProducerDescription;

	class SGraphNodeDataprepProducer : public SGraphNode
	{
	public:
		SLATE_BEGIN_ARGS(SGraphNodeDataprepProducer){}
		SLATE_END_ARGS()

			SGraphNodeDataprepProducer()
		{
		}

		~SGraphNodeDataprepProducer()
		{
		}

		void Construct(const FArguments& InArgs, UK2Node_DataprepProducer* InNode)
		{
			GraphNode = InNode;
			UpdateGraphNode();
		}
	};

	return SNew(SGraphNodeDataprepProducer, this);
}
#endif
