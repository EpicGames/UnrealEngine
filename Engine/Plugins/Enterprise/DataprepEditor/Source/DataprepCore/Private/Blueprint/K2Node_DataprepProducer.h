// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"


#include "K2Node_DataprepProducer.generated.h"

namespace ENodeTitleType { enum Type : int; }

UCLASS()
class UK2Node_DataprepProducer : public UK2Node
{
public:
	GENERATED_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
#if WITH_EDITOR
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
#endif
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override { return false; }
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return true; }
	//~ End K2Node Interface
};
