// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepActionAsset.h"

// Dataprep include
#include "DataprepAsset.h"
#include "DataprepCoreLogCategory.h"
#include "Misc/Paths.h"
#include "Shared/DataprepCorePrivateUtils.h"
#include "DataprepCoreUtils.h"
#include "DataprepOperation.h"
#include "SelectionSystem/DataprepFilter.h"
#include "SelectionSystem/DataprepSelectionTransform.h"
#include "Parameterization/DataprepParameterization.h"

// Engine include
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "LevelSequence.h"
#include "Materials/MaterialInterface.h"
#include "ObjectTools.h"

#include "Editor.h"
#include "Misc/TransactionObjectEvent.h"

void UDataprepActionStep::PostLoad()
{
	Super::PostLoad();

	if ( Operation_DEPRECATED )
	{
		StepObject = Operation_DEPRECATED;
		PathOfStepObjectClass = StepObject->GetClass();
		Operation_DEPRECATED = nullptr;
	}

	if ( Filter_DEPRECATED )
	{
		StepObject = Filter_DEPRECATED;
		PathOfStepObjectClass = StepObject->GetClass();
		Filter_DEPRECATED = nullptr;
	}
}

UDataprepActionAsset::UDataprepActionAsset()
	: ContextPtr( nullptr )
	, Label(TEXT("New Action"))
{
	bExecutionInterrupted = false;
	bIsEnabled = true;

	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		OperationContext = MakeShareable( new FDataprepOperationContext() );

		OperationContext->Context = MakeShareable( new FDataprepContext() );

		OperationContext->AddAssetDelegate = FDataprepAddAsset::CreateUObject( this, &UDataprepActionAsset::OnAddAsset );
		OperationContext->CreateAssetDelegate = FDataprepCreateAsset::CreateUObject( this, &UDataprepActionAsset::OnCreateAsset );
		OperationContext->CreateActorDelegate = FDataprepCreateActor::CreateUObject( this, &UDataprepActionAsset::OnCreateActor );
		OperationContext->RemoveObjectDelegate = FDataprepRemoveObject::CreateUObject( this, &UDataprepActionAsset::OnRemoveObject );
		OperationContext->DeleteObjectsDelegate = FDataprepDeleteObjects::CreateUObject( this, &UDataprepActionAsset::OnDeleteObjects );
		OperationContext->AssetsModifiedDelegate = FDataprepAssetsModified::CreateUObject( this, &UDataprepActionAsset::OnAssetsModified );
	}
}

UDataprepActionAsset::~UDataprepActionAsset()
{
	FEditorDelegates::OnAssetsDeleted.Remove( OnAssetDeletedHandle );
}

void UDataprepActionAsset::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		if (TransactionEvent.GetChangedProperties().Contains(TEXT("Steps")))
		{
			OnStepsOrderChanged.Broadcast();
		}
	}
}

void UDataprepActionAsset::Execute(const TArray<UObject*>& InObjects)
{
	ContextPtr = MakeShareable( new FDataprepActionContext() );

	for(UObject* Object : InObjects)
	{
		if(Object && FDataprepCoreUtils::IsAsset(Object))
		{
			ContextPtr->Assets.Add(Object);
		}
	}

	// Make a copy of the objects to act on
	OperationContext->Context->Objects = InObjects;

	// Execute steps sequentially
	for ( UDataprepActionStep* Step : Steps )
	{
		if ( Step && Step->bIsEnabled )
		{
			UDataprepParameterizableObject* StepObject = Step->GetStepObject();
			UClass* StepType = FDataprepCoreUtils::GetTypeOfActionStep( StepObject );
			if ( StepType == UDataprepOperation::StaticClass() )
			{
				UDataprepOperation* Operation = static_cast<UDataprepOperation*>( StepObject );
				Operation->Execute( OperationContext->Context->Objects );
			}
			else if ( StepType == UDataprepFilter::StaticClass() )
			{
				UDataprepFilter* Filter = static_cast<UDataprepFilter*>( StepObject );

				TArray<TObjectPtr<UObject>>& Objects = OperationContext->Context->Objects;
				OperationContext->Context->Objects = Filter->FilterObjects( TArrayView<TObjectPtr<UObject>>( Objects.GetData(), Objects.Num() ) );
			}
			else if ( StepType == UDataprepFilterNoFetcher::StaticClass() )
			{
				UDataprepFilterNoFetcher* Filter = static_cast<UDataprepFilterNoFetcher*>( StepObject );

				TArray<TObjectPtr<UObject>>& Objects = OperationContext->Context->Objects;
				OperationContext->Context->Objects = Filter->FilterObjects( TArrayView<TObjectPtr<UObject>>( Objects.GetData(), Objects.Num() ) );
			}
			else if ( StepType == UDataprepSelectionTransform::StaticClass() )
			{
				UDataprepSelectionTransform* SelectionTransform = static_cast<UDataprepSelectionTransform*>(StepObject);
				TArray<UObject*> TransformResult;
				SelectionTransform->Execute(OperationContext->Context->Objects, TransformResult);
				OperationContext->Context->Objects = TransformResult;
			}
		}
	}

	// Reset list of selected objects
	OperationContext->Context->Objects.Reset();

	ContextPtr.Reset();
}

int32 UDataprepActionAsset::AddOperation(const TSubclassOf<UDataprepOperation>& OperationClass)
{
	return AddStep( OperationClass );
}

int32 UDataprepActionAsset::AddFilterWithAFetcher(const TSubclassOf<UDataprepFilter>& InFilterClass, const TSubclassOf<UDataprepFetcher>& InFetcherClass)
{
	return AddStep( InFetcherClass );
}

int32 UDataprepActionAsset::AddStep(TSubclassOf<UDataprepParameterizableObject> StepType)
{
	FText ErrorMessage;
	UClass* ValidRootClass = nullptr;

	if ( FDataprepCoreUtils::IsClassValidForStepCreation( StepType, ValidRootClass, ErrorMessage ) )
	{
		if ( ValidRootClass == UDataprepOperation::StaticClass() )
		{
			Modify();
			UDataprepActionStep* ActionStep = NewObject< UDataprepActionStep >(this, UDataprepActionStep::StaticClass(), NAME_None, RF_Transactional);
			ActionStep->StepObject = NewObject< UDataprepOperation >(ActionStep, StepType.Get(), NAME_None, RF_Transactional);
			ActionStep->PathOfStepObjectClass = ActionStep->StepObject->GetClass();
			ActionStep->bIsEnabled = true;
			Steps.Add(ActionStep);
			OnStepsOrderChanged.Broadcast();
			return Steps.Num() - 1;
		}
		if ( ValidRootClass == UDataprepSelectionTransform::StaticClass() )
		{
			Modify();
			UDataprepActionStep* ActionStep = NewObject< UDataprepActionStep >(this, UDataprepActionStep::StaticClass(), NAME_None, RF_Transactional);
			ActionStep->StepObject = NewObject< UDataprepSelectionTransform >(ActionStep, StepType.Get(), NAME_None, RF_Transactional);
			ActionStep->PathOfStepObjectClass = ActionStep->StepObject->GetClass();
			ActionStep->bIsEnabled = true;
			Steps.Add(ActionStep);
			OnStepsOrderChanged.Broadcast();
			return Steps.Num() - 1;
		}
		else if ( ValidRootClass == UDataprepFetcher::StaticClass() )
		{
			if ( UClass* FilterClass = UDataprepFilter::GetFilterTypeForFetcherType( StepType.Get() ) )
			{
				Modify();
				UDataprepActionStep* ActionStep = NewObject< UDataprepActionStep >( this, UDataprepActionStep::StaticClass(), NAME_None, RF_Transactional );
				UDataprepFilter* Filter = NewObject< UDataprepFilter >( ActionStep, FilterClass, NAME_None, RF_Transactional );
				Filter->SetFetcher( StepType.Get() );
				ActionStep->StepObject = Filter;
				ActionStep->PathOfStepObjectClass = ActionStep->StepObject->GetClass();
				ActionStep->bIsEnabled = true;
				Steps.Add( ActionStep );
				OnStepsOrderChanged.Broadcast();
				return Steps.Num() - 1;
			}
			else
			{
				UE_LOGF(LogDataprepCore, Error, "The fetcher type (%ls) is not used by a filter.", *StepType->GetPathName() )
			}
		}
		else if ( ValidRootClass == UDataprepFilterNoFetcher::StaticClass() )
		{
			Modify();
			UDataprepActionStep* ActionStep = NewObject< UDataprepActionStep >( this, UDataprepActionStep::StaticClass(), NAME_None, RF_Transactional );
			UDataprepFilterNoFetcher* Filter = NewObject< UDataprepFilterNoFetcher >( ActionStep, StepType.Get(), NAME_None, RF_Transactional );
			ActionStep->StepObject = Filter;
			ActionStep->PathOfStepObjectClass = ActionStep->StepObject->GetClass();
			ActionStep->bIsEnabled = true;
			Steps.Add( ActionStep );
			OnStepsOrderChanged.Broadcast();
			return Steps.Num() - 1;
		}

		// Please keep this function up to date with FDataprepCoreUtils::IsClassValidForStepCreation
		check( false );
	}

	UE_LOGF( LogDataprepCore, Error, "%ls", *(ErrorMessage.ToString()) );

	return INDEX_NONE;
}

UDataprepActionStep* UDataprepActionAsset::DuplicateStep(const UDataprepActionStep* InActionStep)
{
	UDataprepActionStep* NewActionStep = DuplicateObject<UDataprepActionStep>( InActionStep, this );

	UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject(this);
	check(DataprepAsset);

	NewActionStep->StepObject = DuplicateObject<UDataprepParameterizableObject>(InActionStep->GetStepObject(), NewActionStep);
	NewActionStep->PathOfStepObjectClass = NewActionStep->StepObject->GetClass();

	if (const UDataprepParameterizableObject* OriginalObject = InActionStep->GetStepObject())
	{
		if (UDataprepParameterization* Parameterization = DataprepAsset->GetDataprepParameterization())
		{
			Parameterization->DuplicateObjectParamaterization(InActionStep->GetStepObject(), NewActionStep->GetStepObject());
		}
	}
	
	return NewActionStep;
}

int32 UDataprepActionAsset::AddStep(const UDataprepActionStep* InActionStep)
{
	if ( InActionStep )
	{
		Modify();
		UDataprepActionStep* ActionStep = DuplicateStep(InActionStep);
		ActionStep->SetFlags(EObjectFlags::RF_Transactional);
		Steps.Add( ActionStep );
		OnStepsOrderChanged.Broadcast();
		return Steps.Num() - 1;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::AddStep: The action step is invalid" );
	ensure(false);
	// Invalid
	return INDEX_NONE;
}

int32 UDataprepActionAsset::AddSteps(const TArray<const UDataprepActionStep*>& InActionSteps)
{
	if ( InActionSteps.Num() > 0 && InActionSteps[0] != nullptr )
	{
		Modify();

		Steps.Reserve(Steps.Num() + InActionSteps.Num());

		for(const UDataprepActionStep* InActionStep : InActionSteps)
		{
			if(InActionStep)
			{
				UDataprepActionStep* ActionStep = DuplicateStep( InActionStep );
				ActionStep->SetFlags(EObjectFlags::RF_Transactional);
				Steps.Add( ActionStep );
			}
		}

		OnStepsOrderChanged.Broadcast();
		return Steps.Num() - 1;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::AddSteps: The array is empty or the action steps are invalid" );
	ensure(false);
	// Invalid
	return INDEX_NONE;
}

int32 UDataprepActionAsset::AddStep(const UDataprepParameterizableObject* StepObject)
{
	if (!StepObject)
	{
		UE_LOGF(LogDataprepCore, Error, "UDataprepActionAsset::AddStep: The Step Object is null");
	}
	else if (FDataprepCoreUtils::GetTypeOfActionStep(StepObject))
	{
		Modify();
		UDataprepActionStep* ActionStep = NewObject< UDataprepActionStep >(this, UDataprepActionStep::StaticClass(), NAME_None, RF_Transactional);
		ActionStep->StepObject = DuplicateObject<UDataprepParameterizableObject>(StepObject, ActionStep);
		ActionStep->PathOfStepObjectClass = ActionStep->StepObject->GetClass();
		ActionStep->bIsEnabled = true;
		Steps.Add(ActionStep);
		OnStepsOrderChanged.Broadcast();
		return Steps.Num() - 1;
	}
	else
	{
		UE_LOGF(LogDataprepCore, Error, "UDataprepActionAsset::AddStep: The Step Object is invalid");
	}

	// Invalid
	return INDEX_NONE;
}

bool UDataprepActionAsset::InsertStep(const UDataprepActionStep* InActionStep, int32 Index)
{
	if ( !Steps.IsValidIndex( Index ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::InsertStep: The Index is out of range" );
		ensure( false );
		return false;
	}

	if ( InActionStep )
	{
		Modify();
		UDataprepActionStep* ActionStep = DuplicateStep( InActionStep );
		ActionStep->SetFlags(EObjectFlags::RF_Transactional);
		Steps.Insert( ActionStep, Index );
		OnStepsOrderChanged.Broadcast();
		return true;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::InsertStep: The action step is invalid" );
	ensure(false);
	// Invalid
	return false;
}

bool UDataprepActionAsset::InsertSteps(const TArray<const UDataprepActionStep*>& InActionSteps, int32 Index)
{
	if ( !Steps.IsValidIndex( Index ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::InsertSteps: The Index is out of range" );
		ensure( false );
		return false;
	}

	if ( InActionSteps.Num() > 0 && InActionSteps[0] != nullptr )
	{
		Modify();

		Steps.Reserve(Steps.Num() + InActionSteps.Num());

		for(const UDataprepActionStep* InActionStep : InActionSteps)
		{
			if(InActionStep)
			{
				UDataprepActionStep* ActionStep = DuplicateStep( InActionStep );
				ActionStep->SetFlags(EObjectFlags::RF_Transactional);
				Steps.Insert( ActionStep, Index );
			}
		}

		OnStepsOrderChanged.Broadcast();
		return true;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::InsertSteps: The array is empty or the action steps are invalid" );
	ensure(false);
	// Invalid
	return false;
}

TWeakObjectPtr<UDataprepActionStep> UDataprepActionAsset::GetStep(int32 Index)
{
	// Avoid code duplication
	return static_cast< const UDataprepActionAsset* >( this )->GetStep( Index ) ;
}

const TWeakObjectPtr<UDataprepActionStep> UDataprepActionAsset::GetStep(int32 Index) const
{
	if ( Steps.IsValidIndex( Index ) )
	{
		return Steps[ Index ];
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::GetStep: The Index is out of range" );
	return nullptr;
}

int32 UDataprepActionAsset::GetStepsCount() const
{
	return Steps.Num();
}

bool UDataprepActionAsset::IsStepEnabled(int32 Index) const
{
	if (Steps.IsValidIndex(Index))
	{
		return Steps[Index]->bIsEnabled;
	}

	ensure( false );
	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::IsStepEnabled: The Index is out of range" );
	return false;
}

void UDataprepActionAsset::EnableStep(int32 Index, bool bEnable)
{
	if (Steps.IsValidIndex(Index))
	{
		Modify();
		Steps[Index]->bIsEnabled = bEnable;
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::EnableStep: The Index is out of range" );
	}
}

bool UDataprepActionAsset::MoveStep(int32 StepIndex, int32 DestinationIndex)
{
	if ( Steps.IsValidIndex( StepIndex ) && Steps.IsValidIndex( DestinationIndex ) )
	{
		Modify();
	}

	if ( DataprepCorePrivateUtils::MoveArrayElement( Steps, StepIndex, DestinationIndex ) )
	{
		OnStepsOrderChanged.Broadcast();
		return true;
	}

	if ( !Steps.IsValidIndex( StepIndex ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::MoveStep: The Step Index is out of range" );
	}
	if ( !Steps.IsValidIndex( DestinationIndex ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::MoveStep: The Destination Index is out of range" );
	}
	if ( StepIndex == DestinationIndex )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::MoveStep: a Step shouldn't be move at the location it currently is" );
	}

	return false;
}

bool UDataprepActionAsset::SwapSteps(int32 FirstIndex, int32 SecondIndex)
{
	if ( Steps.IsValidIndex( FirstIndex ) && Steps.IsValidIndex( SecondIndex ) )
	{
		Modify();

		UDataprepActionStep* FirstStep = Steps[FirstIndex];
		Steps[FirstIndex] = Steps[SecondIndex];
		Steps[SecondIndex] = FirstStep;
		OnStepsOrderChanged.Broadcast();
		return true;
	}

	if ( !Steps.IsValidIndex( FirstIndex ) )
	{
		UE_LOGF(LogDataprepCore, Error, "UDataprepActionAsset::SwapStep: The First Index is out of range");
	}
	if ( !Steps.IsValidIndex( SecondIndex ) )
	{
		UE_LOGF(LogDataprepCore, Error, "UDataprepActionAsset::SwapStep: The Second Index is out of range");
	}
	if ( FirstIndex == SecondIndex )
	{
		UE_LOGF(LogDataprepCore, Error, "UDataprepActionAsset::SwapStep: a Step shouldn't be swapped with itself");
	}

	return false;
}

bool UDataprepActionAsset::RemoveStep(int32 Index, bool bDiscardParametrization)
{
	if ( Steps.IsValidIndex( Index ) )
	{
		return RemoveSteps({ Index }, bDiscardParametrization);
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::RemoveStep: The Index is out of range" );
	return false;
}

bool UDataprepActionAsset::RemoveSteps(const TArray<int32>& Indices, bool bDiscardParametrization)
{
	if(Indices.Num() == 0)
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::RemoveSteps: Empty array" );
		return false;
	}

	bool bSuccesfulRemoval = false;

	Modify();

	// Remove steps back to front so array indices stay valid
	TArray<int32> SortedIndices( Indices );
	SortedIndices.Sort( []( int32 Idx0, int32 Idx1 ) -> bool
	{
		return Idx0 > Idx1;
	});

	for(int32 Index : SortedIndices)
	{
		if ( Steps.IsValidIndex( Index ) )
		{
			bSuccesfulRemoval = true;

			if ( bDiscardParametrization )
			{
				if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( this ) )
				{
					TArray< UObject* > Objects;
					GetObjectsWithOuter( Steps[Index], Objects );
					TArray< UDataprepParameterizableObject* > ParameterizableObjects;
					ParameterizableObjects.Reserve( Objects.Num() );
					for ( UObject* Object : Objects )
					{
						if (UDataprepParameterizableObject* ParameterizableObject = Cast<UDataprepParameterizableObject>( Object ) )
						{
							ParameterizableObjects.Add( ParameterizableObject );
						}
					}

					UDataprepAsset::FRestrictedToActionAsset::NotifyAssetOfTheRemovalOfSteps( *DataprepAsset, MakeArrayView<UDataprepParameterizableObject*>( ParameterizableObjects.GetData(), ParameterizableObjects.Num() ) );
				}
			}

			OnStepsAboutToBeRemoved.Broadcast( Steps[Index]->GetStepObject() );
			Steps.RemoveAt( Index );
		}
	}

	if(bSuccesfulRemoval)
	{
		OnStepsOrderChanged.Broadcast();
		return true;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepActionAsset::RemoveSteps: Invalid array of step indices" );
	return false;
}

FOnStepsOrderChanged& UDataprepActionAsset::GetOnStepsOrderChanged()
{
	return OnStepsOrderChanged;
}

FOnStepAboutToBeRemoved& UDataprepActionAsset::GetOnStepAboutToBeRemoved()
{
	return OnStepsAboutToBeRemoved;
}

FOnStepWasEdited& UDataprepActionAsset::GetOnStepWasEdited()
{
	return OnStepWasEdited;
}

UDataprepActionAppearance* UDataprepActionAsset::GetAppearance()
{
	if (nullptr == Appearance)
	{
		Appearance = NewObject<UDataprepActionAppearance>(this, UDataprepActionAppearance::StaticClass(), NAME_None, RF_Transactional);
		Appearance->bIsExpanded = true;
		Appearance->GroupId = INDEX_NONE;
		GetOutermost()->SetDirtyFlag(true);
	}

	return Appearance;
}

void UDataprepActionAsset::NotifyDataprepSystemsOfRemoval()
{
	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( this ) )
	{
		
		TArray< UObject* > Objects;
		TArray< UDataprepParameterizableObject* > ParameterizableObjects;
		for ( UDataprepActionStep* Step : Steps )
		{
			GetObjectsWithOuter( Step, Objects );
			ParameterizableObjects.Reserve( Objects.Num() );
			for ( UObject* Object : Objects )
			{
				if ( UDataprepParameterizableObject* ParameterizableObject = Cast<UDataprepParameterizableObject>( Object ) )
				{
					ParameterizableObjects.Add( ParameterizableObject );
				}
			}
		}

		UDataprepAsset::FRestrictedToActionAsset::NotifyAssetOfTheRemovalOfSteps( *DataprepAsset, MakeArrayView<UDataprepParameterizableObject*>( ParameterizableObjects.GetData(), ParameterizableObjects.Num() ) );
	}
}

void UDataprepActionAsset::ExecuteAction(const TSharedPtr<FDataprepActionContext>& InActionsContext, UDataprepActionStep* SpecificStep, bool bSpecificStepOnly)
{
	ContextPtr = InActionsContext;
	check(ContextPtr.IsValid());

	TArray<TObjectPtr<UObject>>& SelectedObjects = OperationContext->Context->Objects;

	// Make sure cached packages are pointing to the right path, if not reset
	if (PackageForTexture.IsValid() && !PackageForTexture->GetName().StartsWith(ContextPtr->TransientContentFolder))
	{
		PackageForTexture.Reset();
	}

	if (PackageForMaterial.IsValid() && !PackageForMaterial->GetName().StartsWith(ContextPtr->TransientContentFolder))
	{
		PackageForMaterial.Reset();
	}

	if (PackageForStaticMesh.IsValid() && !PackageForStaticMesh->GetName().StartsWith(ContextPtr->TransientContentFolder))
	{
		PackageForStaticMesh.Reset();
	}

	if (PackageForAnimation.IsValid() && !PackageForAnimation->GetName().StartsWith(ContextPtr->TransientContentFolder))
	{
		PackageForAnimation.Reset();
	}

	// Collect all objects the action to work on
	SelectedObjects.Empty( ContextPtr->Assets.Num() );
	for(TWeakObjectPtr<UObject>& ObjectPtr : ContextPtr->Assets)
	{
		if(UObject* Object = ObjectPtr.Get())
		{
			SelectedObjects.Add( Object );
		}
	}

	FDataprepCoreUtils::GetActorsFromWorld( ContextPtr->WorldPtr.Get(), MutableView(SelectedObjects) );

	OperationContext->DataprepLogger = ContextPtr->LoggerPtr;
	OperationContext->DataprepProgressReporter = ContextPtr->ProgressReporterPtr;

	auto ExecuteOneStep = [this, &SelectedObjects](UDataprepActionStep* Step)
	{
		UDataprepParameterizableObject* StepObject = Step->GetStepObject();
		UClass* StepType = FDataprepCoreUtils::GetTypeOfActionStep( StepObject );
		if ( StepType == UDataprepOperation::StaticClass() )
		{
			UDataprepOperation* Operation = static_cast<UDataprepOperation*>( StepObject );

			// Cache number of assets and objects before execution
			int32 AssetsDiffCount = ContextPtr->Assets.Num();
			int32 ActorsDiffCount = OperationContext->Context->Objects.Num();

			TSharedRef<FDataprepOperationContext> OperationContextPtr = this->OperationContext.ToSharedRef();
			Operation->ExecuteOperation( OperationContextPtr );

			// Process the changes in the context if applicable
			this->ProcessWorkingSetChanged();
		}
		else if ( StepType == UDataprepFilter::StaticClass() )
		{
			UDataprepFilter* Filter = static_cast<UDataprepFilter*>( StepObject );
			SelectedObjects = Filter->FilterObjects( SelectedObjects );
		}
		else if ( StepType == UDataprepFilterNoFetcher::StaticClass() )
		{
			UDataprepFilterNoFetcher* Filter = static_cast<UDataprepFilterNoFetcher*>( StepObject );
			SelectedObjects = Filter->FilterObjects( SelectedObjects );
		}
		else if ( StepType == UDataprepSelectionTransform::StaticClass() )
		{
			UDataprepSelectionTransform* SelectionTransform = static_cast<UDataprepSelectionTransform*>(StepObject);
			TArray<UObject*> TransformResult;
			SelectionTransform->Execute(SelectedObjects, TransformResult);
			SelectedObjects = TransformResult;
		}
	};

	if(SpecificStep && bSpecificStepOnly == true)
	{
		if ( SpecificStep->bIsEnabled )
		{
			ExecuteOneStep( SpecificStep );
		}
	}
	// Execute steps sequentially up to SpecificStep if applicable
	else
	{
		for ( UDataprepActionStep* Step : Steps )
		{
			if(Step != nullptr)
			{
				bWorkingSetHasChanged = false;

				if ( Step && Step->bIsEnabled )
				{
					ExecuteOneStep( Step );
				}

				if ( ContextPtr->ProgressReporterPtr && ContextPtr->ProgressReporterPtr->IsWorkCancelled() )
				{
					break;
				}

				if( ContextPtr->ContinueCallback && !ContextPtr->ContinueCallback( this ) )
				{
					break;
				}

				if( Step == SpecificStep )
				{
					break;
				}
			}
		}
	}

	SelectedObjects.Empty();
	ContextPtr.Reset();
}

UObject* UDataprepActionAsset::OnAddAsset(const UObject* Asset, const TCHAR* AssetName)
{
	UObject* NewAsset = nullptr;

	if(ContextPtr != nullptr && Asset != nullptr)
	{
		UObject* Outer = GetAssetOuterByClass( Asset->GetClass() );

		NewAsset = DuplicateObject< UObject >( Asset, Outer, NAME_None );
		check( NewAsset );

		AddAssetToContext( NewAsset, AssetName );
	}

	return NewAsset;
}

UObject* UDataprepActionAsset::OnCreateAsset(UClass* AssetClass, const TCHAR* AssetName)
{
	UObject* NewAsset = nullptr;

	if(ContextPtr != nullptr && AssetClass != nullptr)
	{
		UObject* Outer = GetAssetOuterByClass( AssetClass );

		NewAsset = NewObject<UObject>( Outer, AssetClass, NAME_None, RF_Public );
		check( NewAsset );

		AddAssetToContext( NewAsset, AssetName );
	}

	return NewAsset;
}

void UDataprepActionAsset::AddAssetToContext( UObject* NewAsset, const TCHAR* DesiredName )
{
	check( NewAsset );

	if(DesiredName != nullptr)
	{
		// Rename producer to name of file
		FString AssetName = ObjectTools::SanitizeObjectName( DesiredName );
		if ( !NewAsset->Rename( *AssetName, nullptr, REN_Test ) )
		{
			AssetName = MakeUniqueObjectName( GetOuter(), GetClass(), *AssetName ).ToString();
		}

		FDataprepCoreUtils::RenameObject( NewAsset, *AssetName );
	}

	// Add new asset to local and global contexts
	ContextPtr->Assets.Add( NewAsset );
	OperationContext->Context->Objects.Add( NewAsset );

	AddedObjects.Add( NewAsset );
	bWorkingSetHasChanged = true;
}

AActor* UDataprepActionAsset::OnCreateActor(UClass* ActorClass, const TCHAR* ActorName)
{
	if(ActorClass != nullptr && ContextPtr != nullptr)
	{
		AActor* Actor = ContextPtr->WorldPtr->SpawnActor<AActor>( ActorClass, FTransform::Identity );

		if(ActorName != nullptr)
		{
			FName UniqueName = MakeUniqueObjectName( Actor->GetOuter(), ActorClass, ActorName );
			FDataprepCoreUtils::RenameObject( Actor, *UniqueName.ToString() );
		}

		// Add new actor to local contexts
		OperationContext->Context->Objects.Add( Actor );

		AddedObjects.Add( Actor );
		bWorkingSetHasChanged = true;

		return Actor;
	}

	return nullptr;
}

void UDataprepActionAsset::OnRemoveObject(UObject* Object, bool bLocalContext)
{
	if(Object != nullptr && ContextPtr != nullptr)
	{
		ObjectsToRemove.Emplace(Object, bLocalContext);

		bWorkingSetHasChanged = true;
	}
}

void UDataprepActionAsset::OnAssetsModified(TArray<UObject*> Assets)
{
	if(ContextPtr != nullptr)
	{
		for(UObject* Asset : Assets)
		{
			if(Asset != nullptr)
			{
				ModifiedAssets.Add( Asset );
			}
		}
	}
}

void UDataprepActionAsset::OnDeleteObjects(TArray<UObject*> Objects)
{
	if(ContextPtr != nullptr)
	{
		for(UObject* Object : Objects)
		{
			// Mark object for deletion
			if(Object != nullptr)
			{
				if( FDataprepCoreUtils::IsAsset( Object ) && ModifiedAssets.Contains( Object ) )
				{
					ModifiedAssets.Remove( Object );
				}

				ObjectsToDelete.Add(Object);
				bWorkingSetHasChanged = true;
			}
		}
	}
}

void UDataprepActionAsset::ProcessWorkingSetChanged()
{
	if((bWorkingSetHasChanged || ModifiedAssets.Num() > 0) && ContextPtr != nullptr)
	{
		bool bAssetsChanged = ModifiedAssets.Num() > 0;
		bool bWorldChanged = false;

		for(UObject* Object : AddedObjects)
		{
			const AActor* Actor = Cast<AActor>(Object);
			bAssetsChanged |= Actor == nullptr;
			bWorldChanged |= Actor != nullptr;
		}

		TSet<UObject*> SelectedObjectSet( OperationContext->Context->Objects );

		if(ObjectsToRemove.Num() > 0)
		{
			for(TPair<UObject*, bool>& Pair : ObjectsToRemove)
			{
				if(SelectedObjectSet.Contains(Pair.Key))
				{
					SelectedObjectSet.Remove( Pair.Key );

					// Remove object from Dataprep's context
					if(!Pair.Value)
					{
						if(AActor* Actor = Cast<AActor>(Pair.Key))
						{
							ContextPtr->WorldPtr->RemoveActor( Actor, false );
							bWorldChanged = true;
						}
						else if( FDataprepCoreUtils::IsAsset(Pair.Key) )
						{
							bAssetsChanged = true;
							ContextPtr->Assets.Remove( Pair.Key );
						}
					}
				}
			}

			ObjectsToRemove.Empty( ObjectsToRemove.Num() );
		}

		if(ObjectsToDelete.Num() > 0)
		{
			// Remove all objects to be deleted from action's and Dataprep's context
			for(UObject* Object : ObjectsToDelete)
			{
				if (AActor* Actor = Cast<AActor>(Object))
				{
					if (UWorld* World = Actor->GetWorld())
					{
						World->EditorDestroyActor( Actor, false );
					}
				}

				FDataprepCoreUtils::MoveToTransientPackage( Object );
				if(SelectedObjectSet.Contains(Object))
				{
					SelectedObjectSet.Remove( Object );

					// If object is an asset, remove from array of assets. 
					if(FDataprepCoreUtils::IsAsset(Object))
					{
						bAssetsChanged = true;
						ContextPtr->Assets.Remove( Object );
					}
					else
					{
						bWorldChanged = true;
					}
				}
			}

			FDataprepCoreUtils::PurgeObjects( MoveTemp(ObjectsToDelete) );
		}

		// Build new assets and rebuild modified ones
		DataprepCorePrivateUtils::ClearAssets( ModifiedAssets.Array() );
		ModifiedAssets.Reserve( ModifiedAssets.Num() + AddedObjects.Num() );
		for(UObject* Object : AddedObjects)
		{
			ModifiedAssets.Add( Object );
		}
		FDataprepCoreUtils::BuildAssets( ModifiedAssets.Array(), ContextPtr->ProgressReporterPtr );

		// Update action's context
		OperationContext->Context->Objects = SelectedObjectSet.Array();

		if( ContextPtr->ContextChangedCallback && (bAssetsChanged || bWorldChanged))
		{
			ContextPtr->ContextChangedCallback( this, bWorldChanged, bAssetsChanged, ContextPtr->Assets.Array() );
		}

		bWorkingSetHasChanged = false;
		ModifiedAssets.Empty();
		AddedObjects.Empty();
	}

	bWorkingSetHasChanged = false;
}

UObject * UDataprepActionAsset::GetAssetOuterByClass(UClass* AssetClass)
{
	if(AssetClass == nullptr)
	{
		return nullptr;
	}

	UPackage* Package = nullptr;

	if( AssetClass->IsChildOf( UStaticMesh::StaticClass() ) )
	{
		Package = PackageForStaticMesh.Get();
		if(Package == nullptr)
		{
			PackageForStaticMesh = TWeakObjectPtr< UPackage >( NewObject< UPackage >( nullptr, *FPaths::Combine( ContextPtr->TransientContentFolder, TEXT("Geometries") ), RF_Transient ) );
			PackageForStaticMesh->FullyLoad();
			Package = PackageForStaticMesh.Get();
		}
	}
	else if( AssetClass->IsChildOf( UMaterialInterface::StaticClass() ) )
	{
		Package = PackageForMaterial.Get();
		if(Package == nullptr)
		{
			PackageForMaterial = TWeakObjectPtr< UPackage >( NewObject< UPackage >( nullptr, *FPaths::Combine( ContextPtr->TransientContentFolder, TEXT("Materials") ), RF_Transient ) );
			PackageForMaterial->FullyLoad();
			Package = PackageForMaterial.Get();
		}
	}
	else if( AssetClass->IsChildOf( UTexture::StaticClass() ) )
	{
		Package = PackageForTexture.Get();
		if(Package == nullptr)
		{
			PackageForTexture = TWeakObjectPtr< UPackage >( NewObject< UPackage >( nullptr, *FPaths::Combine( ContextPtr->TransientContentFolder, TEXT("Textures") ), RF_Transient ) );
			PackageForTexture->FullyLoad();
			Package = PackageForTexture.Get();
		}
	}
	else if( AssetClass->IsChildOf( ULevelSequence::StaticClass() ) )
	{
		Package = PackageForAnimation.Get();
		if(Package == nullptr)
		{
			PackageForAnimation = TWeakObjectPtr< UPackage >( NewObject< UPackage >( nullptr, *FPaths::Combine( ContextPtr->TransientContentFolder, TEXT("Animations") ), RF_Transient ) );
			PackageForAnimation->FullyLoad();
			Package = PackageForAnimation.Get();
		}
	}

	return Package;
}
