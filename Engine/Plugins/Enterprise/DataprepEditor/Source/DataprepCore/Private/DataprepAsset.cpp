// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepAsset.h"

#include "DataprepContentConsumer.h"
#include "DataprepCoreLogCategory.h"
#include "Logging/TokenizedMessage.h"
#include "Shared/DataprepCorePrivateUtils.h"
#include "Parameterization/DataprepParameterization.h"



#include "Templates/Greater.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "DataprepAsset"

// UDataprepAsset =================================================================

void UDataprepAsset::PostLoad()
{
	UDataprepAssetInterface::PostLoad();

	// Move content of deprecated properties to the corresponding new ones.
	if(HasAnyFlags(RF_WasLoaded))
	{
		bool bMarkDirty = false;
		if(Producers_DEPRECATED.Num() > 0)
		{
			Inputs->AssetProducers.Reserve(Producers_DEPRECATED.Num());

			while(Producers_DEPRECATED.Num() > 0)
			{
				if(Inputs->AddAssetProducer( Producers_DEPRECATED.Pop(EAllowShrinking::No) ) == INDEX_NONE)
				{
					// #ueent_todo Log message a producer was not properly restored
				}
			}

			Producers_DEPRECATED.Empty();
			bMarkDirty = true;
		}

		if(Consumer_DEPRECATED)
		{
			Output = Consumer_DEPRECATED;
			Consumer_DEPRECATED = nullptr;
			bMarkDirty = true;
		}

		// Most likely a Dataprep asset from 4.24
		if(DataprepRecipeBP_DEPRECATED != nullptr)
		{
			DataprepRecipeBP_DEPRECATED = nullptr;
			StartNode_DEPRECATED = nullptr;
			bMarkDirty = true;

			// Actions must change owner, but it cannot be done in the PostLoad.
			// Register to be called back when FCoreUObjectDelegates::OnAssetLoad broadcasts
			// so the change of ownership can be done
			FCoreUObjectDelegates::OnAssetLoaded.AddUObject(this, &UDataprepAsset::OnOldAssetLoaded);
		}

		if ( !Parameterization )
		{
			Parameterization = NewObject<UDataprepParameterization>( this, FName(), RF_Public | RF_Transactional );
			bMarkDirty = true;
		}

		if(bMarkDirty)
		{
			// Inform user asset has changed and needs to be saved
			const FText AssetName = FText::FromString( GetName() );
			const FText WarningMessage = FText::Format( LOCTEXT( "DataprepAssetOldVersion", "{0} is from an old version and has been updated. Please save asset to complete update."), AssetName );
			const FText NotificationText = FText::Format( LOCTEXT( "DataprepAssetOldVersionNotif", "{0} is from an old version and has been updated."), AssetName );
			DataprepCorePrivateUtils::LogMessage( EMessageSeverity::Warning, WarningMessage, NotificationText );
		}
	}
}

bool UDataprepAsset::Rename(const TCHAR* NewName/* =nullptr */, UObject* NewOuter/* =nullptr */, ERenameFlags Flags/* =REN_None */)
{
	bool bWasRename = Super::Rename( NewName, NewOuter, Flags );
	if ( bWasRename )
	{
		if ( Parameterization )
		{
			bWasRename &= Parameterization->OnAssetRename( Flags );
		}
	}

	return bWasRename;
}

FString UDataprepAsset::HashActionsAppearance() const
{
	TArray<UDataprepActionAsset*> TempArrayActions(ActionAssets);
	Algo::Sort(TempArrayActions);

	TArray<int32> TempArrayGroups;
	TArray<bool> TempArrayGroupsEnabled;
	TempArrayGroups.Reserve(TempArrayActions.Num());
	TempArrayGroupsEnabled.Reserve(TempArrayActions.Num());
	for (UDataprepActionAsset* Action : TempArrayActions)
	{
		UDataprepActionAppearance* Appearance = Action->GetAppearance();
		TempArrayGroups.Add(Appearance->GroupId);
		TempArrayGroupsEnabled.Add(Appearance->bGroupIsEnabled);
	}
	TArray<FVector2D> TempArrayNodeSizes;
	TempArrayNodeSizes.Reserve(TempArrayActions.Num());
	for (UDataprepActionAsset* Action : TempArrayActions)
	{
		TempArrayNodeSizes.Add(Action->GetAppearance()->NodeSize);
	}

	uint8 Digest[16];

	FMD5 AppearanceHash;
	AppearanceHash.Update((uint8*)TempArrayGroups.GetData(), TempArrayGroups.Num() * sizeof(int32));
	AppearanceHash.Update((uint8*)TempArrayGroupsEnabled.GetData(), TempArrayGroupsEnabled.Num() * sizeof(bool));
	AppearanceHash.Update((uint8*)TempArrayNodeSizes.GetData(), TempArrayNodeSizes.Num() * sizeof(FVector2D));
	AppearanceHash.Final(Digest);

	FString Result;
	for (int32 i = 0; i < 16; i++)
	{
		Result += FString::Printf(TEXT("%02x"), Digest[i]);
	}
	
	return Result;
}

void UDataprepAsset::PreEditUndo()
{
	UDataprepAssetInterface::PreEditUndo();

	// Cache signature of ordered array of actions before Undo/Redo
	TArray<UDataprepActionAsset*> TempArray(ActionAssets);
	Algo::Sort(TempArray);

	SignatureBeforeUndoRedo = FMD5::HashBytes((uint8*)TempArray.GetData(), TempArray.Num() * sizeof(UDataprepActionAsset*));

	AppearanceSignatureBeforeUndoRedo = HashActionsAppearance();
}

void UDataprepAsset::PostEditUndo()
{
	UDataprepAssetInterface::PostEditUndo();

	// Compute signature of ordered array of actions after Undo/Redo
	TArray<UDataprepActionAsset*> TempArray(ActionAssets);
	Algo::Sort(TempArray);

	const FString SignatureAfterUndoRedo = FMD5::HashBytes((uint8*)TempArray.GetData(), TempArray.Num() * sizeof(UDataprepActionAsset*));
	const FString AppearanceSignatureAfterUndoRedo = HashActionsAppearance();

	if(!SignatureBeforeUndoRedo.IsEmpty())
	{
		if (SignatureAfterUndoRedo != SignatureBeforeUndoRedo)
		{
			OnActionChanged.Broadcast(nullptr, FDataprepAssetChangeType::ActionRemoved);
		}
		else
		{
			// Moved or appearance changed (includes grouping)
			OnActionChanged.Broadcast(nullptr, (AppearanceSignatureBeforeUndoRedo != AppearanceSignatureAfterUndoRedo) ? FDataprepAssetChangeType::ActionAppearanceModified : FDataprepAssetChangeType::ActionMoved);
		}
	}
	else
	{
		ensure(false);
	}

	SignatureBeforeUndoRedo.Reset(0);
	AppearanceSignatureBeforeUndoRedo.Reset(0);
}

void UDataprepAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	UDataprepAssetInterface::PostDuplicate(DuplicateMode);

	if (Output)
	{
		// Make sure the output level does not override the one from the original
		FText OutReason;
		Output->SetLevelName(GetName() + "_MAP", OutReason);
	}
}

const UDataprepActionAsset* UDataprepAsset::GetAction(int32 Index) const
{
	if ( ActionAssets.IsValidIndex( Index ) )
	{
		return ActionAssets[Index];
	}
	else
	{
		UE_LOGF( LogDataprepCore
			, Error
			, "The action to retrieve is out of bound. (Passed index: %d, Number of actions: %d, Dataprepsset: %ls)"
			, Index
			, ActionAssets.Num()
			, *GetPathName()
			);
	}

	return nullptr;
}

bool UDataprepAsset::CreateParameterization()
{
	if( !Parameterization )
	{
		Parameterization = NewObject<UDataprepParameterization>( this, FName(), RF_Public | RF_Transactional );
		MarkPackageDirty();
		return true;
	}

	return false;
}

void UDataprepAsset::ExecuteRecipe(const TSharedPtr<FDataprepActionContext>& InActionsContext)
{
	ExecuteRecipe_Internal(	InActionsContext, ActionAssets );
}

TArray<UDataprepActionAsset*> UDataprepAsset::GetCopyOfActions(TMap<UObject*,UObject*>& OutOriginalToCopy) const
{
	TArray<UDataprepActionAsset*> CopyOfActionAssets;
	CopyOfActionAssets.Reserve( ActionAssets.Num() );
	for ( UDataprepActionAsset* ActionAsset : ActionAssets )
	{
		FObjectDuplicationParameters DuplicationParameter( ActionAsset, GetTransientPackage() );
		DuplicationParameter.CreatedObjects = &OutOriginalToCopy;

		UDataprepActionAsset* CopyOfAction = static_cast<UDataprepActionAsset*>( StaticDuplicateObjectEx( DuplicationParameter ) );
		CopyOfAction->SetFlags(EObjectFlags::RF_Transactional);
		check( CopyOfAction );

		OutOriginalToCopy.Add( ActionAsset, CopyOfAction );
		CopyOfActionAssets.Add( CopyOfAction );
	}

	return CopyOfActionAssets;
}

UObject* UDataprepAsset::GetParameterizationObject()
{
	return Parameterization->GetDefaultObject();
}

void UDataprepAsset::BindObjectPropertyToParameterization(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain, const FName& Name)
{
	bool bPassConditionCheck = false;

	if ( InPropertyChain.Num() > 0 )
	{
		// Validate that the object is part of this asset
		UObject* Outer = Object;
		while ( Outer && !bPassConditionCheck )
		{
			Outer =  Outer->GetOuter();
			bPassConditionCheck = Outer == this;
		}
	}

	if ( bPassConditionCheck )
	{
		Parameterization->BindObjectProperty( Object, InPropertyChain, Name );
	}

}

bool UDataprepAsset::IsObjectPropertyBinded(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain) const
{
	return Parameterization->IsObjectPropertyBinded( Object, InPropertyChain );
}

FName UDataprepAsset::GetNameOfParameterForObjectProperty(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain) const
{
	return Parameterization->GetNameOfParameterForObjectProperty( Object, InPropertyChain );
}

void UDataprepAsset::RemoveObjectPropertyFromParameterization(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain)
{
	Parameterization->RemoveBindedObjectProperty( Object, InPropertyChain );
}

void UDataprepAsset::GetExistingParameterNamesForType(FProperty* Property, bool bIsDescribingFullProperty, TSet<FString>& OutValidExistingNames, TSet<FString>& OutInvalidNames) const
{
	Parameterization->GetExistingParameterNamesForType( Property, bIsDescribingFullProperty, OutValidExistingNames, OutInvalidNames );
}

void UDataprepAsset::FRestrictedToActionAsset::NotifyAssetOfTheRemovalOfSteps(UDataprepAsset& DataprepAsset, const TArrayView<UDataprepParameterizableObject*>& StepObjects)
{
	if ( DataprepAsset.Parameterization )
	{
		DataprepAsset.Parameterization->RemoveBindingFromObjects( StepObjects );
	}

	DataprepAsset.OnStepObjectsAboutToBeRemoved.Broadcast( StepObjects );
}

int32 UDataprepAsset::AddAction(const UDataprepActionAsset* InAction)
{
	UDataprepActionAsset* Action = InAction ? DuplicateObject<UDataprepActionAsset>( InAction, this) : NewObject<UDataprepActionAsset>( this, UDataprepActionAsset::StaticClass(), NAME_None, RF_Transactional );

	if ( Action )
	{
		Modify();

		Action->SetFlags(EObjectFlags::RF_Transactional);
		Action->SetLabel( InAction ? InAction->GetLabel() : TEXT("New Action") );

		ActionAssets.Add( Action );
		OnActionChanged.Broadcast(Action, FDataprepAssetChangeType::ActionAdded);

		return ActionAssets.Num() - 1;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::AddAction: The action is invalid" );
	ensure(false);

	// Invalid
	return INDEX_NONE;
}

int32 UDataprepAsset::AddActions(const TArray<const UDataprepActionAsset*>& InActions)
{
	if ( InActions.Num() > 0 && InActions[0] != nullptr )
	{
		Modify();

		int32 PreviousActionCount = ActionAssets.Num();

		for(const UDataprepActionAsset* InAction : InActions)
		{
			if(InAction)
			{
				UDataprepActionAsset* Action = DuplicateObject<UDataprepActionAsset>( InAction, this);
				Action->SetFlags(EObjectFlags::RF_Transactional);
				Action->SetLabel( InAction->GetLabel() );

				if( Action->GetAppearance()->GroupId != INDEX_NONE )
				{
					int32 InsertIndex = INDEX_NONE;

					// Maintain the correct grouping (actions with the same group need to be tightly packed)
					for( int32 ActionIndex = 0; ActionIndex < ActionAssets.Num(); ++ActionIndex )
					{
						if( ActionAssets[ActionIndex]->GetAppearance()->GroupId == Action->GetAppearance()->GroupId )
						{
							InsertIndex = ActionIndex + 1;
						}
						else if( InsertIndex != INDEX_NONE )
						{
							break;
						}
					}

					if( InsertIndex != INDEX_NONE )
					{
						ActionAssets.Insert( Action, InsertIndex );
					}
					else
					{
						ActionAssets.Add( Action );
					}
				}
				else
				{
					ActionAssets.Add( Action );
				}
			}
		}

		if(PreviousActionCount != ActionAssets.Num())
		{
			OnActionChanged.Broadcast(ActionAssets.Last(), FDataprepAssetChangeType::ActionAdded);

			return ActionAssets.Num() - 1;
		}
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::AddActions: None of the action steps is invalid" );
	ensure(false);

	// Invalid
	return INDEX_NONE;
}

int32 UDataprepAsset::AddAction(const TArray<const UDataprepActionStep*>& InActionSteps)
{
	if ( InActionSteps.Num() > 0 && InActionSteps[0] != nullptr )
	{
		Modify();

		UDataprepActionAsset* Action = NewObject<UDataprepActionAsset>(this, UDataprepActionAsset::StaticClass(), NAME_None, RF_Transactional);
		Action->SetLabel(TEXT("New Action"));

		ActionAssets.Add(Action);

		Action->AddSteps(InActionSteps);

		OnActionChanged.Broadcast(Action, FDataprepAssetChangeType::ActionAdded);

		return ActionAssets.Num() - 1;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::AddActionSteps: None of the action steps is invalid" );
	ensure(false);

	// Invalid
	return INDEX_NONE;
}

bool UDataprepAsset::InsertAction(const UDataprepActionAsset* InAction, int32 Index)
{
	if( !( Index >= 0 && Index <= ActionAssets.Num() ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::InsertAction: The index is invalid" );
		return false;
	}

	if ( InAction )
	{
		Modify();

		UDataprepActionAsset* Action = DuplicateObject<UDataprepActionAsset>( InAction, this );
		Action->SetFlags( EObjectFlags::RF_Transactional );
		Action->SetLabel( InAction->GetLabel() );

		ActionAssets.Insert( Action, Index );

		OnActionChanged.Broadcast( Action, FDataprepAssetChangeType::ActionAdded );

		return true;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::InsertAction: The action is invalid" );
	ensure(false);

	// Invalid
	return false;
}

bool UDataprepAsset::InsertActions(const TArray<const UDataprepActionAsset*>& InActions, int32 Index)
{
	if(!ActionAssets.IsValidIndex(Index))
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::InsertActions: The index is invalid" );
		return false;
	}

	if ( InActions.Num() > 0 && InActions[0] != nullptr )
	{
		Modify();

		int32 PreviousActionCount = ActionAssets.Num();

		int32 InsertIndex = Index;
		UDataprepActionAsset* Action = nullptr;

		for(const UDataprepActionAsset* InAction : InActions)
		{
			if(InAction)
			{
				Action = DuplicateObject<UDataprepActionAsset>( InAction, this);
				Action->SetFlags(EObjectFlags::RF_Transactional);
				Action->SetLabel( InAction->GetLabel() );

				ActionAssets.Insert( Action, InsertIndex );

				++InsertIndex;
			}
		}

		if(Action != nullptr)
		{
			OnActionChanged.Broadcast(Action, FDataprepAssetChangeType::ActionAdded);

			return true;
		}
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::InsertActions: one of the actions is invalid" );
	ensure(false);

	// Invalid
	return false;
}

bool UDataprepAsset::InsertAction(int32 Index)
{
	if ( Index >= 0 && Index <= ActionAssets.Num() )
	{
		Modify();

		UDataprepActionAsset* Action = NewObject<UDataprepActionAsset>( this, UDataprepActionAsset::StaticClass(), NAME_None, RF_Transactional );
		Action->SetLabel( TEXT("New Action") );

		ActionAssets.Insert( Action, Index );

		OnActionChanged.Broadcast( Action, FDataprepAssetChangeType::ActionAdded );
		return true;
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::AddAction: The index is invalid" );
	ensure(false);
	return false;
}

bool UDataprepAsset::InsertAction(const TArray<const UDataprepActionStep*>& InActionSteps, int32 Index)
{
	if ( InActionSteps.Num() > 0 && InActionSteps[0] != nullptr )
	{
		if( Index >= 0 && Index <= ActionAssets.Num() )
		{
			Modify();

			int32 PreviousActionCount = ActionAssets.Num();

			UDataprepActionAsset* Action = NewObject<UDataprepActionAsset>( this, UDataprepActionAsset::StaticClass(), NAME_None, RF_Transactional );
			Action->SetLabel( TEXT("New Action") );

			ActionAssets.Insert( Action, Index );

			Action->AddSteps( InActionSteps );

			OnActionChanged.Broadcast( Action, FDataprepAssetChangeType::ActionAdded );
			return true;
		}
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::InsertAction: One of the action steps is invalid or the index is invalid" );
	ensure(false);

	// Invalid
	return false;
}

bool UDataprepAsset::MoveAction(int32 SourceIndex, int32 DestinationIndex)
{
	if ( SourceIndex == DestinationIndex )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: Nothing done. Moving to current location" );
		return true;
	}

	if ( !ActionAssets.IsValidIndex( SourceIndex ) || !ActionAssets.IsValidIndex( DestinationIndex ) )
	{
		if ( !ActionAssets.IsValidIndex( SourceIndex ) )
		{
			UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: The source index is out of range" );
		}

		if ( !ActionAssets.IsValidIndex( DestinationIndex ) )
		{
			UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: The destination index is out of range" );
		}
		return false;
	}

	Modify();

	if ( DataprepCorePrivateUtils::MoveArrayElement( ActionAssets, SourceIndex, DestinationIndex ) )
	{
		OnActionChanged.Broadcast(ActionAssets[DestinationIndex], FDataprepAssetChangeType::ActionMoved);
		return true;
	}

	ensure( false );
	return false;
}

bool UDataprepAsset::MoveActions(int32 FirstIndex, int32 Count, int32 MovePositions)
{
	if ( MovePositions == 0 )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: Nothing done. Moving to current location" );
		return true;
	}

	if ( !ActionAssets.IsValidIndex( FirstIndex ) || !ActionAssets.IsValidIndex( FirstIndex + Count - 1 ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: The source index is out of range" );
		return false;
	}
	else if ( ( MovePositions > 0 && !ActionAssets.IsValidIndex( FirstIndex + Count - 1 + MovePositions ) ) || !ActionAssets.IsValidIndex( FirstIndex + MovePositions ) )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::MoveAction: The destination index is out of range" );
		return false;
	}

	Modify();

	if (MovePositions > 0)
	{
		for (int SourceIndex = FirstIndex + Count - 1; SourceIndex >= FirstIndex; --SourceIndex)
		{
			DataprepCorePrivateUtils::MoveArrayElement( ActionAssets, SourceIndex, SourceIndex + MovePositions );
		}
	}
	else
	{
		for (int SourceIndex = FirstIndex; SourceIndex < FirstIndex + Count; ++SourceIndex)
		{
			DataprepCorePrivateUtils::MoveArrayElement( ActionAssets, SourceIndex, SourceIndex + MovePositions );
		}
	}
	
	OnActionChanged.Broadcast(ActionAssets[FirstIndex + MovePositions], FDataprepAssetChangeType::ActionMoved);

	return true;
}

bool UDataprepAsset::SwapActions(int32 FirstActionIndex, int32 SecondActionIndex)
{
	if ( FirstActionIndex == SecondActionIndex )
	{
		UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::SwapAction: Nothing done. First and second indices are identical" );
		return true;
	}

	if ( !ActionAssets.IsValidIndex( FirstActionIndex ) || !ActionAssets.IsValidIndex( SecondActionIndex ) )
	{
		if ( !ActionAssets.IsValidIndex( FirstActionIndex ) )
		{
			UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::SwapAction: The first index is out of range" );
		}

		if ( !ActionAssets.IsValidIndex( SecondActionIndex ) )
		{
			UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::SwapAction: The second index is out of range" );
		}

		return false;
	}

	Modify();

	// Directly use TArray::SwapMemory since we have validated the indices
	ActionAssets.SwapMemory(FirstActionIndex, SecondActionIndex);

	OnActionChanged.Broadcast(ActionAssets[FirstActionIndex], FDataprepAssetChangeType::ActionMoved);

	return true;
}

bool UDataprepAsset::RemoveAction(int32 Index, bool bDiscardParametrization)
{
	if ( ActionAssets.IsValidIndex( Index ) )
	{
		Modify();

		UDataprepActionAsset* ActionAsset = ActionAssets[Index];
		if(ActionAsset && bDiscardParametrization)
		{
			ActionAsset->NotifyDataprepSystemsOfRemoval();
		}

		ActionAssets.RemoveAt( Index );

		OnActionChanged.Broadcast(ActionAsset, FDataprepAssetChangeType::ActionRemoved);

		return true;
	}

	ensure( false );
	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::RemoveAction: The Index is out of range" );

	return false;
}

bool UDataprepAsset::RemoveActions(const TArray<int32>& Indices, bool bDiscardParametrization)
{
	bool bHasValidIndices = false;
	for(int32 Index : Indices)
	{
		if(ActionAssets.IsValidIndex( Index ))
		{
			bHasValidIndices = true;
			break;
		}
	}

	if ( bHasValidIndices )
	{
		Modify();

		// Used to cache last action removed
		UDataprepActionAsset* ActionAsset = nullptr;

		// Sort array in reverse order before removal
		TArray<int32> LocalIndices = Indices;
		LocalIndices.Sort(TGreater<int32>());

		// Now safe to use TArray::RemoveAt
		for(int32 Index : LocalIndices)
		{
			if(ActionAssets.IsValidIndex( Index ))
			{
				ActionAsset = ActionAssets[Index];
				if(ActionAsset && bDiscardParametrization)
				{
					ActionAsset->NotifyDataprepSystemsOfRemoval();
				}

				ActionAssets.RemoveAt(Index);
			}
		}

		// Notify on last action removed
		OnActionChanged.Broadcast(ActionAsset, FDataprepAssetChangeType::ActionRemoved);

		return true;
	}

	ensure( false );
	UE_LOGF( LogDataprepCore, Error, "UDataprepAsset::RemoveActions: None of the indices are in range" );

	return false;
}

void UDataprepAsset::OnOldAssetLoaded(UObject* Asset)
{
	if(Asset == this)
	{
		// Move ownership from the deprecated K2Node to the Dataprep asset itself
		for(UDataprepActionAsset* ActionAsset : ActionAssets)
		{
			if(ActionAsset && ActionAsset->GetOuter() != this)
			{
				ActionAsset->Rename(nullptr, this, REN_NonTransactional | REN_DontCreateRedirectors);
			}
		}

		// Unregister to the OnAssetLoad event as it is not needed anymore
		FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
	}
}

#undef LOCTEXT_NAMESPACE
