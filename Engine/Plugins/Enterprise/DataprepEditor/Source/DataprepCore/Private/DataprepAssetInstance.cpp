// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepAssetInstance.h"

#include "DataprepAsset.h"
#include "DataprepContentConsumer.h"
#include "DataprepCoreLogCategory.h"
#include "Shared/DataprepCorePrivateUtils.h"
#include "Parameterization/DataprepParameterization.h"
#include "UObject/Package.h"

void UDataprepAssetInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Check if parenting has changed, either through a force delete on the parent or a replace on deletion
	bool bParentHasChanged = PropertyChangedEvent.ChangeType == EPropertyChangeType::Redirected && PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == TEXT("Parent");
	
	if(bParentHasChanged)
	{
		if(Parent != nullptr)
		{
			UE_LOGF( LogDataprepCore, Error, "UDataprepAssetInstance::PostEditChangeProperty: Reparenting to non null parent is not supported yet." );
			Parent = nullptr;
		}

		// Recreate the ParameterizationInstance property
		Parameterization = NewObject<UDataprepParameterizationInstance>( this, NAME_None, RF_Public | RF_Transactional );

		// Assign a temporary source parameterization
		UDataprepParameterization* SourceParameterization = NewObject<UDataprepParameterization>( GetTransientPackage(), FName(), RF_Public );
		Parameterization->SetParameterizationSource( *SourceParameterization );

		ActionsFromDataprepAsset.Empty();
	}

	UDataprepAssetInterface::PostEditChangeProperty( PropertyChangedEvent );
}

// UDataprepAssetInstance =================================================================

void UDataprepAssetInstance::ExecuteRecipe(const TSharedPtr<FDataprepActionContext>& InActionsContext)
{
	if(Parent != nullptr)
	{
		// Doing the parameterization
		TMap<UObject*,UObject*> SourceToCopy;
		ActionsFromDataprepAsset = GetCopyOfActions( SourceToCopy );
		Parameterization->ApplyParameterization( SourceToCopy );

		ExecuteRecipe_Internal( InActionsContext, ActionsFromDataprepAsset );

		ActionsFromDataprepAsset.Empty();
	}
}

UObject* UDataprepAssetInstance::GetParameterizationObject()
{
	return Parameterization->GetParameterizationInstance();
}

TArray<UDataprepActionAsset*> UDataprepAssetInstance::GetCopyOfActions(TMap<UObject*,UObject*>& OutOriginalToCopy) const
{
	return Parent ? Parent->GetCopyOfActions( OutOriginalToCopy ) : TArray<UDataprepActionAsset*>();
}

const TArray<UDataprepActionAsset*>& UDataprepAssetInstance::GetActions() const
{
	static TArray<UDataprepActionAsset*> Dummy;

	return Parent ? Parent->GetActions() : Dummy;
}


bool UDataprepAssetInstance::SetParent(UDataprepAssetInterface* InParent, bool bNotifyChanges )
{
	// #ueent_remark: Setting a null parent is not supported yet.
	check(InParent);

	if(bNotifyChanges)
	{
		Modify();
	}

	// Copy set of producers if the instance does not have any yet
	if(Inputs == nullptr || Inputs->GetProducersCount() == 0)
	{
		if(Inputs != nullptr)
		{
			Inputs->GetOnChanged().RemoveAll( this );
			DataprepCorePrivateUtils::DeleteRegisteredAsset( Inputs );
		}

		Inputs = DuplicateObject<UDataprepAssetProducers>( InParent->GetProducers(), this );
	}

	// Copy consumer if one is not set yet
	if(Output == nullptr)
	{
		Output = DuplicateObject<UDataprepContentConsumer>( InParent->GetConsumer(), this );
	}

	// For the time being an instance of instance is not supported
	Parent = GetRootParent( InParent );
	
	Parameterization = NewObject<UDataprepParameterizationInstance>( this, NAME_None, RF_Public | RF_Transactional );

	// #ueent_nextversion: Get parameterization from root parent and copy values from actual parent if parent is an instance
	if( UDataprepAsset* RootParent = Cast<UDataprepAsset>(Parent) )
	{
		Parameterization->SetParameterizationSource( *( RootParent->GetDataprepParameterization() ) );
	}
	else
	{
		// Assign a temporary source parameterization
		UDataprepParameterization* SourceParameterization = NewObject<UDataprepParameterization>( GetTransientPackage(), FName(), RF_Public );
		Parameterization->SetParameterizationSource( *SourceParameterization );
	}

	if(bNotifyChanges)
	{
		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified );
		OnChanged.Broadcast( FDataprepAssetChangeType::ConsumerModified );
		OnParentChanged.Broadcast();
	}

	return true;
}

UDataprepAsset* UDataprepAssetInstance::GetRootParent(UDataprepAssetInterface* InParent)
{
	UDataprepAsset* RootParent = Cast<UDataprepAsset>(InParent);

	while(RootParent == nullptr)
	{
		if(UDataprepAssetInstance* InstanceOfInstance = Cast<UDataprepAssetInstance>(InParent))
		{
			if(InstanceOfInstance->Parent == nullptr)
			{
				break;
			}

			InParent = Cast<UDataprepAssetInstance>(InstanceOfInstance->Parent);
			RootParent = Cast<UDataprepAsset>(InParent);
		}
	}

	return RootParent;
}
