// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepAssetInterface.h"

#include "DataprepActionAsset.h"
#include "DataprepAssetProducers.h"
#include "DataprepContentConsumer.h"
#include "DataprepCoreLogCategory.h"
#include "IDataprepProgressReporter.h"
#include "Shared/DataprepCorePrivateUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"


#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DataprepAssetInterface"

// UDataprepAssetInterface =================================================================

UDataprepAssetInterface::UDataprepAssetInterface()
{
#if WITH_EDITORONLY_DATA
	Inputs = nullptr;
	Output = nullptr;
	Recipe = nullptr;
#endif
}

void UDataprepAssetInterface::PostLoad()
{
	Super::PostLoad();

	// Initialize Inputs property for Dataprep assets created before the introduction of Dataprep asset interface
	if(Inputs == nullptr)
	{
		Inputs = NewObject< UDataprepAssetProducers >( this, NAME_None, RF_Transactional );
		check( Inputs );

		FAssetRegistryModule::AssetCreated( Inputs );
		Inputs->MarkPackageDirty();
	}

	ensure(Inputs);
	Inputs->GetOnChanged().AddUObject(this, &UDataprepAssetInterface::OnProducersChanged);

	if(Output != nullptr)
	{
		Output->GetOnChanged().AddUObject( this, &UDataprepAssetInterface::OnConsumerChanged );
	}
	else
	{
		// Check if a consumer class has become available since last load
		for( TObjectIterator< UClass > It ; It ; ++It )
		{
			UClass* CurrentClass = (*It);

			if ( !CurrentClass->HasAnyClassFlags( CLASS_Abstract ) )
			{
				if( CurrentClass->IsChildOf( UDataprepContentConsumer::StaticClass() ) )
				{
					UE_LOGF( LogDataprepCore, Warning, "Default Dataprep consumer was assigned" );
					SetConsumer( CurrentClass );
					break;
				}
			}
		}
	}

	if(Recipe != nullptr)
	{
		Recipe->GetOnChanged().AddUObject( this, &UDataprepAssetInterface::OnRecipeChanged );
	}
}

void UDataprepAssetInterface::PostInitProperties()
{
	Super::PostInitProperties();

	// Initialize Inputs property
	if(!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		Inputs = NewObject< UDataprepAssetProducers >( this, NAME_None, RF_Transactional );
		check( Inputs );

		Inputs->GetOnChanged().AddUObject( this, &UDataprepAssetInterface::OnProducersChanged );

		FAssetRegistryModule::AssetCreated( Inputs );
		Inputs->MarkPackageDirty();
	}
}

UDataprepContentConsumer* UDataprepAssetInterface::SetConsumer(UClass* NewConsumerClass, bool bNotifyChanges)
{
	if( NewConsumerClass && NewConsumerClass->IsChildOf(UDataprepContentConsumer::StaticClass()))
	{
		if(Output != nullptr)
		{
			Output->GetOnChanged().RemoveAll( this );
			DataprepCorePrivateUtils::DeleteRegisteredAsset( Output );
		}

		Modify();

		FString BaseName = GetName() + TEXT("_Consumer");
		FName ConsumerName = MakeUniqueObjectName( this, NewConsumerClass, *BaseName );
		Output = NewObject< UDataprepContentConsumer >( this, NewConsumerClass, ConsumerName, RF_Transactional );
		check( Output );

		// Set the name of the output level
		FText OutText;
		Output->SetLevelName(GetName() + TEXT("_Map"), OutText);

		FAssetRegistryModule::AssetCreated( Output );
		Output->MarkPackageDirty();

		Output->GetOnChanged().AddUObject( this, &UDataprepAssetInterface::OnConsumerChanged );

		if(bNotifyChanges)
		{
			OnChanged.Broadcast( FDataprepAssetChangeType::ConsumerModified );
		}

		return Output;
	}

	return nullptr;
}

TArray< TWeakObjectPtr< UObject > > UDataprepAssetInterface::RunProducers(const FDataprepProducerContext& InContext)
{
	return Inputs->Produce(InContext);
}

bool UDataprepAssetInterface::RunConsumer( const FDataprepConsumerContext& InContext )
{
	return Output ? Output->Consume( InContext ) : false;
}

void UDataprepAssetInterface::OnConsumerChanged()
{
	if(Output)
	{
		MarkPackageDirty();

		// Relay change notification to observers of this object
		OnChanged.Broadcast( FDataprepAssetChangeType::ConsumerModified );
	}
}

void UDataprepAssetInterface::OnProducersChanged( FDataprepAssetChangeType ChangeType, int32 /* Index */)
{
	MarkPackageDirty();

	// Relay change notification to observers of this object
	OnChanged.Broadcast( ChangeType );
}

void UDataprepAssetInterface::OnRecipeChanged()
{
	if(Recipe)
	{
		MarkPackageDirty();

		// Relay change notification to observers of this object
		OnChanged.Broadcast( FDataprepAssetChangeType::RecipeModified );
	}
}

void UDataprepAssetInterface::ExecuteRecipe_Internal(const TSharedPtr<FDataprepActionContext>& InActionsContext, const TArray<UDataprepActionAsset*>& ActionAssets)
{
	if (ActionAssets.Num() == 0)
	{
		return;
	}

	FDataprepWorkReporter Task(InActionsContext->ProgressReporterPtr, LOCTEXT("RunActions", "Executing pipeline ..."), (float)ActionAssets.Num(), 1.0f);

	for (UDataprepActionAsset* ActionAsset : ActionAssets)
	{
		if (ActionAsset != nullptr)
		{
			const bool bGroupEnabled = ActionAsset->GetAppearance()->GroupId != INDEX_NONE ? ActionAsset->GetAppearance()->bGroupIsEnabled : true;
		
			if (ActionAsset->bIsEnabled && bGroupEnabled)
			{
				Task.ReportNextStep(FText::Format(LOCTEXT("ExecutingAction", "Executing \"{0}\" ..."), FText::FromString(ActionAsset->GetLabel())));

				ActionAsset->ExecuteAction(InActionsContext);
				if (Task.IsWorkCancelled())
				{
					Task.ReportNextStep(LOCTEXT("InterruptedExecution", "Execution interrupted ..."));
					break;
				}
			}
		}
		else
		{
			Task.ReportNextStep(LOCTEXT("SkippingAction", "Skipping null action ..."));
		}
	}
}

const TArray<UDataprepActionAsset*>& UDataprepAssetInterface::GetActions() const
{
	static TArray<UDataprepActionAsset*> Dummy;

	unimplemented();

	return Dummy; 
}

#undef LOCTEXT_NAMESPACE
