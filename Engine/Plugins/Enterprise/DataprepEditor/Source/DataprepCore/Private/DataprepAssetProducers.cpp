// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepAssetProducers.h"

#include "DataprepAssetInterface.h"
#include "DataprepContentProducer.h"
#include "DataprepCoreLogCategory.h"
#include "Shared/DataprepCorePrivateUtils.h"
#include "DataprepCoreUtils.h"

#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "DataprepAssetProducers"

// UDataprepAssetProducers =================================================================

void UDataprepAssetProducers::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	if( Ar.IsLoading() )
	{
		for( FDataprepAssetProducer& Producer : AssetProducers )
		{
			if(Producer.Producer)
			{
				Producer.Producer->GetOnChanged().AddUObject( this, &UDataprepAssetProducers::OnProducerChanged );
			}
		}
	}
}

void UDataprepAssetProducers::PostEditUndo()
{
	Super::PostEditUndo();

	OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, INDEX_NONE );
}

TArray< TWeakObjectPtr< UObject > > UDataprepAssetProducers::Produce( const FDataprepProducerContext& InContext )
{
	if( AssetProducers.Num() == 0 )
	{
		return TArray< TWeakObjectPtr< UObject > >();
	}

	TArray< TWeakObjectPtr< UObject > > OutAssets;
	{
		FDataprepWorkReporter Task( InContext.ProgressReporterPtr, LOCTEXT( "ProducerReport_Importing", "Importing ..." ), (float)AssetProducers.Num(), 1.0f );

		TArray< TWeakObjectPtr< UObject > > ProducerAssets;

		for ( FDataprepAssetProducer& AssetProducer : AssetProducers )
		{
			if( UDataprepContentProducer* Producer = AssetProducer.Producer )
			{
				Task.ReportNextStep( FText::Format( LOCTEXT( "ProducerReport_ImportingX", "Importing {0} ..."), FText::FromString( Producer->GetName() ) ) );

				// Run producer if enabled and, if superseded, superseder is disabled
				const bool bIsOkToRun = AssetProducer.bIsEnabled &&	( AssetProducer.SupersededBy == INDEX_NONE || !AssetProducers[AssetProducer.SupersededBy].bIsEnabled );

				if ( bIsOkToRun )
				{
					if( !Producer->Produce( InContext, OutAssets ) )
					{
						if ( Task.IsWorkCancelled() )
						{
							FText Report = FText::Format(LOCTEXT( "ProducerReport_Cancelled", "Producer {0} was cancelled during execution."), FText::FromString( Producer->GetName() ) );
							InContext.LoggerPtr->LogInfo( Report, *Producer );
							break;
						}
						else
						{
							FText Report = FText::Format(LOCTEXT( "ProducerReport_Failed", "Producer {0} failed to execute."), FText::FromString( Producer->GetName() ) );
							InContext.LoggerPtr->LogError( Report, *Producer );
						}
					}
				}
			}
			else
			{
				Task.ReportNextStep( LOCTEXT( "ProducerReport_Skipped", "Skipped invalid producer ...") );
			}
		}

		// Skip finalization of import if user has canceled process
		if(IDataprepProgressReporter* ProgressReporter = InContext.ProgressReporterPtr.Get())
		{
			if(ProgressReporter->IsWorkCancelled())
			{
				TArray<UObject*> ObjectsToDelete;
				for ( TWeakObjectPtr<UObject> WeakObjectPtr : OutAssets )
				{
					if ( UObject* Object = WeakObjectPtr.Get() )
					{
						ObjectsToDelete.Add( Object );
					}
				}

				FDataprepCoreUtils::PurgeObjects( MoveTemp( ObjectsToDelete ) );

				OutAssets.Empty();

				return OutAssets;
			}
		}

		for ( TWeakObjectPtr<UObject> WeakObjectPtr : OutAssets )
		{
			if ( UObject* Object = WeakObjectPtr.Get() )
			{
				// We must ensure that public face of an asset is public
				Object->SetFlags( RF_Public );
			}
		}
	}

	FDataprepCoreUtils::BuildAssets( OutAssets, InContext.ProgressReporterPtr );

	return OutAssets;
}

UDataprepContentProducer* UDataprepAssetProducers::AddProducer(UClass* ProducerClass)
{
	constexpr bool bIsAutomated = false;
	return AddProducer_Internal( ProducerClass, bIsAutomated );
}

UDataprepContentProducer* UDataprepAssetProducers::AddProducerAutomated(UClass* ProducerClass)
{
	constexpr bool bIsAutomated = true;
	return AddProducer_Internal( ProducerClass, bIsAutomated );
}

UDataprepContentProducer* UDataprepAssetProducers::CopyProducer(const UDataprepContentProducer* InProducer)
{
	if( InProducer )
	{
		UDataprepContentProducer* Producer = DuplicateObject<UDataprepContentProducer>( InProducer, this );
		Producer->SetFlags( RF_Transactional );

		Producer->GetOnChanged().AddUObject(this, &UDataprepAssetProducers::OnProducerChanged);

		Producer->MarkPackageDirty();

		Modify();

		int32 ProducerNextIndex = AssetProducers.Num();
		AssetProducers.Emplace( Producer, true );

		bool bChangeAll = false;
		ValidateProducerChanges( ProducerNextIndex, bChangeAll );

		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerAdded, ProducerNextIndex );

		if(bChangeAll)
		{
			OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, INDEX_NONE );
		}

		return Producer;
	}

	return nullptr;
}

int32 UDataprepAssetProducers::AddAssetProducer(const FDataprepAssetProducer& AssetProducer)
{
	if( AssetProducer.Producer )
	{
		UDataprepContentProducer* Producer = DuplicateObject<UDataprepContentProducer>(AssetProducer.Producer, this);

		Producer->GetOnChanged().AddUObject(this, &UDataprepAssetProducers::OnProducerChanged);

		Producer->MarkPackageDirty();

		Modify();

		return AssetProducers.Emplace( Producer, AssetProducer.bIsEnabled, AssetProducer.SupersededBy );
	}

	return INDEX_NONE;
}

bool UDataprepAssetProducers::RemoveProducer(int32 IndexToRemove)
{
	if( AssetProducers.IsValidIndex( IndexToRemove ) )
	{
		// Update superseding status for producers depending on removed producer
		bool bChangeAll = false;

		// Array of producers superseded by removed producer
		TArray<int32> ProducersToRevisit;
		ProducersToRevisit.Reserve(AssetProducers.Num());

		{
			const FScopedTransaction Transaction(LOCTEXT("Producers_RemoveProducer", "Remove Producer"));

			Modify();

			if (UDataprepContentProducer* Producer = AssetProducers[IndexToRemove].Producer)
			{
				Producer->GetOnChanged().RemoveAll(this);

				Producer->PreEditChange(nullptr);
				Producer->Modify();

				DataprepCorePrivateUtils::DeleteRegisteredAsset(Producer);

				Producer->PostEditChange();
			}

			AssetProducers.RemoveAt(IndexToRemove);

			if (AssetProducers.Num() == 1)
			{
				AssetProducers[0].SupersededBy = INDEX_NONE;
			}
			else if (AssetProducers.Num() > 1)
			{
				// Update value stored in SupersededBy property where applicable
				for (int32 Index = 0; Index < AssetProducers.Num(); ++Index)
				{
					FDataprepAssetProducer& AssetProducer = AssetProducers[Index];

					if (AssetProducer.SupersededBy == IndexToRemove)
					{
						AssetProducer.SupersededBy = INDEX_NONE;
						ProducersToRevisit.Add(Index);
					}
					else if (AssetProducer.SupersededBy > IndexToRemove)
					{
						--AssetProducer.SupersededBy;
					}
				}
			}

			for (int32 ProducerIndex : ProducersToRevisit)
			{
				bool bLocalChangeAll = false;
				ValidateProducerChanges(ProducerIndex, bLocalChangeAll);
				bChangeAll |= bLocalChangeAll;
			}
		}

		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerRemoved, IndexToRemove );

		// Notify observes on additional changes
		if( bChangeAll )
		{
			OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, INDEX_NONE );
		}
		else
		{
			for( int32 ProducerIndex : ProducersToRevisit )
			{
				OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, ProducerIndex );
			}
		}

		return true;
	}

	UE_LOGF( LogDataprepCore
		, Error
		, "The producer to remove is out of bound. (Passed index: %d, Number of producer: %d, DataprepAssetProducers: %ls)"
		, IndexToRemove
		, AssetProducers.Num()
		, *GetPathName()
		);
	return false;
}

void UDataprepAssetProducers::RemoveAllProducers()
{
	for ( FDataprepAssetProducer& AssetProducer : AssetProducers )
	{
		if( UDataprepContentProducer* Producer = AssetProducer.Producer )
		{
			Producer->GetOnChanged().RemoveAll( this );

			DataprepCorePrivateUtils::DeleteRegisteredAsset( Producer );
		}
	}

	Modify();

	AssetProducers.Empty( AssetProducers.Num() );

	// Notify observers on changes
	OnChanged.Broadcast( FDataprepAssetChangeType::ProducerRemoved, INDEX_NONE );
}

void UDataprepAssetProducers::EnableProducer(int32 Index, bool bValue)
{
	if( AssetProducers.IsValidIndex( Index ) )
	{
		Modify();

		AssetProducers[Index].bIsEnabled = bValue;

		// Relay change notification to observers of this object
		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, Index );
	}
}

bool UDataprepAssetProducers::EnableAllProducers(bool bValue)
{
	if( AssetProducers.Num() > 0 )
	{
		Modify();

		for( FDataprepAssetProducer& Producer : AssetProducers )
		{
			Producer.bIsEnabled = bValue;
		}

		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, INDEX_NONE );

		return true;
	}

	return false;
}

const UDataprepContentProducer* UDataprepAssetProducers::GetProducer(int32 Index) const
{
	if ( AssetProducers.IsValidIndex( Index ) )
	{
		return AssetProducers[Index].Producer;
	}

	UE_LOGF(LogDataprepCore
		, Error
		, "The producer to retrive is out of bound. (Passed index: %d, Number of producer: %d, DataprepAssetProducers: %ls)"
		, Index
		, AssetProducers.Num()
		, *GetPathName()
	);
	return nullptr;
}

void UDataprepAssetProducers::OnProducerChanged( const UDataprepContentProducer* InProducer )
{
	int32 FoundIndex = 0;
	for( FDataprepAssetProducer& AssetProducer : AssetProducers )
	{
		if( AssetProducer.Producer == InProducer )
		{
			break;
		}

		++FoundIndex;
	}

	// Verify found producer is not now superseded by another one
	if( FoundIndex < AssetProducers.Num() )
	{
		Modify();

		bool bChangeAll = false;
		ValidateProducerChanges( FoundIndex, bChangeAll );

		// Relay change notification to observers of this object
		OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, bChangeAll ? INDEX_NONE : FoundIndex );
	}
}

void UDataprepAssetProducers::ValidateProducerChanges( int32 InIndex, bool &bChangeAll )
{
	bChangeAll = false;

	if( AssetProducers.IsValidIndex( InIndex ) && AssetProducers.Num() > 1 )
	{
		FDataprepAssetProducer& InAssetProducer = AssetProducers[InIndex];

		// Check if input producer is still superseded if applicable
		if( InAssetProducer.SupersededBy != INDEX_NONE )
		{
			FDataprepAssetProducer& SupersedingAssetProducer = AssetProducers[ InAssetProducer.SupersededBy ];

			if( SupersedingAssetProducer.Producer != nullptr && !SupersedingAssetProducer.Producer->Supersede( InAssetProducer.Producer ) )
			{
				InAssetProducer.SupersededBy = INDEX_NONE;
			}
		}

		// Check if producer is now superseded by any other producer
		int32 SupersederIndex = 0;
		for( FDataprepAssetProducer& AssetProducer : AssetProducers )
		{
			if( AssetProducer.Producer != nullptr &&
				AssetProducer.Producer != InAssetProducer.Producer &&
				AssetProducer.bIsEnabled == true && 
				AssetProducer.SupersededBy == INDEX_NONE && 
				AssetProducer.Producer->Supersede( InAssetProducer.Producer ) )
			{
				// Disable found producer if another producer supersedes its production
				InAssetProducer.SupersededBy = SupersederIndex;
				break;
			}
			SupersederIndex++;
		}

		// If input producer superseded any other producer, check if this is still valid
		// Check if input producer does not supersede other producers
		if( InAssetProducer.Producer != nullptr )
		{
			for( FDataprepAssetProducer& AssetProducer : AssetProducers )
			{
				if( AssetProducer.Producer != InAssetProducer.Producer )
				{
					if( AssetProducer.SupersededBy == InIndex )
					{
						if( !InAssetProducer.Producer->Supersede( AssetProducer.Producer ) )
						{
							bChangeAll = true;
							AssetProducer.SupersededBy = INDEX_NONE;
						}
					}
					else if( InAssetProducer.SupersededBy == INDEX_NONE && InAssetProducer.Producer->Supersede( AssetProducer.Producer ) )
					{
						bChangeAll = true;
						AssetProducer.SupersededBy = InIndex;
					}
				}
			}
		}
	}
}

UDataprepContentProducer* UDataprepAssetProducers::AddProducer_Internal(UClass* ProducerClass, bool bIsAutomated)
{
	if( ProducerClass && ProducerClass->IsChildOf( UDataprepContentProducer::StaticClass() ) )
	{
		Modify();

		UDataprepContentProducer* Producer = NewObject< UDataprepContentProducer >( this, ProducerClass, NAME_None, RF_Transactional );
		
		if ( Producer->CanAddToProducersArray( bIsAutomated ) )
		{
			Producer->GetOnChanged().AddUObject(this, &UDataprepAssetProducers::OnProducerChanged);

			int32 ProducerNextIndex = AssetProducers.Num();
			AssetProducers.Emplace( Producer, true );

			bool bChangeAll = false;
			ValidateProducerChanges( ProducerNextIndex, bChangeAll );

			OnChanged.Broadcast( FDataprepAssetChangeType::ProducerAdded, ProducerNextIndex );

			if(bChangeAll)
			{
				OnChanged.Broadcast( FDataprepAssetChangeType::ProducerModified, INDEX_NONE );
			}

			return Producer;
		}
		else
		{
			Producer->MarkAsGarbage();
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
