// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepContentConsumer.h"

#include "DataprepAssetInterface.h"
#include "DataprepAssetUserData.h"
#include "DataprepCoreUtils.h"

#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "DataprepContentConsumer"

const FString UDataprepContentConsumer::RelativeOutput = TEXT("ContentConsumer_RelativeOutput");

const FString& UDataprepConsumerUserData::GetMarker(const FString& MarkerName) const
{
	static FString NullString;

	const FString* ValuePtr = Markers.Find(MarkerName);
	return ValuePtr ? *ValuePtr : NullString;
}

UDataprepContentConsumer::UDataprepContentConsumer()
{
	TargetContentFolder = FPaths::GetPath( GetOutermost()->GetPathName() );
}

void UDataprepContentConsumer::PostEditUndo()
{
	Super::PostEditUndo();
	OnChanged.Broadcast();
}

bool UDataprepContentConsumer::Consume(const FDataprepConsumerContext& InContext)
{
	if(!InContext.WorldPtr.IsValid())
	{
		return false;
	}

	bool bSuccessful = false;
	
	Context = InContext;

	// Set package path if empty
	if( TargetContentFolder.IsEmpty() )
	{
		TargetContentFolder = FPaths::GetPath( GetOutermost()->GetPathName() );
	}

	if( Initialize() )
	{
		// Mark all incoming assets and actor's root components as done by the outer Dataprep asset
		AddDataprepAssetUserData();
		bSuccessful = Run();
	}

	Reset();

	Context.WorldPtr.Reset();
	Context.ProgressReporterPtr.Reset();
	Context.LoggerPtr.Reset();
	Context.Assets.Empty();

	Context = FDataprepConsumerContext();

	return bSuccessful;
}

bool UDataprepContentConsumer::SetTargetContentFolder(const FString& InTargetContentFolder, FText& OutFailureReason)
{
	constexpr bool bIsAutomated = false;
	return SetTargetContentFolderImplementation( InTargetContentFolder, OutFailureReason, bIsAutomated );
}

bool UDataprepContentConsumer::SetTargetContentFolderAutomated(const FString& InTargetContentFolder, FText& OutFailureReason)
{
	constexpr bool bIsAutomated = true;
	return SetTargetContentFolderImplementation(InTargetContentFolder, OutFailureReason, bIsAutomated);
}

bool UDataprepContentConsumer::SetTargetContentFolderImplementation(const FString& InTargetContentFolder, FText& OutFailureReason, const bool bIsAutomated)
{
	bool bValidContentFolder = true;

	if (!InTargetContentFolder.IsEmpty())
	{
		// Pretend creating a dummy package to verify packages could be created under this content folder.
		bValidContentFolder = FPackageName::IsValidLongPackageName(InTargetContentFolder / TEXT("DummyPackageName"), false, &OutFailureReason);
	}

	if (bValidContentFolder)
	{
		Modify();

		TargetContentFolder = !InTargetContentFolder.IsEmpty() ? InTargetContentFolder : FPaths::GetPath(GetOutermost()->GetPathName());

		// Remove ending '/' if applicable
		if (TargetContentFolder[TargetContentFolder.Len() - 1] == L'/')
		{
			TargetContentFolder.RemoveAt(TargetContentFolder.Len() - 1, 1);
		}

		OnChanged.Broadcast();
	}

	return bValidContentFolder;
}

FString UDataprepContentConsumer::GetTargetPackagePath() const
{
	FString TargetPackagePath(TargetContentFolder);

	if( TargetPackagePath.IsEmpty() )
	{
		TargetPackagePath = TEXT("/Game/");
	}
	else if( TargetPackagePath.StartsWith( TEXT("/Content") ) )
	{
		TargetPackagePath = TargetPackagePath.Replace( TEXT("/Content"), TEXT("/Game") );
	}

	// If path is one level deep, make sure it ends with a '/'
	int32 Index = INDEX_NONE;
	TargetPackagePath.FindLastChar(L'/', Index);
	check(Index >= 0);
	if(Index == 0)
	{
		TargetPackagePath.Append( TEXT("/") );
	}

	return TargetPackagePath;
}

bool UDataprepContentConsumer::SetLevelName(const FString & InLevelName, FText& OutFailureReason)
{
	constexpr bool bIsAutomated = false;
	return SetLevelNameImplementation( InLevelName, OutFailureReason, bIsAutomated );
}

bool UDataprepContentConsumer::SetLevelNameAutomated(const FString& InLevelName, FText& OutFailureReason)
{
	constexpr bool bIsAutomated = true;
	return SetLevelNameImplementation( InLevelName, OutFailureReason, bIsAutomated );
}

bool UDataprepContentConsumer::SetLevelNameImplementation(const FString& InLevelName, FText& OutFailureReason, const bool bIsAutomated)
{
	OutFailureReason = LOCTEXT("DataprepContentConsumer_SetLevelName", "Not implemented");
	ensure( false );
	return false;
}

void UDataprepContentConsumer::AddDataprepAssetUserData()
{
	UDataprepAssetInterface* DataprepAssetInterface = Cast<UDataprepAssetInterface>( GetOuter() );
	check( DataprepAssetInterface );

	auto AddUserData = [&DataprepAssetInterface](UObject* Object)
	{
		if (Object && Object->GetClass()->ImplementsInterface( UInterface_AssetUserData::StaticClass() ) )
		{
			if(IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(Object))
			{
				UDataprepAssetUserData* DataprepAssetUserData = AssetUserDataInterface->GetAssetUserData< UDataprepAssetUserData >();

				if(!DataprepAssetUserData)
				{
					EObjectFlags Flags = RF_Public /*| RF_Transactional*/; // RF_Transactional Disabled as it can cause a crash in the transaction system for blueprints

					DataprepAssetUserData = NewObject< UDataprepAssetUserData >( Object, NAME_None, Flags );

					AssetUserDataInterface->AddAssetUserData( DataprepAssetUserData );
				}

				DataprepAssetUserData->DataprepAssetPtr = DataprepAssetInterface;
			}
		}
	};

	// Add Dataprep user data to assets
	for( TWeakObjectPtr< UObject >& AssetPtr : Context.Assets )
	{
		AddUserData( AssetPtr.Get() );
	}

	TArray< AActor* > ActorsInWorld;
	FDataprepCoreUtils::GetActorsFromWorld( Context.WorldPtr.Get(), ActorsInWorld );

	for( AActor* Actor : ActorsInWorld )
	{
		if( Actor )
		{
			AddUserData(  Actor->GetRootComponent() );
		}
	}
}

#undef LOCTEXT_NAMESPACE

