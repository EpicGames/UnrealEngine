// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepContentProducer.h"

#include "DataprepCoreUtils.h"

#include "AssetToolsModule.h"
#include "EngineUtils.h"
#include "IAssetTools.h"
#include "LevelSequence.h"
#include "LevelVariantSets.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataprepContentProducer)

const FString DefaultNamespace( TEXT("void") );

FString UDataprepContentProducer::GetNamespace() const
{
	return DefaultNamespace;
}

bool UDataprepContentProducer::Produce(const FDataprepProducerContext& InContext, TArray< TWeakObjectPtr< UObject > >& OutAssets)
{
	Context = InContext;

	if( !IsValid() || !Initialize() )
	{
		Terminate();
		return false;
	}

	TSet< AActor* > ExistingActors;
	ExistingActors.Reserve( Context.WorldPtr->GetCurrentLevel()->Actors.Num() );

	// Cache all actors in the world before the producer is run
	const EActorIteratorFlags Flags = EActorIteratorFlags::SkipPendingKill;
	for (TActorIterator<AActor> It(Context.WorldPtr.Get(), AActor::StaticClass(), Flags); It; ++It)
	{
		if(*It != nullptr)
		{
			ExistingActors.Add( *It );
		}
	}

	// Cache number of assets to go through new assets after call to Execute
	int32 LastAssetCount = OutAssets.Num();

	if(!Execute( OutAssets ))
	{
		Terminate();
		return false;
	}

	// Collect all packages containing LevelSequence and LevelVariantSet assets to remap their
	// reference to an newly created actor
	TSet< UPackage* > PackagesToCheck;
	for(int32 Index = LastAssetCount; Index < OutAssets.Num(); ++Index)
	{
		if(UObject* Asset = OutAssets[Index].Get())
		{
			// Verify each asset is within its own package
			FString PackageName = Asset->GetOutermost()->GetName();
			FString AssetName = Asset->GetName();
			if(AssetName != FPaths::GetBaseFilename(PackageName))
			{
				UPackage* NewPackage = NewObject<UPackage>( nullptr, *FPaths::Combine( PackageName, AssetName ), Asset->GetOutermost()->GetFlags() );
				NewPackage->FullyLoad();

				Asset->Rename( nullptr, NewPackage, REN_NonTransactional | REN_DontCreateRedirectors );
			}

			if( ULevelSequence* LevelSequence = Cast<ULevelSequence>(Asset) )
			{
				PackagesToCheck.Add( LevelSequence->GetOutermost() );
			}
			else if( ULevelVariantSets* ThisLevelVariantSets = Cast<ULevelVariantSets>(Asset) )
			{
				PackagesToCheck.Add( ThisLevelVariantSets->GetOutermost() );
			}
		}
	}

	// Map between old path and new path of newly created actors
	TMap< FSoftObjectPath, FSoftObjectPath > ActorRedirectorMap;

	// Prefix all newly created actors with the namespace of the producer
	const FString Namespace = GetNamespace();

	if (Namespace.Len() > 0)
	{
		for (TActorIterator<AActor> It(Context.WorldPtr.Get(), AActor::StaticClass(), Flags); It; ++It)
		{
			if (*It != nullptr && ExistingActors.Find(*It) == nullptr)
			{
				AActor* Actor = *It;

				FSoftObjectPath PreviousActorSoftPath(Actor);

				const FString ActorName = Namespace + TEXT("_") + Actor->GetName();
				FDataprepCoreUtils::RenameObject(Actor, *ActorName);

				ActorRedirectorMap.Emplace(PreviousActorSoftPath, Actor);
			}
		}
	}

	// Update reference of LevelSequence or LevelVariantSets assets if necessary
	if(PackagesToCheck.Num() > 0)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths( PackagesToCheck.Array(), ActorRedirectorMap );
	}

	Terminate();

	return true;
}
