// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepCoreLogCategory.h"

DEFINE_LOG_CATEGORY(LogDataprepCore);
