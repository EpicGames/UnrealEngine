// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/DataprepCorePrivateUtils.h"

#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"

class FDataprepCoreModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		IFileManager::Get().MakeDirectory( *DataprepCorePrivateUtils::GetRootTemporaryDir(), true );

		// Register mount point for Dataprep core library root package folder
		FPackageName::RegisterMountPoint( DataprepCorePrivateUtils::GetRootPackagePath() + TEXT("/"), DataprepCorePrivateUtils::GetRootTemporaryDir() );
	}

	virtual void ShutdownModule() override
	{
		// Unregister mount point for Dataprep core library root package folder
		FPackageName::UnRegisterMountPoint( DataprepCorePrivateUtils::GetRootPackagePath() + TEXT("/"), DataprepCorePrivateUtils::GetRootTemporaryDir() );
	}
};

IMPLEMENT_MODULE( FDataprepCoreModule, DataprepCore )
