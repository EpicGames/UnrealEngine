// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/DataprepCorePrivateUtils.h"

#include "DataprepActionAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepCoreUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "EngineAnalytics.h"
#include "EngineLogs.h"
#include "IMessageLogListing.h"
#include "IAnalyticsProviderET.h"
#include "MessageLogModule.h"
#include "MaterialShared.h"
#include "StaticMeshAttributes.h"
#include "Materials/MaterialInstanceConstant.h"
#include "StaticMeshResources.h"
#include "Misc/EngineVersion.h"

#define LOCTEXT_NAMESPACE "DataprepAsset"

void DataprepCorePrivateUtils::DeleteRegisteredAsset(UObject* Asset)
{
	if(Asset != nullptr)
	{
		FDataprepCoreUtils::MoveToTransientPackage( Asset );

		Asset->ClearFlags(RF_Standalone | RF_Public);
		Asset->RemoveFromRoot();
		Asset->MarkAsGarbage();

		FAssetRegistryModule::AssetDeleted( Asset ) ;
	}
}


const FString& DataprepCorePrivateUtils::GetRootTemporaryDir()
{
	static FString RootTemporaryDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DataprepTemp") );
	return RootTemporaryDir;
}

const FString& DataprepCorePrivateUtils::GetRootPackagePath()
{
	static FString RootPackagePath( TEXT("/DataprepTransient") );
	return RootPackagePath;
}

void DataprepCorePrivateUtils::LogMessage( EMessageSeverity::Type Severity, const FText& Message, const FText& NotificationText )
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	TSharedPtr<IMessageLogListing> LogListing = MessageLogModule.GetLogListing( TEXT("DataprepCore") );
	LogListing->SetLabel( LOCTEXT("MessageLogger", "Dataprep Core") );

	LogListing->AddMessage( FTokenizedMessage::Create( Severity, Message ), /*bMirrorToOutputLog*/ true );

	if( !NotificationText.IsEmpty() )
	{
		LogListing->NotifyIfAnyMessages( NotificationText, EMessageSeverity::Info);
	}
}

void DataprepCorePrivateUtils::BuildStaticMeshes(TSet<UStaticMesh*>& StaticMeshes, TFunction<bool(UStaticMesh*)> ProgressFunction, bool bForceBuild)
{
	TArray<UStaticMesh*> BuiltMeshes;
	BuiltMeshes.Reserve( StaticMeshes.Num() );

	if(bForceBuild)
	{
		BuiltMeshes.Append( StaticMeshes.Array() );
	}
	else
	{
		for(UStaticMesh* StaticMesh : StaticMeshes)
		{
			if(StaticMesh && (!StaticMesh->GetRenderData() || !StaticMesh->GetRenderData()->IsInitialized()))
			{
				BuiltMeshes.Add( StaticMesh );
			}
		}
	}

	if(BuiltMeshes.Num() > 0)
	{
		// Start with the biggest mesh first to help balancing tasks on threads
		BuiltMeshes.Sort(
			[](const UStaticMesh& Lhs, const UStaticMesh& Rhs) 
		{ 
			int32 LhsVerticesNum = Lhs.IsMeshDescriptionValid(0) ? Lhs.GetMeshDescription(0)->Vertices().Num() : 0;
			int32 RhsVerticesNum = Rhs.IsMeshDescriptionValid(0) ? Rhs.GetMeshDescription(0)->Vertices().Num() : 0;

			return LhsVerticesNum > RhsVerticesNum;
		}
		);

		TArray< TArray<FMeshBuildSettings> > StaticMeshesSettings;
		StaticMeshesSettings.Reserve( BuiltMeshes.Num() );

		//Cache the BuildSettings and update them before building the meshes.
		for (UStaticMesh* StaticMesh : BuiltMeshes)
		{
			int32 NumSourceModels = StaticMesh->GetNumSourceModels();
			TArray<FMeshBuildSettings> BuildSettings;
			BuildSettings.Reserve(NumSourceModels);

			for(int32 Index = 0; Index < NumSourceModels; ++Index)
			{
				FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(Index);

				BuildSettings.Add( SourceModel.BuildSettings );

				if(FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(Index))
				{
					FStaticMeshAttributes Attributes(*MeshDescription);
					if(SourceModel.BuildSettings.DstLightmapIndex != -1)
					{
						TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
						//If the importer have enabled lightmap generation, disabling it may interfere with the build process, so we only allow enabling it.
						SourceModel.BuildSettings.bGenerateLightmapUVs |= VertexInstanceUVs.IsValid() && VertexInstanceUVs.GetNumChannels() > SourceModel.BuildSettings.SrcLightmapIndex;
					}
					else
					{
						SourceModel.BuildSettings.bGenerateLightmapUVs = false;
					}

					SourceModel.BuildSettings.bRecomputeNormals = !(Attributes.GetVertexInstanceNormals().IsValid() && Attributes.GetVertexInstanceNormals().GetNumChannels() > 0);
					SourceModel.BuildSettings.bRecomputeTangents = false;
					SourceModel.BuildSettings.DistanceFieldResolutionScale = 0;
					//SourceModel.BuildSettings.bBuildReversedIndexBuffer = false;
				}
				
				// As soon as StaticMeshes are built, the mesh description can be released
				// This generate a significant freeing of memory
				StaticMesh->ClearMeshDescription(Index);
			}

			StaticMeshesSettings.Add(MoveTemp(BuildSettings));				
		}

		// Disable warnings from LogStaticMesh. Not useful
		ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
		LogStaticMesh.SetVerbosity( ELogVerbosity::Error );

		UStaticMesh::BatchBuild(BuiltMeshes, true, ProgressFunction);

		// Restore LogStaticMesh verbosity
		LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );

		for(int32 Index = 0; Index < BuiltMeshes.Num(); ++Index)
		{
			UStaticMesh* StaticMesh = BuiltMeshes[Index];
			TArray<FMeshBuildSettings>& PrevBuildSettings = StaticMeshesSettings[Index];

			int32 NumSourceModels = StaticMesh->GetNumSourceModels();
			for(int32 SourceModelIndex = 0; SourceModelIndex < NumSourceModels; ++SourceModelIndex)
			{
				StaticMesh->GetSourceModel(SourceModelIndex).BuildSettings = PrevBuildSettings[SourceModelIndex];
			}

			if(FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData())
			{
				for ( FStaticMeshLODResources& LODResources : RenderData->LODResources )
				{
					LODResources.bHasColorVertexData = true;
				}
			}
		}
	}
}

void DataprepCorePrivateUtils::ClearAssets(const TArray<TWeakObjectPtr<UObject>>& Assets)
{
	for(const TWeakObjectPtr<UObject>& ObjectPtr : Assets)
	{
		if(UStaticMesh* StaticMesh = Cast<UStaticMesh>(ObjectPtr.Get()))
		{
			StaticMesh->PreEditChange( nullptr );
			StaticMesh->SetRenderData( nullptr );
		}
	}
}

void DataprepCorePrivateUtils::CompileMaterial(UMaterialInterface* MaterialInterface)
{
	if (MaterialInterface == nullptr)
	{
		return;
	}

	FMaterialUpdateContext MaterialUpdateContext;

	MaterialUpdateContext.AddMaterialInterface( MaterialInterface );

	if ( UMaterialInstanceConstant* ConstantMaterialInstance = Cast< UMaterialInstanceConstant >( MaterialInterface ) )
	{
		// If BlendMode override property has been changed, make sure this combination of the parent material is compiled
		if ( ConstantMaterialInstance->BasePropertyOverrides.bOverride_BlendMode == true )
		{
			ConstantMaterialInstance->ForceRecompileForRendering();
		}
		else
		{
			// If a switch is overridden, we need to recompile
			FStaticParameterSet StaticParameters;
			ConstantMaterialInstance->GetStaticParameterValues( StaticParameters );

			for ( FStaticSwitchParameter& Switch : StaticParameters.StaticSwitchParameters )
			{
				if ( Switch.bOverride )
				{
					ConstantMaterialInstance->ForceRecompileForRendering();
					break;
				}
			}
		}
	}

	MaterialInterface->PreEditChange( nullptr );
	MaterialInterface->PostEditChange();
}

void DataprepCorePrivateUtils::Analytics::RecipeExecuted( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );
		
		const TArray<UDataprepActionAsset*>& Actions = InDataprepAsset->GetActions();
		int32 NumStepsTotal = 0;

		for( const UDataprepActionAsset* Action : Actions )
		{
			NumStepsTotal += Action->GetStepsCount();
		}

		EventAttributes.Emplace( TEXT("ActionsCount"), Actions.Num() );
		EventAttributes.Emplace( TEXT("StepsCount"), NumStepsTotal );

		const bool bIsDataprepInstance = InDataprepAsset->IsA<UDataprepAssetInstance>();
		const FString EventText = bIsDataprepInstance ? TEXT("Editor.Dataprep.Executed.Instance") : TEXT("Editor.Dataprep.Executed.Asset");

		FEngineAnalytics::GetProvider().RecordEvent( EventText, EventAttributes );
	}
}

void DataprepCorePrivateUtils::Analytics::DataprepAssetCreated( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );

		const bool bIsDataprepInstance = InDataprepAsset->IsA<UDataprepAssetInstance>();
		const FString EventText = bIsDataprepInstance ? TEXT("Editor.Dataprep.Created.Instance") : TEXT("Editor.Dataprep.Created.Asset");

		FEngineAnalytics::GetProvider().RecordEvent( EventText, EventAttributes );
	}
}

void DataprepCorePrivateUtils::Analytics::DataprepEditorOpened( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Editor.Dataprep.EditorOpened"), EventAttributes );
	}
}

void DataprepCorePrivateUtils::Analytics::ExecuteTriggered( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Editor.Dataprep.ExecuteTriggered"), EventAttributes );
	}
}

void DataprepCorePrivateUtils::Analytics::ImportTriggered( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Editor.Dataprep.ImportTriggered"), EventAttributes );
	}
}

void DataprepCorePrivateUtils::Analytics::CommitTriggered( UDataprepAssetInterface* InDataprepAsset )
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace( TEXT("EngineVersion"), FEngineVersion::Current().ToString( EVersionComponent::Patch ) );

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Editor.Dataprep.CommitTriggered"), EventAttributes );
	}
}

#undef LOCTEXT_NAMESPACE
