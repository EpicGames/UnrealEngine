// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepCoreUtils.h"
#include "AssetRegistry/ARFilter.h"
#include "Engine/Texture.h"

#ifdef NEW_DATASMITHSCENE_WORKFLOW
#include "DataprepAssetUserData.h"
#endif

#include "DataprepAsset.h"
#include "DataprepContentConsumer.h"
#include "DataprepContentProducer.h"
#include "DataprepCoreLogCategory.h"
#include "EngineLogs.h"
#include "Shared/DataprepCorePrivateUtils.h"
#include "DataprepOperation.h"
#include "SelectionSystem/DataprepFilter.h"
#include "Materials/MaterialInterface.h"
#include "SelectionSystem/DataprepSelectionTransform.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/FileManager.h"
#include "LevelSequence.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "ActorEditorUtils.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Editor.h"
#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif


#define LOCTEXT_NAMESPACE "DataprepCoreUtils"

namespace DataprepCoreUtilsPrivate
{
	template<class ArrayOfClass>
	void GetActorsFromWorld(const UWorld* World, TArray<ArrayOfClass*>& OutActors)
	{
		if ( World != nullptr )
		{
			int32 ActorsCount = 0;
			for ( ULevel* Level : World->GetLevels() )
			{
				ActorsCount += Level->Actors.Num();
			}

			OutActors.Reserve( ActorsCount );

			for ( ULevel* Level : World->GetLevels() )
			{
				for ( AActor* Actor : Level->Actors )
				{
					const bool bIsValidActor = IsValid(Actor)
						&& Actor->IsEditable()
						&& !Actor->IsTemplate()
						&& !FActorEditorUtils::IsABuilderBrush( Actor )
						&& !Actor->IsA( AWorldSettings::StaticClass() )
						&& !Actor->HasAnyFlags( RF_Transient );

					if ( bIsValidActor )
					{
						OutActors.Add( Actor );
					}
				}
			}
		}
	}
}


UDataprepAsset* FDataprepCoreUtils::GetDataprepAssetOfObject(UObject* Object)
{
	while ( Object )
	{
		if ( UDataprepAsset::StaticClass() == Object->GetClass() )
		{
			return static_cast<UDataprepAsset*>( Object );
		}
		Object = Object->GetOuter();
	}

	return nullptr;
}

UDataprepActionAsset* FDataprepCoreUtils::GetDataprepActionAssetOf(UObject* Object)
{
	while (Object)
	{
		if (UDataprepActionAsset::StaticClass() == Object->GetClass())
		{
			return static_cast<UDataprepActionAsset*>(Object);
		}
		Object = Object->GetOuter();
	}

	return nullptr;
}

void FDataprepCoreUtils::PurgeObjects(TArray<UObject*> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepCoreUtils::PurgeObjects);

	// Build an array of unique objects
	TSet<UObject*> ObjectsSet;
	ObjectsSet.Reserve( InObjects.Num() );
	ObjectsSet.Append( InObjects );
	TArray<UObject*> Objects( ObjectsSet.Array() );

	TArray<UObject*> ObjectsToPurge;
	ObjectsToPurge.Reserve( Objects.Num() );
#if WITH_EDITOR
	TArray<UObject*> PublicObjectsToPurge;
	PublicObjectsToPurge.Reserve( Objects.Num() );
#endif // WITH_EDITOR

	auto MakeObjectPurgeable = [&ObjectsToPurge](UObject* InObject)
	{
#if WITH_EDITOR
		if ( InObject->IsAsset() )
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset( InObject );
		}
#endif // WITH_EDITOR
		if ( InObject->IsRooted() )
		{
			InObject->RemoveFromRoot();
		}

		InObject->ClearFlags( RF_Public | RF_Standalone );
		InObject->MarkAsGarbage();
		ObjectsToPurge.Add( InObject );
	};

	auto MakeSourceObjectPurgeable = [MakeObjectPurgeable](UObject* InSourceObject)
	{
		MakeObjectPurgeable( InSourceObject );
		ForEachObjectWithOuter( InSourceObject, [MakeObjectPurgeable](UObject* InObject)
		{
			MakeObjectPurgeable( InObject );
		});
	};

	// Clean-up any in-memory packages that should be purged and check if we are purging the current map
	ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
	LogStaticMesh.SetVerbosity( ELogVerbosity::Error );
	for ( UObject* Object : Objects )
	{
		if (Object)
		{
#if WITH_EDITOR
			/**
			 * Add object for reference removal if it's public
			 * This is used to emulate the workflow that is used by the editor when deleting a asset.
			 * Due to the transient package we can't simply use IsAsset()
			 */
			if ( Object->HasAnyFlags( RF_Public ) )
			{
				PublicObjectsToPurge.Add( Object );
			}
#endif // WITH_EDITOR

			MakeSourceObjectPurgeable( Object );
		}
	}
	// Restore LogStaticMesh verbosity
	LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );

	/** 
	 * If we have any public object that were made purgeable, null out their references so we can safely garbage collect
	 * Additionally, ObjectTools::ForceReplaceReferences is calling PreEditChange and PostEditChange on all impacted objects.
	 * Consequently, making sure async tasks processing those objects are notified and act accordingly.
	 * This is the way to make sure that all dependencies are taken in account and properly handled
	 */
#if WITH_EDITOR
	if ( PublicObjectsToPurge.Num() > 0 )
	{
		/**
		 * Due to way that some render proxy are created we must remove the current rendering scene.
		 * This is to ensure that the render proxies won't have a dangling pointer to an asset while removing then on the next tick
		 */
		FGlobalComponentRecreateRenderStateContext RefreshRendering;
		ObjectTools::ForceReplaceReferences(nullptr, PublicObjectsToPurge);

		// Ensure that all the rendering commands were processed before doing the garbage collection (see above comment)
		FlushRenderingCommands();
	}
#endif // WITH_EDITOR

	// if we have object to purge but the map isn't one of them collect garbage (if we purged the map it has already been done)
	if ( ObjectsToPurge.Num() > 0 )
	{
		FlushRenderingCommands();

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

bool FDataprepCoreUtils::IsAsset(UObject* Object)
{
	const bool bHasValidObjectFlags = Object && IsValidChecked(Object) && !Object->HasAnyFlags(RF_ClassDefaultObject) && Object->HasAnyFlags(RF_Public);

	if(!bHasValidObjectFlags)
	{
		// Otherwise returns true if Object is of the supported classes
		return Object && (Object->IsA<UStaticMesh>() || Object->IsA<UMaterialInterface>() || Object->IsA<UTexture>() || Object->IsA<ULevelSequence>() || Object->IsAsset());
	}

	return true;
}

bool FDataprepCoreUtils::ExecuteDataprep(UDataprepAssetInterface* DataprepAssetInterface, const TSharedPtr<IDataprepLogger>& Logger, const TSharedPtr<IDataprepProgressReporter>& Reporter)
{
	if ( DataprepAssetInterface != nullptr )
	{
		// The temporary folders are used for the whole session of the Unreal Editor
		static FString RelativeTempFolder = FString::FromInt( FPlatformProcess::GetCurrentProcessId() ) / FGuid::NewGuid().ToString();
		static FString TransientContentFolder = DataprepCorePrivateUtils::GetRootPackagePath() / RelativeTempFolder;

		// Create transient world to host data from producer
		FName UniqueWorldName = MakeUniqueObjectName( GetTransientPackage(), UWorld::StaticClass(), FName( *( LOCTEXT("TransientWorld", "Preview").ToString() ) ) );
		TStrongObjectPtr<UWorld> TransientWorld = TStrongObjectPtr<UWorld>( NewObject<UWorld>( GetTransientPackage(), UniqueWorldName ) );
		TransientWorld->WorldType = EWorldType::EditorPreview;

		FWorldContext& WorldContext = GEngine->CreateNewWorldContext( TransientWorld->WorldType );
		WorldContext.SetCurrentWorld( TransientWorld.Get() );

		TransientWorld->InitializeNewWorld( UWorld::InitializationValues()
			.AllowAudioPlayback( false )
			.CreatePhysicsScene( false )
			.RequiresHitProxies( false )
			.CreateNavigation( false )
			.CreateAISystem( false )
			.ShouldSimulatePhysics( false )
			.SetTransactional( false )
			);

		TArray<TWeakObjectPtr<UObject>> Assets;

		FText DataprepAssetTextName = FText::FromString( DataprepAssetInterface->GetName() );
		FText TaskDescription = FText::Format( LOCTEXT("ExecutingDataprepAsset", "Executing Dataprep Asset \"{0}\" ..."), DataprepAssetTextName );
		FDataprepWorkReporter ProgressTask( Reporter, TaskDescription, 3.0f, 1.0f );

		bool bSuccessfulExecute = true;

		// Run the producers
		{
			// Create package to pass to the producers
			UPackage* TransientPackage = NewObject<UPackage>( nullptr, *TransientContentFolder, RF_Transient );
			TransientPackage->FullyLoad();

			TSharedPtr<FDataprepCoreUtils::FDataprepFeedbackContext> FeedbackContext = MakeShared<FDataprepCoreUtils::FDataprepFeedbackContext>();

			FDataprepProducerContext Context;
			Context.SetWorld( TransientWorld.Get() )
				.SetRootPackage( TransientPackage )
				.SetLogger( Logger )
				.SetProgressReporter( Reporter );

			FText Message = FText::Format( LOCTEXT("Running_Producers", "Running \"{0}\'s Producers ..."), DataprepAssetTextName );
			ProgressTask.ReportNextStep( Message );
			Assets = DataprepAssetInterface->GetProducers()->Produce( Context );
		}

		// Trigger execution of data preparation operations on world attached to recipe
		TSet<TWeakObjectPtr<UObject>> CachedAssets;
		{
			DataprepActionAsset::FCanExecuteNextStepFunc CanExecuteNextStepFunc = [](UDataprepActionAsset* ActionAsset) -> bool
			{
				return true;
			};


			TSharedPtr<FDataprepActionContext> ActionsContext = MakeShared<FDataprepActionContext>();

			ActionsContext->SetTransientContentFolder( TransientContentFolder / TEXT("Pipeline") )
				.SetLogger( Logger )
				.SetProgressReporter( Reporter )
				.SetCanExecuteNextStep( CanExecuteNextStepFunc )
				.SetWorld( TransientWorld.Get() )
				.SetAssets( Assets );

			FText Message = FText::Format( LOCTEXT("Executing_Recipe", "Executing \"{0}\'s Recipe ..."), DataprepAssetTextName );
			ProgressTask.ReportNextStep( Message );
			DataprepAssetInterface->ExecuteRecipe( ActionsContext );

			// Update list of assets with latest ones
			Assets = ActionsContext->Assets.Array();

			for ( TWeakObjectPtr<UObject>& Asset : Assets )
			{
				if ( Asset.IsValid() )
				{
					CachedAssets.Add( Asset );
				}
			}

		}

		// Run consumer to output result of recipe
		{
			FDataprepConsumerContext Context;
			Context.SetWorld( TransientWorld.Get() )
				.SetAssets( Assets )
				.SetTransientContentFolder( TransientContentFolder )
				.SetSilentMode(true)
				.SetLogger( Logger )
				.SetProgressReporter( Reporter );

			FText Message = FText::Format( LOCTEXT("Running_Consumer", "Running \"{0}\'s Consumer ..."), DataprepAssetTextName );
			ProgressTask.ReportNextStep( Message );

			bSuccessfulExecute = DataprepAssetInterface->RunConsumer( Context );
		}

		// Clean all temporary data created by the Dataprep asset
		{
			// Delete all actors of the transient world
			TArray<AActor*> TransientActors;
			FDataprepCoreUtils::GetActorsFromWorld( TransientWorld.Get(), TransientActors );
			for ( AActor* Actor : TransientActors )
			{
				if ( IsValid(Actor) )
				{
					TransientWorld->EditorDestroyActor( Actor, true );

					// Since deletion can be delayed, rename to avoid future name collision
					// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
					Actor->UObject::Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}
			}

			// Delete assets which are still in the transient content folder
			TArray<UObject*> ObjectsToDelete;
			for ( TWeakObjectPtr<UObject>& Asset : CachedAssets )
			{
				if ( UObject* ObjectToDelete = Asset.Get() )
				{
					FString PackagePath = ObjectToDelete->GetOutermost()->GetName();
					if ( PackagePath.StartsWith( TransientContentFolder ) )
					{
						FDataprepCoreUtils::MoveToTransientPackage( ObjectToDelete );
						ObjectsToDelete.Add( ObjectToDelete );
					}
				}
			}

			// Disable warnings from LogStaticMesh because FDataprepCoreUtils::PurgeObjects is pretty verbose on harmless warnings
			ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
			LogStaticMesh.SetVerbosity( ELogVerbosity::Error );

			FDataprepCoreUtils::PurgeObjects( MoveTemp( ObjectsToDelete ) );

			// Erase all temporary packages and files created by the Dataprep asset
			DeleteTemporaryFolders( TransientContentFolder );

			// Restore LogStaticMesh verbosity
			LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );
		}

		DataprepCorePrivateUtils::Analytics::RecipeExecuted( DataprepAssetInterface );

		// Destroy transient world
		GEngine->DestroyWorldContext(TransientWorld.Get());
		TransientWorld->DestroyWorld(true);
		TransientWorld.Reset();

		return bSuccessfulExecute;
	}

	return false;
}

bool FDataprepCoreUtils::IsClassValidForStepCreation(const TSubclassOf<UDataprepParameterizableObject>& StepType, UClass*& OutValidRootClass, FText& OutMessageIfInvalid)
{
	UClass* Class = StepType.Get();
	if ( !Class )
	{
		OutMessageIfInvalid = LOCTEXT("StepTypeNull", "The class to use for the step is none.");
		return false;
	}

	if ( Class->HasAnyClassFlags( CLASS_Abstract ) )
	{
		OutMessageIfInvalid = LOCTEXT("StepTypeIsAbstract", "The class to use for the creation of the step is abstract. We can use that to create a step.");
		return false;
	}

	if ( Class->HasAnyClassFlags( CLASS_Transient ) )
	{
		OutMessageIfInvalid = LOCTEXT("StepTypeIsTransient", "The class to use for the creation of the step is transient. We can't save transient types. So we can't use them .");
		return false;
	}

	if ( Class->HasAnyClassFlags( CLASS_NewerVersionExists ) )
	{
		OutMessageIfInvalid = LOCTEXT("StepTypeHasBeenRemplaced", "The class to use for the creation of the step is a old version of newer class.");
		return false;
	}

	UClass* DataprepFilterClass = UDataprepFilter::StaticClass();
	UClass* DataprepFilterNoFetcherClass = UDataprepFilterNoFetcher::StaticClass();
	UClass* DataprepTopLevelClass = UDataprepParameterizableObject::StaticClass();
	UClass* DataprepOperationClass = UDataprepOperation::StaticClass();
	UClass* DataprepFetcherClass = UDataprepFetcher::StaticClass();
	UClass* DataprepTransformClass = UDataprepSelectionTransform::StaticClass();

	while ( Class )
	{
		if ( Class == DataprepFilterClass )
		{
			OutMessageIfInvalid = LOCTEXT("StepTypeIsAFilter", "The class to use for the creation of the step is filter. Please use the desired fetcher for the filter instead.");
			return false;
		}

		if ( Class == DataprepFilterNoFetcherClass )
		{
			OutValidRootClass = DataprepFilterNoFetcherClass;
			return true;
		}

		if ( Class == DataprepTopLevelClass )
		{
			OutMessageIfInvalid = LOCTEXT("StepTypeIsUnknow", "The class to use for the creation of the step is unknow to the dataprep ecosystem.");
			return false;
		}

		if ( Class == DataprepOperationClass )
		{
			OutValidRootClass = DataprepOperationClass;
			return true;
		}

		if ( Class == DataprepFetcherClass )
		{
			OutValidRootClass = DataprepFetcherClass;
			return true;
		}

		if ( Class == DataprepTransformClass )
		{
			OutValidRootClass = DataprepTransformClass;
			return true;
		}

		Class = Class->GetSuperClass();
	}

	return false;
}

UClass* FDataprepCoreUtils::GetTypeOfActionStep(const UDataprepParameterizableObject* Object)
{
	UClass* CurrentClass = Object ? Object->GetClass() : nullptr;

	const UClass* DataprepFilterClass = UDataprepFilter::StaticClass();
	const UClass* DataprepFilterNoFetcherClass = UDataprepFilterNoFetcher::StaticClass();
	const UClass* DataprepOperationClass = UDataprepOperation::StaticClass();
	const UClass* DataprepTransformClass = UDataprepSelectionTransform::StaticClass();

	while ( CurrentClass )
	{
		if ( CurrentClass == DataprepFilterClass
			|| CurrentClass == DataprepOperationClass
			|| CurrentClass == DataprepTransformClass
			|| CurrentClass == DataprepFilterNoFetcherClass
			)
		{
			break;
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}
	
	return CurrentClass;
}

FDataprepWorkReporter::FDataprepWorkReporter(const TSharedPtr<IDataprepProgressReporter>& InReporter, const FText& InDescription, float InAmountOfWork, float InIncrementOfWork, bool bInterruptible )
	: Reporter( InReporter )
	, DefaultIncrementOfWork( InIncrementOfWork )
{
	if( Reporter.IsValid())
	{
		Reporter->BeginWork( InDescription, InAmountOfWork, bInterruptible );
	}
}

FDataprepWorkReporter::~FDataprepWorkReporter()
{
	if( Reporter.IsValid() )
	{
		Reporter->EndWork();
	}
}

void FDataprepWorkReporter::ReportNextStep(const FText & InMessage, float InIncrementOfWork )
{
	if( Reporter.IsValid() )
	{
		Reporter->ReportProgress( InIncrementOfWork, InMessage );
	}
}

bool FDataprepWorkReporter::IsWorkCancelled() const
{
	if ( Reporter.IsValid() )
	{
		return Reporter->IsWorkCancelled();
	}
	return false;
}

#ifdef NEW_DATASMITHSCENE_WORKFLOW
void FDataprepCoreUtils::AddDataprepAssetUserData(UObject* Target, UDataprepAssetInterface* DataprepAssetInterface)
{
	if (!Target || !Target->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		return;
	}

	if (Target->GetClass()->IsChildOf(AActor::StaticClass()))
	{
		// The root Component holds AssetUserData on behalf of the actor
		Target = Cast<AActor>(Target)->GetRootComponent();
	}

	if(IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( Target ))
	{
		UDataprepAssetUserData* UserData = AssetUserDataInterface->GetAssetUserData< UDataprepAssetUserData >();

		if(!UserData)
		{
			EObjectFlags Flags = RF_Public /*| RF_Transactional*/; // RF_Transactional Disabled as is can cause a crash in the transaction system for blueprints

			UserData = NewObject< UDataprepAssetUserData >(Target, NAME_None, Flags);

			AssetUserDataInterface->AddAssetUserData( UserData );
		}

		UserData->DataprepAssetPtr = DataprepAssetInterface;
	}
}
#endif

void FDataprepCoreUtils::FDataprepLogger::LogInfo(const FText& InLogText, const UObject& InObject)
{
	UE_LOGF( LogDataprepCore, Log, "%ls : %ls", *InObject.GetName(), *InLogText.ToString() );
}

void FDataprepCoreUtils::FDataprepLogger::LogWarning(const FText& InLogText, const UObject& InObject)
{
	UE_LOGF( LogDataprepCore, Warning, "%ls : %ls", *InObject.GetName(), *InLogText.ToString() );
}

void FDataprepCoreUtils::FDataprepLogger::LogError(const FText& InLogText,  const UObject& InObject)
{
	UE_LOGF( LogDataprepCore, Error, "%ls : %ls", *InObject.GetName(), *InLogText.ToString() );
}

void FDataprepCoreUtils::FDataprepProgressUIReporter::BeginWork( const FText& InTitle, float InAmountOfWork, bool bInterruptible )
{
	ProgressTasks.Emplace( new FScopedSlowTask( InAmountOfWork, InTitle, true, FeedbackContext.IsValid() ? *FeedbackContext.Get() : *GWarn ) );
	ProgressTasks.Last()->MakeDialog( bInterruptible );
}

void FDataprepCoreUtils::FDataprepProgressUIReporter::EndWork()
{
	if(ProgressTasks.Num() > 0)
	{
		ProgressTasks.Pop();
	}
}

void FDataprepCoreUtils::FDataprepProgressUIReporter::ReportProgress( float Progress, const FText& InMessage )
{
	if( ProgressTasks.Num() > 0 )
	{
		TSharedPtr<FScopedSlowTask>& ProgressTask = ProgressTasks.Last();
		ProgressTask->EnterProgressFrame( Progress, InMessage );
	}
}

bool FDataprepCoreUtils::FDataprepProgressUIReporter::IsWorkCancelled()
{
	if( !bIsCancelled && ProgressTasks.Num() > 0 )
	{
		const TSharedPtr<FScopedSlowTask>& ProgressTask = ProgressTasks.Last();
		bIsCancelled |= ProgressTask->ShouldCancel();
	}
	return bIsCancelled;
}

FFeedbackContext* FDataprepCoreUtils::FDataprepProgressUIReporter::GetFeedbackContext() const
{
	return FeedbackContext.IsValid() ? FeedbackContext.Get() : GWarn;
}

void FDataprepCoreUtils::FDataprepProgressTextReporter::BeginWork( const FText& InTitle, float InAmountOfWork, bool /*bInterruptible*/ )
{
	UE_LOGF( LogDataprepCore, Log, "Start: %ls ...", *InTitle.ToString() );
	++TaskDepth;
}

void FDataprepCoreUtils::FDataprepProgressTextReporter::EndWork()
{
	if(TaskDepth > 0)
	{
		--TaskDepth;
	}
}

void FDataprepCoreUtils::FDataprepProgressTextReporter::ReportProgress( float Progress, const FText& InMessage )
{
	if( TaskDepth > 0 )
	{
		UE_LOGF( LogDataprepCore, Log, "Doing %ls ...", *InMessage.ToString() );
	}
}

bool FDataprepCoreUtils::FDataprepProgressTextReporter::IsWorkCancelled()
{
	return false;
}

FFeedbackContext* FDataprepCoreUtils::FDataprepProgressTextReporter::GetFeedbackContext() const
{
	return FeedbackContext.Get();
}

void FDataprepCoreUtils::BuildAssets(const TArray<TWeakObjectPtr<UObject>>& Assets, const TSharedPtr<IDataprepProgressReporter>& ProgressReporterPtr)
{
	TSet<UStaticMesh*> StaticMeshes;
	TSet<UMaterialInterface*> MaterialInterfaces;

	// Unregister all actors components to avoid excessive refresh in the 3D engine while updating materials.
	for (TObjectIterator<AActor> ActorIterator; ActorIterator; ++ActorIterator)
	{
		if (ActorIterator->GetWorld())
		{
			ActorIterator->UnregisterAllComponents( /* bForReregister = */true);
		}
	}

	for( const TWeakObjectPtr<UObject>& AssetPtr : Assets )
	{
		UObject* AssetObject = AssetPtr.Get();
		if( AssetObject )
		{
			if(UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetObject))
			{
				if (!MaterialInterfaces.Contains(MaterialInterface))
				{
					// Force compilation of materials which have no render proxy
					DataprepCorePrivateUtils::CompileMaterial(MaterialInterface);
					MaterialInterfaces.Add(MaterialInterface);
				}
			}
			else if(UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetObject))
			{
				StaticMeshes.Add( StaticMesh );
			}
		}
	}

	// Materials have been updated, we can register everything back.
	for (TObjectIterator<AActor> ActorIterator; ActorIterator; ++ActorIterator)
	{
		if (ActorIterator->GetWorld())
		{
			ActorIterator->RegisterAllComponents();
		}
	}

	FDataprepWorkReporter Task( ProgressReporterPtr, LOCTEXT( "BuildAssets_Building", "Building static meshes ..." ), (float)StaticMeshes.Num(), 1.0f, false );

	// Build static meshes
	int32 AssetBuiltCount = 0;
	DataprepCorePrivateUtils::BuildStaticMeshes(StaticMeshes, [&](UStaticMesh* StaticMesh) -> bool {
		++AssetBuiltCount;
		Task.ReportNextStep(FText::Format(LOCTEXT( "BuildAssets_Building_Meshes", "Building static meshes ({0} / {1})" ), AssetBuiltCount, StaticMeshes.Num()), 1.0f);
		return true;
	});
}

bool FDataprepCoreUtils::RemoveSteps(UDataprepActionAsset* ActionAsset, const TArray<int32>& Indices, int32& ActionIndex, bool bDiscardParametrization)
{
	ActionIndex = INDEX_NONE;

	UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject(ActionAsset);
	check(DataprepAsset);

	if(ActionAsset->GetStepsCount() == Indices.Num())
	{
		// Find index of source action 
		ActionIndex = DataprepAsset->GetActionIndex(ActionAsset);
		ensure(ActionIndex != INDEX_NONE);

		return DataprepAsset->RemoveAction(ActionIndex, bDiscardParametrization);
	}

	return ActionAsset->RemoveSteps( Indices, bDiscardParametrization );
}

void FDataprepCoreUtils::GetActorsFromWorld(const UWorld* World, TArray<UObject*>& OutActors)
{
	DataprepCoreUtilsPrivate::GetActorsFromWorld( World, OutActors );
}

void FDataprepCoreUtils::GetActorsFromWorld(const UWorld* World, TArray<AActor*>& OutActors)
{
	DataprepCoreUtilsPrivate::GetActorsFromWorld( World, OutActors );
}

void FDataprepCoreUtils::DeleteTemporaryFolders(const FString& BaseTemporaryPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepCoreUtils::DeleteTemporaryFolders);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>( TEXT("AssetRegistry") );
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray< UObject* > ObjectsToDelete;

	// Find all registered objects which are in  memory and under the temporary path
	{
		// Query for a list of assets in the path to delete
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Emplace( *BaseTemporaryPath );

		TArray<FAssetData> AssetDataList;
		AssetRegistry.GetAssets( Filter, AssetDataList );

		ObjectsToDelete.Reserve( AssetDataList.Num() );
		for(const FAssetData& AssetData : AssetDataList)
		{
			FSoftObjectPath ObjectPath( AssetData.GetSoftObjectPath() );

			if(UObject* Object = ObjectPath.ResolveObject())
			{
				ObjectsToDelete.Add( Object );
			}
		}
	}

	// Find all packages under the temporary path
	{
		for (TObjectIterator<UPackage> It; It; ++It)
		{
			UPackage*	Package = *It;

			if(Package->GetName().StartsWith( BaseTemporaryPath ))
			{
				if (AssetRegistry.PathExists(Package->GetPathName()))
				{
					// Remove package path from asset registry
					AssetRegistry.RemovePath(Package->GetPathName());
				}

				ObjectsToDelete.Add( Package );
			}
		}
	}

	if(ObjectsToDelete.Num() > 0)
	{
		PurgeObjects( MoveTemp( ObjectsToDelete ) );
	}
	// No object to delete, force a garbage collection anyway
	else
	{
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
	}

	// Delete all assets on disk
	{
		struct FEmptyFolderVisitor : public IPlatformFile::FDirectoryVisitor
		{
			bool bIsEmpty;

			FEmptyFolderVisitor()
				: bIsEmpty( true )
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if ( !bIsDirectory )
				{
					bIsEmpty = false;
					return false; // abort searching
				}

				return true; // continue searching
			}
		};

		IFileManager& FileManager = IFileManager::Get();

		FString PathToDeleteOnDisk;
		if ( FPackageName::TryConvertLongPackageNameToFilename( BaseTemporaryPath, PathToDeleteOnDisk ) )
		{
			if( FileManager.DirectoryExists( *PathToDeleteOnDisk ) )
			{
				// Look for files on disk in case the folder contains things not tracked by the asset registry
				FEmptyFolderVisitor EmptyFolderVisitor;
				FileManager.IterateDirectoryRecursively( *PathToDeleteOnDisk, EmptyFolderVisitor );

				if ( EmptyFolderVisitor.bIsEmpty && IFileManager::Get().DeleteDirectory( *PathToDeleteOnDisk, false, true ) )
				{
					AssetRegistry.RemovePath( BaseTemporaryPath );
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
