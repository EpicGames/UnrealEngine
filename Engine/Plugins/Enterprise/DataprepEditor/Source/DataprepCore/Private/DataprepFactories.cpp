// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepFactories.h"

#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepContentConsumer.h"
#include "DataprepCoreLogCategory.h"
#include "Shared/DataprepCorePrivateUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/UObjectIterator.h"

UDataprepAssetFactory::UDataprepAssetFactory()
{
	SupportedClass = UDataprepAsset::StaticClass();

	bCreateNew = true;
	bText = false;
	bEditorImport = false;
}

bool UDataprepAssetFactory::ShouldShowInNewMenu() const
{
	return true;
}

UObject * UDataprepAssetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext *Warn)
{
	if ( !InClass )
	{
		// Default to dataprep asset
		InClass = UDataprepAsset::StaticClass();
	}
	else if ( !InClass->IsChildOf( UDataprepAsset::StaticClass() ) )
	{
		return nullptr;
	}

	// Find potential Consumer classes
	TArray<UClass*> ConsumerClasses;
	for( TObjectIterator< UClass > It ; It ; ++It )
	{
		UClass* CurrentClass = (*It);

		if ( !CurrentClass->HasAnyClassFlags( CLASS_Abstract ) )
		{
			if( CurrentClass->IsChildOf( UDataprepContentConsumer::StaticClass() ) )
			{
				ConsumerClasses.Add( CurrentClass );
			}
		}
	}

	UDataprepAsset* DataprepAsset = NewObject<UDataprepAsset>(InParent, InClass, InName, Flags | RF_Transactional);
	check(DataprepAsset);

	if(ConsumerClasses.Num() > 0)
	{
		// Initialize Dataprep asset's consumer
		if(ConsumerClasses.Num() == 1)
		{
			DataprepAsset->SetConsumer( ConsumerClasses[0], /* bNotifyChanges = */ false );
		}
		else
		{
			// #ueent_todo: Propose user to choose from the list of Consumers.
			DataprepAsset->SetConsumer( ConsumerClasses[0], /* bNotifyChanges = */ false );
		}
		check( DataprepAsset->GetConsumer() );
	}
	else
	{
		UE_LOGF( LogDataprepCore, Warning, "no Dataprep content consumers found" );
	}

	DataprepAsset->CreateParameterization();

	FAssetRegistryModule::AssetCreated( DataprepAsset );
	DataprepAsset->MarkPackageDirty();

	DataprepCorePrivateUtils::Analytics::DataprepAssetCreated( DataprepAsset );
	
	return DataprepAsset;
}

UDataprepAssetInstanceFactory::UDataprepAssetInstanceFactory()
{
	SupportedClass = UDataprepAssetInstance::StaticClass();

	bCreateNew = false;
	bText = false;
	bEditorImport = false;
}

UObject* UDataprepAssetInstanceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if( Parent )
	{
		UDataprepAssetInstance* DataprepAssetInstance = NewObject<UDataprepAssetInstance>(InParent, InClass, InName, Flags);

		if(DataprepAssetInstance)
		{
			if (!Parent->GetConsumer())
			{
				UE_LOGF( LogDataprepCore, Warning, "no Dataprep content consumers found" );
			}

			if(DataprepAssetInstance->SetParent(Parent, /* bNotifyChanges = */ false))
			{
				FAssetRegistryModule::AssetCreated( DataprepAssetInstance );
				DataprepAssetInstance->MarkPackageDirty();

				DataprepCorePrivateUtils::Analytics::DataprepAssetCreated( DataprepAssetInstance );

				return DataprepAssetInstance;
			}
		}
	}

	return nullptr;
}
