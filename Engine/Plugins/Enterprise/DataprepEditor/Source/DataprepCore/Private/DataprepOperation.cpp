// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepOperation.h"

#include "DataprepCoreLogCategory.h"
#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"

#define LOCTEXT_NAMESPACE "DataprepOperation"

FText FDataprepOperationCategories::ActorOperation(LOCTEXT("DataprepOperation_ActorOperationName", "On Actor"));
FText FDataprepOperationCategories::AssetOperation(LOCTEXT("DataprepOperationAssetOperationName", "On Asset"));
FText FDataprepOperationCategories::MeshOperation( LOCTEXT("DataprepOperation_MeshOperationName", "On Mesh"));
FText FDataprepOperationCategories::ObjectOperation( LOCTEXT("DataprepOperation_ObjectOperationName", "On Object"));

void UDataprepOperation::Execute(const TArray<UObject *>& InObjects)
{
	FDataprepContext Context;
	Context.Objects = InObjects;
	OnExecution( Context );
}

void UDataprepOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	// This function should never called on a UDataprepOperation since this class is a abstract base.
	LogError( LOCTEXT("OnExecutionNotOverrided","Please define an implementation to OnExecution for your operation.") );
}

void UDataprepOperation::LogInfo(const FText& InLogText)
{
	if ( OperationContext && OperationContext->DataprepLogger )
	{
		OperationContext->DataprepLogger->LogInfo( InLogText, *this );
	}
}

void UDataprepOperation::LogWarning(const FText& InLogText)
{
	if ( OperationContext && OperationContext->DataprepLogger )
	{
		OperationContext->DataprepLogger->LogWarning( InLogText, *this );
	}
}

void UDataprepOperation::LogError(const FText& InLogText)
{
	if ( OperationContext && OperationContext->DataprepLogger )
	{
		OperationContext->DataprepLogger->LogError( InLogText, *this );
	}
}

void UDataprepOperation::BeginWork( const FText& InDescription, float InAmountOfWork )
{
	if ( OperationContext && OperationContext->DataprepProgressReporter )
	{
		OperationContext->DataprepProgressReporter->BeginWork( InDescription, InAmountOfWork );
	}
}

void UDataprepOperation::EndWork()
{
	if ( OperationContext && OperationContext->DataprepProgressReporter )
	{
		OperationContext->DataprepProgressReporter->EndWork();
	}
}

void UDataprepOperation::ReportProgress( float IncrementOfWork, const FText& InMessage )
{
	if ( OperationContext && OperationContext->DataprepProgressReporter )
	{
		OperationContext->DataprepProgressReporter->ReportProgress( IncrementOfWork, InMessage );
	}
}

TSharedPtr<FDataprepWorkReporter> UDataprepOperation::CreateTask(const FText & InDescription, float InAmountOfWork, float InIncrementOfWork)
{
	if( OperationContext )
	{
		return TSharedPtr<FDataprepWorkReporter>( new FDataprepWorkReporter( OperationContext->DataprepProgressReporter, InDescription, InAmountOfWork, InIncrementOfWork ) );
	}

	return TSharedPtr<FDataprepWorkReporter>();
}

bool UDataprepOperation::IsCancelled()
{
	return OperationContext && OperationContext->DataprepProgressReporter ? OperationContext->DataprepProgressReporter->IsWorkCancelled() : false;
}


void UDataprepOperation::ExecuteOperation(TSharedRef<FDataprepOperationContext>&  InOperationContext)
{
	OperationContext = InOperationContext;
	if ( OperationContext->Context.IsValid() )
	{
		OnExecution( *OperationContext->Context );
	}
	else
	{
		ensureMsgf( false, TEXT("ExcuteOperation should never be called with an operation context without a context!") );
	}
}

FText UDataprepOperation::GetDisplayOperationName_Implementation() const
{
	return this->GetClass()->GetDisplayNameText();
}

FText UDataprepOperation::GetTooltip_Implementation() const
{
	return this->GetClass()->GetToolTipText();
}

FText UDataprepOperation::GetCategory_Implementation() const
{
	return LOCTEXT("DefaultOperationCategory", "User-defined");
}

FText UDataprepOperation::GetAdditionalKeyword_Implementation() const
{
	return FText();
}

void UDataprepOperation::AssetsModified( TArray<UObject*> Assets )
{
	if ( OperationContext && OperationContext->AssetsModifiedDelegate.IsBound() )
	{
		OperationContext->AssetsModifiedDelegate.Execute( MoveTemp( Assets ) );
	}
}

UObject* UDataprepEditingOperation::AddAsset( const UObject* Asset, const FString& AssetName )
{
	if ( OperationContext && OperationContext->AddAssetDelegate.IsBound() )
	{
		return OperationContext->AddAssetDelegate.Execute( Asset, AssetName.Len() > 0 ? *AssetName : nullptr );
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::AddAsset called without context" );

	return nullptr;
}

UObject* UDataprepEditingOperation::CreateAsset( UClass* AssetClass, const FString& AssetName )
{
	if ( OperationContext && OperationContext->CreateAssetDelegate.IsBound() )
	{
		return OperationContext->CreateAssetDelegate.Execute( AssetClass, AssetName.Len() > 0 ? *AssetName : nullptr );
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::CreateAsset called without context" );

	return nullptr;
}

AActor * UDataprepEditingOperation::CreateActor(UClass* ActorClass, const FString& ActorName)
{
	if ( OperationContext && OperationContext->CreateActorDelegate.IsBound() )
	{
		return OperationContext->CreateActorDelegate.Execute( ActorClass, ActorName.Len() > 0 ? *ActorName : nullptr );
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::CreateActor called without context" );

	return nullptr;
}

void UDataprepEditingOperation::RemoveObject( UObject* Object, bool bLocalContext )
{
	if ( OperationContext && OperationContext->RemoveObjectDelegate.IsBound() )
	{
		OperationContext->RemoveObjectDelegate.Execute( Object, bLocalContext );
		return;
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::RemoveObject called without context" );
}

void UDataprepEditingOperation::RemoveObjects(TArray<UObject*> Objects, bool bLocalContext)
{
	if ( OperationContext && OperationContext->RemoveObjectDelegate.IsBound() )
	{
		for(UObject* Object : Objects)
		{
			OperationContext->RemoveObjectDelegate.Execute( Object, bLocalContext );
		}
		return;
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::RemoveObjects called without context" );
}

void UDataprepEditingOperation::DeleteObject( UObject* Object )
{
	if ( OperationContext && OperationContext->DeleteObjectsDelegate.IsBound() )
	{
		TArray<UObject*> Objects;
		Objects.Add( Object );
		OperationContext->DeleteObjectsDelegate.Execute( MoveTemp( Objects ) );
		return;
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::DeleteObject called without context" );
}

void UDataprepEditingOperation::DeleteObjects( TArray<UObject*> Objects )
{
	if ( OperationContext && OperationContext->DeleteObjectsDelegate.IsBound() )
	{
		OperationContext->DeleteObjectsDelegate.Execute( MoveTemp( Objects ) );
	}

	UE_LOGF( LogDataprepCore, Log, "UDataprepEditingOperation::DeleteObjects called without context" );
}

#undef LOCTEXT_NAMESPACE
