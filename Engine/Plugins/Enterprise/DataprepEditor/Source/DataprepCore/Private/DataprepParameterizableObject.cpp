// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepParameterizableObject.h"



void UDataprepParameterizableObject::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty( PropertyChangedEvent );

	if ( !bool( PropertyChangedEvent.ChangeType & ( EPropertyChangeType::Interactive ) ) )
	{
		OnPostEdit.Broadcast( *this, PropertyChangedEvent );
	}
}

