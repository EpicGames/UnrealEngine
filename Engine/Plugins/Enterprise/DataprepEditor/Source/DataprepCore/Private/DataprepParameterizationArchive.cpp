// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepParameterizationArchive.h"

#include "Parameterization/DataprepParameterization.h"

#include "Misc/PackageName.h"
#include "UObject/Package.h"

//////////////////////////////////////////////////////////////
/**
 * FDataprepParameterizationWriter
 */

FArchive& FDataprepParameterizationWriter::operator<<(FName& Value)
{
	FString NameAsString = Value.ToString();
	*this << (NameAsString);
	return *this;
}

FArchive& FDataprepParameterizationWriter::operator<<(UObject*& Value)
{
	FString Path;
	if ( Value && Value->IsValidLowLevelFast() )
	{
		if ( UObject* Outer = Value->GetOuter() )
		{
			checkf( Outer->GetClass() != UDataprepParameterization::StaticClass(), TEXT("A dataprep parameterization can't have a Subobject") );
		}

		Path = Value->GetPathName();
	}

	return *this << Path;
}

FArchive& FDataprepParameterizationWriter::operator<<(FLazyObjectPtr& Value)
{
	UObject* Object = Value.Get();
	return *this << Object;
}

FArchive& FDataprepParameterizationWriter::operator<<(FObjectPtr& Value)
{
	UObject* Object = Value.Get();
	return *this << Object;
}

FArchive& FDataprepParameterizationWriter::operator<<(FSoftObjectPtr& Value)
{
	FSoftObjectPath SoftObjectPath = Value.ToSoftObjectPath();
	return *this << SoftObjectPath;
}

FArchive& FDataprepParameterizationWriter::operator<<(FSoftObjectPath& Value)
{
	FString PathInString = Value.ToString();
	return *this << PathInString;
}

FArchive& FDataprepParameterizationWriter::operator<<(FWeakObjectPtr& Value)
{
	UObject* Object = Value.Get();
	return *this << Object;
}

FString FDataprepParameterizationWriter::GetArchiveName() const
{
	return TEXT("FDataprepParameterizationWriter");
}


//////////////////////////////////////////////////////////////
/**
 * FDataprepParameterizationReader
 */

FArchive& FDataprepParameterizationReader::operator<<(FName& Value)
{
	FString NameAsString;
	*this << NameAsString;
	Value = FName(*NameAsString);
	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(UObject*& Value)
{
	FString ObjectPath;
	*this << ObjectPath;

	if ( ObjectPath.IsEmpty() )
	{
		Value = nullptr;
	}
	else
	{
		// Always attempt to find an in-memory object first
		Value = StaticFindObject( UObject::StaticClass(), nullptr, *ObjectPath );

		if ( !Value )
		{
			// If the outer name is a package path that isn't currently loaded, then we need to try loading it to avoid 
			// creating an in-memory version of the package (which would prevent the real package ever loading)
			if ( FPackageName::IsValidLongPackageName( ObjectPath ) )
			{
				Value = LoadPackage( nullptr, *ObjectPath, LOAD_NoWarn );
			}
			else
			{
				Value = StaticLoadObject( UObject::StaticClass(), nullptr, *ObjectPath );
			}
		}
	}

	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(FLazyObjectPtr& Value)
{
	UObject* Object = nullptr;
	*this << Object;
	Value = Object != nullptr ? FUniqueObjectGuid(Object) : FUniqueObjectGuid();
	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(FObjectPtr& Value)
{
	UObject* Object = nullptr;
	*this << Object;
	Value = Object;
	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(FSoftObjectPtr& Value)
{
	FSoftObjectPath SoftObjectPath;
	*this << SoftObjectPath;
	Value = SoftObjectPath;
	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(FSoftObjectPath& Value)
{
	FString String;
	*this << String;
	Value.SetPath( String );
	return *this;
}

FArchive& FDataprepParameterizationReader::operator<<(FWeakObjectPtr& Value)
{
	UObject* Object = nullptr;
	*this << Object;
	Value = Object;
	return *this;
}

FString FDataprepParameterizationReader::GetArchiveName() const
{
	return TEXT("FDataprepParameterizationReader");
}
