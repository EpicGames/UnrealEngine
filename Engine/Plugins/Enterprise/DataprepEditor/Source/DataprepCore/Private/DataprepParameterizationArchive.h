// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"

class FDataprepParameterizationWriter : public FMemoryWriter
{
public:
	FDataprepParameterizationWriter(UObject* Object, TArray<uint8>& InBytes)
		: FMemoryWriter(InBytes, true)
	{
		InBytes.Empty( InBytes.Num() );

		ArIgnoreClassRef = true;
		ArIgnoreArchetypeRef = true;
		ArNoDelta = true;
		ArPortFlags |= PPF_ForceTaggedSerialization;

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Object));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		Object->Serialize(*this);
	}

	// Avoid hiding non overrided overloads
	using FArchive::operator<<;

	virtual FArchive& operator<<(FName& Value) override;

	virtual FArchive& operator<<(UObject*& Value) override;

	virtual FArchive& operator<<(FText& Value) override
	{
		// todo
		return *this;
	}

	virtual FArchive& operator<<(struct FLazyObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FSoftObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FSoftObjectPath& Value) override;

	virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;

	virtual FString GetArchiveName() const override;

};

class FDataprepParameterizationReader : public FMemoryReader
{
public:
	FDataprepParameterizationReader(UObject* Object, TArray<uint8>& InBytes)
		: FMemoryReader(InBytes, true)
	{
		if ( InBytes.Num() > 0 )
		{ 
			this->SetIsLoading(true);
			this->SetIsPersistent(false);
			ArIgnoreClassRef = true;
			ArIgnoreArchetypeRef = true;

#if USE_STABLE_LOCALIZATION_KEYS
			if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
			{
				SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Object));
			}
#endif // USE_STABLE_LOCALIZATION_KEYS

			Object->Serialize(*this);
		}
	}


	// Avoid hiding non overrided overloads
	using FArchive::operator<<;

	virtual FArchive& operator<<(FName& Value) override;
	
	virtual FArchive& operator<<(UObject*& Value) override;

	virtual FArchive& operator<<(FText& Value) override
	{
		// todo
		return *this;
	}

	virtual FArchive& operator<<(struct FLazyObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FSoftObjectPtr& Value) override;

	virtual FArchive& operator<<(struct FSoftObjectPath& Value) override;

	virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;
	
	virtual FString GetArchiveName() const override;

};


