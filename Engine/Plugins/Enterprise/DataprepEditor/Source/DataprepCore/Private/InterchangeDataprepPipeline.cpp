// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDataprepPipeline.h"

#include "InterchangeActorFactoryNode.h"
#include "InterchangeLevelSequenceFactoryNode.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeSceneImportAssetFactoryNode.h"
#include "InterchangeSceneVariantSetsFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

void UInterchangeDataprepLevelPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!NodeContainer)
	{
		return;
	}

	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	// Compute unique prefix based on file path
	const FString PackageSubPath = FPaths::GetBaseFilename(InSourceDatas[0]->GetFilename());

	auto UpdateFactoryNodes = [this,&PackageSubPath](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
	{
		// Append prefix to prevent actor's name collision in Dataprep world
		if (UInterchangeActorFactoryNode* ActorFactoryNode = Cast< UInterchangeActorFactoryNode>(FactoryNode))
		{
			FString NewLabel = PackageSubPath + TEXT("_") + ActorFactoryNode->GetDisplayLabel();
			ActorFactoryNode->SetDisplayLabel(NewLabel);
		}
		else if (UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(FactoryNode))
		{
			TextureFactoryNode->SetCustomSubPath(TEXT("Textures"));
		}
		else if (UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(FactoryNode))
		{
			if (MaterialFactoryNode->IsA<UInterchangeMaterialFactoryNode>())
			{
				MaterialFactoryNode->SetCustomSubPath(TEXT("Materials/References"));
			}
			else if (MaterialFactoryNode->IsA<UInterchangeMaterialFunctionFactoryNode>())
			{
				MaterialFactoryNode->SetCustomSubPath(TEXT("Materials/References/Functions"));
			}
			else
			{
				MaterialFactoryNode->SetCustomSubPath(TEXT("Materials"));
			}
		}
		else if (UInterchangeStaticMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(FactoryNode))
		{
			MeshFactoryNode->SetCustomSubPath(TEXT("Geometries"));
		}
		else if (UInterchangeLevelSequenceFactoryNode* SequenceFactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(FactoryNode))
		{
			SequenceFactoryNode->SetCustomSubPath(TEXT("Animations"));
		}
		else if (UInterchangeSceneVariantSetsFactoryNode* VariantFactoryNode = Cast<UInterchangeSceneVariantSetsFactoryNode>(FactoryNode))
		{
			VariantFactoryNode->SetCustomSubPath(TEXT("Variants"));
		}
		// Remove UInterchangeSceneImportAssetFactoryNode from the import
		// #dataprep_todo : Revisit when consumer will have been updated.
		else if (UInterchangeSceneImportAssetFactoryNode* SceneImportFactoryNode = Cast<UInterchangeSceneImportAssetFactoryNode>(FactoryNode))
		{
			SceneImportFactoryNode->SetEnabled(false);
		}
	};

	//Find all factory node we need for this pipeline
	NodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>(UpdateFactoryNodes);
}

void UInterchangeDataprepLevelPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* NodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (!NodeContainer || !CreatedAsset)
	{
		return;
	}

	Super::ExecutePostImportPipeline(NodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);

	CreatedAsset->ClearFlags(RF_Public);
	CreatedAsset->SetFlags(RF_Transactional|RF_Standalone);
}
