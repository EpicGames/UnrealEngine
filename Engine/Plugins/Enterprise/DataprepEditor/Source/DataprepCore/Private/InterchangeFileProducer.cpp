// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeFileProducer.h"

#include "DataprepAssetProducers.h"
#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"
#include "InterchangeDataprepPipeline.h"

#include "InterchangeProjectSettings.h"
#include "InterchangeManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/ARFilter.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "InterchangeFileProducer"

const FText InterchangeFileProducerLabel( LOCTEXT( "InterchangeFileProducerLabel", "Interchange file producer" ) );
const FText InterchangeFileProducerDescription( LOCTEXT( "InterchangeFileProducerDesc", "Reads files of formats supported by Interchange, i.e. FBX, glTF, ..." ) );

bool bGEnableInterchangeProducer = true;
FAutoConsoleVariableRef GEnableInterchangeProducerVar(
	TEXT("Dataprep.Interchange.Producer"),
	bGEnableInterchangeProducer,
	TEXT("Enable/disable the Interchange file producer.\nDefault is enabled\n"),
	ECVF_Default
);

namespace FInterchangeFileProducerUtils
{
	/** Delete all the assets stored under the specified path */
	void DeletePackagePath( const FString& PathToDelete );

	/** Delete all the packages created by the Interchange importer */
	void DeletePackagesPath(TSet<FString>& PathsToDelete)
	{
		for(FString& PathToDelete : PathsToDelete)
		{
			DeletePackagePath( PathToDelete );
		}
	}

	/** Display OS browser, i.e. Windows explorer, to let user select a file */
	FString SelectFileToImport();

	/** Display OS browser, i.e. Windows explorer, to let user select a directory */
	FString SelectDirectory();


	class SFileProducerImportOptionsWindow : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SFileProducerImportOptionsWindow)
		{}
		SLATE_ARGUMENT(TArray<UObject*>, ImportOptions)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(int32, MinDetailHeight)
		SLATE_ARGUMENT(int32, MinDetailWidth)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs)
		{
			ImportOptions = InArgs._ImportOptions;
			Window = InArgs._WidgetWindow;

			TSharedPtr<SBox> DetailsViewBox;

			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SAssignNew(DetailsViewBox, SBox)
					.MinDesiredHeight(InArgs._MinDetailHeight)
					.MinDesiredWidth(InArgs._MinDetailWidth)
				]
			];

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

			DetailsViewBox->SetContent(DetailsView.ToSharedRef());
			DetailsView->SetObjects(ImportOptions);
		}

		virtual bool SupportsKeyboardFocus() const override { return true; }

		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
		{
			if (InKeyEvent.GetKey() == EKeys::Escape)
			{
				if (Window.IsValid())
				{
					Window.Pin()->RequestDestroyWindow();
				}
				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

	private:
		TArray<UObject*> ImportOptions;
		TWeakPtr< SWindow > Window;
	};

	void GetPipelinesFromSourceData(UInterchangeSourceData& SourceData, TArray<FSoftObjectPath>& OutPipelines)
	{
		const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(true);
		const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;
		FName DefaultStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(true, SourceData);
		
		OutPipelines.Empty();

		UE::Interchange::FScopedTranslator ScopedTranslator(&SourceData);

		for (const TPair<FName, FInterchangePipelineStack>& PipelineStackInfo : DefaultPipelineStacks)
		{
			if (PipelineStackInfo.Key == DefaultStackName)
			{
				const FInterchangePipelineStack& PipelineStack = PipelineStackInfo.Value;
				OutPipelines = PipelineStack.Pipelines;

				for (const FInterchangeTranslatorPipelines& TranslatorPipelines : PipelineStack.PerTranslatorPipelines)
				{
					const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
					if (ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
					{
						OutPipelines = TranslatorPipelines.Pipelines;
						break;
					}
				}

				break;
			}
		}
	}
}

UInterchangeFileProducer::UInterchangeFileProducer()
{
}

bool UInterchangeFileProducer::IsActive()
{
	return bGEnableInterchangeProducer;
}

bool UInterchangeFileProducer::Initialize()
{
	FText TaskDescription = FText::Format( LOCTEXT( "InterchangeFileProducer_LoadingFile", "Loading {0} ..."), FText::FromString( FilePath ) );
	ProgressTaskPtr = MakeUnique< FDataprepWorkReporter >( Context.ProgressReporterPtr, TaskDescription, 10.0f, 1.0f );

	ProgressTaskPtr->ReportNextStep( TaskDescription, 7.0f );

	if( FilePath.IsEmpty() )
	{
		LogError( LOCTEXT( "InterchangeFileProducer_Incomplete", "No file has been selected." ) );
		return false;
	}

	// Check file exists
	if(!FPaths::FileExists( FilePath ))
	{
		LogError( FText::Format( LOCTEXT( "InterchangeFileProducer_NotFound", "File {0} does not exist." ), FText::FromString( FilePath ) ) );
		return false;
	}

	// Create transient package if it wasn't specified
	if (!TransientPackage)
	{
		TransientPackage = NewObject< UPackage >(nullptr, *FPaths::Combine(Context.RootPackagePtr->GetPathName(), *GetName()), RF_Transient);
	}

	TransientPackage->FullyLoad();

	constexpr EObjectFlags LocalObjectFlags = RF_Public | RF_Standalone | RF_Transactional;
	FFeedbackContext* InFeedbackContext = Context.ProgressReporterPtr ? Context.ProgressReporterPtr->GetFeedbackContext() : nullptr;

	return true;
}

bool UInterchangeFileProducer::InitTranslator()
{
	return true;
}

bool UInterchangeFileProducer::Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets)
{
	using namespace FInterchangeFileProducerUtils;
	using namespace UE::Interchange;
	using FInterchangeResults = TTuple<FAssetImportResultRef, FSceneImportResultRef>;

	if ( !IsValid() || IsCancelled() )
	{
		return false;
	}

	ProgressTaskPtr->ReportNextStep( FText::Format( LOCTEXT( "InterchangeFileProducer_ConvertingFile", "Converting {0} ..."), FText::FromString( FilePath ) ), 2.0f );

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	FScopedSourceData ScopedSourceData(FilePath);

	UInterchangeSourceData* SourceData = ScopedSourceData.GetSourceData();

	if (!InterchangeManager.CanTranslateSourceData(SourceData))
	{
		return false;
	}

	FImportAssetParameters ImportAssetParameters;
	
	ImportAssetParameters.bIsAutomated = true;
	ImportAssetParameters.ImportLevel = Context.WorldPtr.Get()->GetCurrentLevel();

	GetPipelinesFromSourceData(*SourceData, ImportAssetParameters.OverridePipelines);
	ImportAssetParameters.OverridePipelines.Add(FSoftObjectPath(TEXT("/DataprepEditor/DefaultDataprepPipeline.DefaultDataprepPipeline")));

	const FString PackagePath = TransientPackage->GetPathName();
	FInterchangeResults ImportResults = InterchangeManager.ImportSceneAsync(PackagePath, SourceData, ImportAssetParameters);

	ImportResults.Get<0>()->WaitUntilDone();
	ImportResults.Get<1>()->WaitUntilDone();

	if (!ImportResults.Key->IsValid() || !ImportResults.Value->IsValid())
	{
		return false;
	}


	TSet<UObject*> AssetsToAdd;
	auto AddAssets = [&AssetsToAdd, &PackagePath](const TArray<UObject*>& Objects)
		{
			for (UObject* Object : Objects)
			{
				if (Object && Object->GetPathName().StartsWith(PackagePath) && !Object->IsA<AActor>())
				{
					AssetsToAdd.Add(Object);
				}
			}
		};

	AddAssets(ImportResults.Get<0>()->GetImportedObjects());
	AddAssets(ImportResults.Get<1>()->GetImportedObjects());

	OutAssets.Reserve(OutAssets.Num() + AssetsToAdd.Num());
	OutAssets.Append(AssetsToAdd.Array());

	return !IsCancelled();
}

void UInterchangeFileProducer::OnFilePathChanged()
{
	FilePath = FPaths::ConvertRelativePathToFull( FilePath );
	UpdateName();
	OnChanged.Broadcast( this );
}

void UInterchangeFileProducer::Reset()
{
	ProgressTaskPtr.Reset();
	Assets.Empty();
	TransientPackage = nullptr;

	UDataprepContentProducer::Reset();
}

const FText& UInterchangeFileProducer::GetLabel() const
{
	return InterchangeFileProducerLabel;
}

const FText& UInterchangeFileProducer::GetDescription() const
{
	return InterchangeFileProducerDescription;
}

FString UInterchangeFileProducer::GetNamespace() const
{
	return {};
}

void UInterchangeFileProducer::SetFilePath( const FString& InFilePath )
{
	Modify();

	FilePath = InFilePath;

	OnFilePathChanged();
}

void UInterchangeFileProducer::UpdateName()
{
	if(!FilePath.IsEmpty())
	{
		// Rename producer to name of file
		FString CleanName = ObjectTools::SanitizeObjectName( FPaths::GetCleanFilename( FilePath ) );
		if ( !Rename( *CleanName, nullptr, REN_Test ) )
		{
			CleanName = MakeUniqueObjectName( GetOuter(), GetClass(), *CleanName ).ToString();
		}

		Rename( *CleanName, nullptr, REN_DontCreateRedirectors | REN_NonTransactional );
	}
}

bool UInterchangeFileProducer::Supersede(const UDataprepContentProducer* OtherProducer) const
{
	const UInterchangeFileProducer* OtherFileProducer = Cast<const UInterchangeFileProducer>(OtherProducer);

	return OtherFileProducer != nullptr &&
		!OtherFileProducer->FilePath.IsEmpty() &&
		FilePath == OtherFileProducer->FilePath;
}

bool UInterchangeFileProducer::CanAddToProducersArray(bool bIsAutomated)
{
	if ( bIsAutomated )
	{
		return true;
	}

	FilePath = FInterchangeFileProducerUtils::SelectFileToImport();
	if ( !FilePath.IsEmpty() )
	{
		UpdateName();
		return true;
	}

	return false;
}

void UInterchangeFileProducer::PostEditUndo()
{
	UDataprepContentProducer::PostEditUndo();

	OnChanged.Broadcast( this );
}

void UInterchangeFileProducer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* EditedProperty = PropertyChangedEvent.Property;

	if ( !EditedProperty || EditedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInterchangeFileProducer, FilePath) )
	{
		OnFilePathChanged();
	}
}

class SInterchangeFileProducerFileProperty : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInterchangeFileProducerFileProperty)
	{}

	SLATE_ARGUMENT(UInterchangeFileProducer*, Producer)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		ProducerPtr = TWeakObjectPtr< UInterchangeFileProducer >( InArgs._Producer );

		FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(FileName, SEditableText)
				.IsReadOnly(true)
				.Text(this, &SInterchangeFileProducerFileProperty::GetFilenameText)
				.ToolTipText(this, &SInterchangeFileProducerFileProperty::GetFilenameText)
				.Font( FontInfo )
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SInterchangeFileProducerFileProperty::OnChangePathClicked )
				.ToolTipText(LOCTEXT("ChangeSourcePath_Tooltip", "Browse for a new source file path"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("...", "..."))
					.Font( FontInfo )
				]
			]
		];
	}

private:
	FReply OnChangePathClicked() const
	{
		if( !ProducerPtr.IsValid() )
		{
			return FReply::Unhandled();
		}

		FString SelectedFile = FInterchangeFileProducerUtils::SelectFileToImport();
		if(!SelectedFile.IsEmpty())
		{
			const FScopedTransaction Transaction( LOCTEXT("Producer_SetFilename", "Set Filename") );

			ProducerPtr->SetFilePath( SelectedFile );
			FileName->SetText( GetFilenameText() );
		}

		return FReply::Handled();
	}

	FText GetFilenameText() const
	{
		return ProducerPtr->FilePath.IsEmpty() ? FText::FromString( TEXT("Select a file") ) : FText::FromString( ProducerPtr->FilePath );
	}

private:
	TWeakObjectPtr< UInterchangeFileProducer > ProducerPtr;
	TSharedPtr< SEditableText > FileName;
};

void FInterchangeContentProducerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(Objects.Num() > 0);

	Producer = Cast< UDataprepContentProducer >(Objects[0].Get());
	check(Producer);

	AssetProducers = Cast<UDataprepAssetProducers>(Producer->GetOuter());
	check(AssetProducers);

	ProducerIndex = INDEX_NONE;
	for (int Index = 0; Index < AssetProducers->GetProducersCount(); ++Index)
	{
		if (Producer == AssetProducers->GetProducer(Index))
		{
			ProducerIndex = Index;
			break;
		}
	}
}

FSlateColor FInterchangeContentProducerDetails::GetStatusColorAndOpacity() const
{
	return  IsProducerSuperseded() ? FLinearColor::Red : FAppStyle::Get().GetSlateColor("DefaultForeground");
}

bool FInterchangeContentProducerDetails::IsProducerSuperseded() const
{
	return ProducerIndex != INDEX_NONE ? AssetProducers->IsProducerSuperseded(ProducerIndex) : false;
}

void FInterchangeFileProducerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FInterchangeContentProducerDetails::CustomizeDetails(DetailBuilder);

	UInterchangeFileProducer* FileProducer = Cast< UInterchangeFileProducer >(Producer);
	check( FileProducer );

	// #ueent_todo: Remove handling of warning category when this is not considered experimental anymore
	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );
	CategoryNames.Remove( FName(TEXT("Warning")) );

	DetailBuilder.HideCategory(FName( TEXT( "Warning" ) ) );

	//FName CategoryName( TEXT("InterchangeFileProducerCustom") );
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory( NAME_None, FText::GetEmpty(), ECategoryPriority::Important );

	TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty( TEXT("FilePath"), UInterchangeFileProducer::StaticClass() );
	PropertyHandle->MarkHiddenByCustomization();

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow( FText::FromString( TEXT( "Import File" ) ) );

	TSharedPtr<STextBlock> IconText;

	CustomAssetImportRow.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 3, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.IsEnabled(false) // Options button
			.OnClicked_Lambda( [FileProducer]() -> FReply 
			{ 
				//FileProducer->OnChangeImportSettings(); 
				return FReply::Handled(); 
			})
			.ToolTipText(LOCTEXT("ChangeImportSettings_Tooltip", "Import Settings"))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.ColorAndOpacity(FLinearColor::White)
				.Text(FEditorFontGlyphs::Cog)
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew(IconText, STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(MakeAttributeLambda([this]
				{
					return this->IsProducerSuperseded() ? FEditorFontGlyphs::Exclamation_Triangle : FEditorFontGlyphs::File;
				}))
				.ColorAndOpacity(this, &FInterchangeContentProducerDetails::GetStatusColorAndOpacity)
		]
	];

	IconText->SetToolTipText(MakeAttributeLambda([this]
	{
		if (IsProducerSuperseded())
		{
			return LOCTEXT("FInterchangeFileProducerDetails_StatusTextTooltip_Superseded", "This producer is superseded by another one and will be skipped when run.");
		}
		return LOCTEXT("FInterchangeFileProducerDetails_StatusTextTooltip", "File input");
	}));

	CustomAssetImportRow.ValueContent()
	.MinDesiredWidth( 2000.0f )
	[
		SNew( SInterchangeFileProducerFileProperty )
		.Producer( FileProducer )
	];
}

void FInterchangeFileProducerUtils::DeletePackagePath( const FString& PathToDelete )
{
	if(PathToDelete.IsEmpty())
	{
		return;
	}

	// Inspired from ContentBrowserUtils::DeleteFolders
	// Inspired from ContentBrowserUtils::LoadAssetsIfNeeded

	// Form a filter from the paths
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	new (Filter.PackagePaths) FName(*PathToDelete);

	// Query for a list of assets in the selected paths
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetDataList);

	bool bAllowFolderDelete = false;

	{
		struct FEmptyFolderVisitor : public IPlatformFile::FDirectoryVisitor
		{
			bool bIsEmpty;

			FEmptyFolderVisitor()
				: bIsEmpty(true)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (!bIsDirectory)
				{
					bIsEmpty = false;
					return false; // abort searching
				}

				return true; // continue searching
			}
		};

		FString PathToDeleteOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(PathToDelete, PathToDeleteOnDisk))
		{
			// Look for files on disk in case the folder contains things not tracked by the asset registry
			FEmptyFolderVisitor EmptyFolderVisitor;
			IFileManager::Get().IterateDirectoryRecursively(*PathToDeleteOnDisk, EmptyFolderVisitor);

			if (EmptyFolderVisitor.bIsEmpty && IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, false, true))
			{
				AssetRegistryModule.Get().RemovePath(PathToDelete);
			}
		}
	}
}

FString FInterchangeFileProducerUtils::SelectFileToImport()
{
	TArray<FString> Formats = UInterchangeManager::GetInterchangeManager().GetSupportedFormats(EInterchangeTranslatorType::Scenes);

	FString FileTypes;
	FString AllExtensions;

	for( const FString& Format : Formats )
	{
		TArray<FString> FormatComponents;
		Format.ParseIntoArray( FormatComponents, TEXT( ";" ), false );

		for ( int32 ComponentIndex = 0; ComponentIndex < FormatComponents.Num(); ComponentIndex += 2 )
		{
			check( FormatComponents.IsValidIndex( ComponentIndex + 1 ) );
			const FString& Extension = FormatComponents[ComponentIndex];
			const FString& Description = FormatComponents[ComponentIndex + 1];

			if ( !AllExtensions.IsEmpty() )
			{
				AllExtensions.AppendChar( TEXT( ';' ) );
			}
			AllExtensions.Append( TEXT( "*." ) );
			AllExtensions.Append( Extension );

			if ( !FileTypes.IsEmpty() )
			{
				FileTypes.AppendChar( TEXT( '|' ) );
			}

			FileTypes.Append( FString::Printf( TEXT( "%s (*.%s)|*.%s" ), *Description, *Extension, *Extension ) );
		}
	}

	FString SupportedExtensions( FString::Printf( TEXT( "All Files (%s)|%s|%s" ), *AllExtensions, *AllExtensions, *FileTypes ) );

	TArray<FString> OpenedFiles;
	FString DefaultLocation( FEditorDirectories::Get().GetLastDirectory( ELastDirectory::GENERIC_IMPORT ) );
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	bool bOpened = false;
	if ( DesktopPlatform )
	{
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs( nullptr ),
			LOCTEXT( "FileDialogTitle", "Import Interchange" ).ToString(),
			DefaultLocation,
			TEXT( "" ),
			SupportedExtensions,
			EFileDialogFlags::None,
			OpenedFiles
		);
	}

	if ( bOpened && OpenedFiles.Num() > 0 )
	{
		const FString& OpenedFile = OpenedFiles[0];
		FEditorDirectories::Get().SetLastDirectory( ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(OpenedFile) );

		return FPaths::ConvertRelativePathToFull( OpenedFile );
	}

	return {};
}

FString FInterchangeFileProducerUtils::SelectDirectory()
{
	return {};
}

#undef LOCTEXT_NAMESPACE
