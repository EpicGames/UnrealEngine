// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepContentProducer.h"

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"

#include "InterchangeFileProducer.generated.h"

class IDetailLayoutBuilder;
class UInterchangePipelineBase;

UCLASS(Experimental, HideCategories = (InterchangeProducer_Internal))
class DATAPREPCORE_API UInterchangeFileProducer : public UDataprepContentProducer
{
	GENERATED_BODY()

	UInterchangeFileProducer();

public:
	// UObject interface
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	/** Update producer with the desired file */
	void SetFilePath( const FString& InFilePath );

	const FString& GetFilePath() const { return FilePath; }

	// Begin UDataprepContentProducer overrides
	virtual const FText& GetLabel() const override;
	virtual const FText& GetDescription() const override;
	virtual FString GetNamespace() const override;
	virtual bool Supersede(const UDataprepContentProducer* OtherProducer) const override;
	virtual bool CanAddToProducersArray(bool bIsAutomated) override;

protected:
	virtual bool Initialize() override;
	virtual bool Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets) override;
	virtual void Reset() override;
	virtual bool IsActive() override;
	// End UDataprepContentProducer overrides

	UPROPERTY( EditAnywhere, Category = InterchangeProducer )
	FString FilePath;

private:
	/** Does what is required after setting a new FilePath */
	void OnFilePathChanged();

	/** Update the name of the producer based on the filename */
	void UpdateName();

	bool InitTranslator();

private:
	TUniquePtr< FDataprepWorkReporter > ProgressTaskPtr;

	UPROPERTY( Transient, DuplicateTransient )
	TObjectPtr<UPackage> TransientPackage = nullptr;

	TArray< TWeakObjectPtr< UObject > > Assets;

	friend class SInterchangeFileProducerFileProperty;
};

class FInterchangeContentProducerDetails : public IDetailCustomization
{
public:
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	FSlateColor GetStatusColorAndOpacity() const;
	bool IsProducerSuperseded() const;

protected:
	class UDataprepAssetProducers* AssetProducers = nullptr;
	UDataprepContentProducer* Producer = nullptr;
	int32 ProducerIndex = INDEX_NONE; // This index does not change for the lifetime of the property widget
};

// Customization of the details of the Interchange producer for the data prep editor.
class DATAPREPCORE_API FInterchangeFileProducerDetails : public FInterchangeContentProducerDetails
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FInterchangeFileProducerDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};
