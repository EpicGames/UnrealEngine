// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/DataprepParameterization.h"

#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepBindingCommandChange.h"
#include "DataprepParameterizationArchive.h"

#include "DataprepCoreLogCategory.h"
#include "Engine/Engine.h"
#include "Misc/ITransaction.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"

namespace DataprepParameterization
{
	void PopulateValueTypeValidationData(FArrayProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FSetProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FMapProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FSoftClassProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FClassProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FObjectPropertyBase* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FEnumProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FStructProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);
	void PopulateValueTypeValidationData(FProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData);

	void PopulateValueTypeValidationData(FArrayProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			PopulateValueTypeValidationData( CurrentProperty->Inner, ValueTypeValidationData );
		}
	}

	void PopulateValueTypeValidationData(FSetProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			PopulateValueTypeValidationData( CurrentProperty->ElementProp, ValueTypeValidationData );
		}
	}

	void PopulateValueTypeValidationData(FMapProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 3 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			PopulateValueTypeValidationData( CurrentProperty->KeyProp, ValueTypeValidationData );
			PopulateValueTypeValidationData( CurrentProperty->ValueProp, ValueTypeValidationData );
		}
	}

	void PopulateValueTypeValidationData(FSoftClassProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			// Property class don't matter here since it's always UObject::StaticClass
			ValueTypeValidationData.Add( CurrentProperty->MetaClass );
		}
	}

	void PopulateValueTypeValidationData(FClassProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			ValueTypeValidationData.Add( CurrentProperty->PropertyClass );
			ValueTypeValidationData.Add( CurrentProperty->MetaClass );
		}
	}

	void PopulateValueTypeValidationData(FObjectPropertyBase* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			ValueTypeValidationData.Add( CurrentProperty->PropertyClass );
		}
	}

	void PopulateValueTypeValidationData(FEnumProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 3 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			ValueTypeValidationData.Add( CurrentProperty->GetEnum() );
			PopulateValueTypeValidationData( CurrentProperty->GetUnderlyingProperty(), ValueTypeValidationData );
		}
	}

	void PopulateValueTypeValidationData(FStructProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			ValueTypeValidationData.Reserve( 2 );
			ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			ValueTypeValidationData.Add( CurrentProperty->Struct );
		}
	}

	void PopulateValueTypeValidationData(FProperty* CurrentProperty, FValueTypeValidationData& ValueTypeValidationData)
	{
		if ( CurrentProperty )
		{
			int32 NumberOfObject = ValueTypeValidationData.Num();
			FFieldClass* CurrentClass = CurrentProperty->GetClass();
			while ( CurrentClass && NumberOfObject == ValueTypeValidationData.Num() )
			{
				if ( CurrentClass == FArrayProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FArrayProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FSetProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FSetProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FMapProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FMapProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FSoftClassProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FSoftClassProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FClassProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FClassProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FObjectPropertyBase::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FObjectPropertyBase*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FEnumProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FEnumProperty*>( CurrentProperty ), ValueTypeValidationData );
				}
				else if ( CurrentClass == FStructProperty::StaticClass() )
				{
					PopulateValueTypeValidationData( static_cast<FStructProperty*>( CurrentProperty ), ValueTypeValidationData );
				}

				CurrentClass = CurrentClass->GetSuperClass();
			}

			if ( NumberOfObject == ValueTypeValidationData.Num() )
			{
				ValueTypeValidationData.Add( CurrentProperty->GetClass() );
			}
		}
	}


	void* GetAddressOf(const FArrayProperty& Property, void* BaseAddress, int32 ContainerIndex)
	{
		void* AddressOfArray = Property.ContainerPtrToValuePtr<void*>( BaseAddress, 0 );
		if ( ContainerIndex == INDEX_NONE )
		{
			// The index none is used when we want to get the container itself
			return AddressOfArray;
		}

		FScriptArrayHelper ArrayHelper( &Property, AddressOfArray );
		if ( ArrayHelper.IsValidIndex(ContainerIndex) )
		{
			return ArrayHelper.GetRawPtr( ContainerIndex );
		}

		return nullptr;
	}

	void* GetAddressOf(const FSetProperty& Property, void* BaseAddress, int32 ContainerIndex)
	{
		void* AddressOfSet = Property.ContainerPtrToValuePtr<void*>( BaseAddress, 0 );
		if ( ContainerIndex == INDEX_NONE )
		{
			// The index none is used when we want to get the container itself
			return AddressOfSet;
		}

		FScriptSetHelper SetHelper(&Property, Property.ContainerPtrToValuePtr<void*>( BaseAddress, 0 ) );
		int32 RealIndex = SetHelper.FindInternalIndex( ContainerIndex );
		if ( SetHelper.IsValidIndex( RealIndex ) )
		{
			return SetHelper.GetElementPtr( RealIndex );
		}

		return nullptr;
	}

	void* GetAddressOf(const FMapProperty& Property, void* BaseAddress, int32 ContainerIndex)
	{
		void* AddressOfMap = Property.ContainerPtrToValuePtr<void*>( BaseAddress, 0 );
		if ( ContainerIndex == INDEX_NONE )
		{
			// The index none is used when we want to get the container itself
			return AddressOfMap;
		}

		FScriptMapHelper MapHelper(&Property, Property.ContainerPtrToValuePtr<void*>( BaseAddress, 0 ) );
		int32 RealIndex = MapHelper.FindInternalIndex( ContainerIndex );
		if ( MapHelper.IsValidIndex( RealIndex ) )
		{
			return MapHelper.GetPairPtr( RealIndex );
		}

		return nullptr;
	}

	void* GetAddressOf(const FProperty& Property, void* BaseAddress, int32 ContainerIndex)
	{
		FFieldClass* Class = Property.GetClass();
		if ( Class == FArrayProperty::StaticClass() )
		{
			return GetAddressOf( static_cast<const FArrayProperty&>( Property ), BaseAddress, ContainerIndex );
		}
		else if ( Class == FSetProperty::StaticClass() )
		{
			return GetAddressOf( static_cast<const FSetProperty&>( Property ), BaseAddress, ContainerIndex );
		}
		else if ( Class == FMapProperty::StaticClass() )
		{
			return GetAddressOf( static_cast<const FMapProperty&>( Property ), BaseAddress, ContainerIndex );
		}

		if ( Property.ArrayDim > ContainerIndex )
		{
			return Property.ContainerPtrToValuePtr<void*>( BaseAddress, FMath::Max( ContainerIndex, 0 ) );
		}

		return nullptr;
	}

	/**
	 * Get the outer on which we should look when searching for a child property
	 * @return Return a nullptr if the current property is not supported
	 */
	FFieldVariant GetOuterForPropertyFinding(FProperty* Property)
	{
		if ( !Property )
		{
			return nullptr;
		}

		FFieldClass* PropertyClass = Property->GetClass();
		while ( PropertyClass )
		{
			if ( PropertyClass == FStructProperty::StaticClass() )
			{
				FStructProperty* StructProperty = static_cast<FStructProperty*>(Property);
				UScriptStruct* ScriptStruct = StructProperty->Struct;
				/**
				 * We don't want to support struct that exist for the reinstancing
				 * That heuristic might not be good for hot reloaded c++ struct
+				 */
				if (ScriptStruct
					&& ScriptStruct->GetName().StartsWith( TEXT("STRUCT_REINST_") ) )
				{
					return nullptr;
				}
				return ScriptStruct;
			}
			else if (PropertyClass == FObjectPropertyBase::StaticClass())
			{
				FObjectPropertyBase* ObjectProperty = static_cast<FObjectPropertyBase*>(Property);
				UClass* Class = ObjectProperty->PropertyClass;
				// We reject properties that points to a class where there is newer version that exists
				if ( Class
					&& bool(Class->ClassFlags & (CLASS_NewerVersionExists | CLASS_Transient) ) )
				{
					return nullptr;
				}
				return Class;
			}

			PropertyClass = PropertyClass->GetSuperClass();
		}

		return Property;
	};


	/**
	 * Walk a binding using only the cached properties on the object
	 * We use a outer and class/struct heuristic to validate that properties are still valid
	 */
	int32 GetDeepestLevelOfValidCache(const FDataprepParameterizationBinding& Binding, void*& OutPropertyValueAddress)
	{
		FFieldVariant CurrentOuter;

		if ( UObject* ObjectBinded = Binding.ObjectBinded )
		{
			// We use the current outer has a heuristic to validate that the property is still valid
			CurrentOuter = ObjectBinded->GetClass();
			void* CurrentPropertyValueAddresss = ObjectBinded;
			void* LastValidValueAddress = ObjectBinded;

			const TArray<FDataprepPropertyLink>& PropertyChain = Binding.PropertyChain;

			for ( int32 Level = 0; Level < PropertyChain.Num();  Level++ )
			{
				bool bAbortCacheValidation = true;

				FProperty* CurentProperty = PropertyChain[Level].CachedProperty.Get();
				if ( CurentProperty )
				{
					CurrentPropertyValueAddresss = GetAddressOf( *CurentProperty, CurrentPropertyValueAddresss, PropertyChain[Level].ContainerIndex );

					// We look if the outer of the property is the right one as a heuristic for the validity of the current property
					bool bPropertySeamsValid = false;
					if ( FFieldVariant CurrentPropertyOuter = CurentProperty->GetOwnerVariant() )
					{
						if ( CurrentPropertyOuter == CurrentOuter )
						{
							bPropertySeamsValid = true;
						}
						else if ( UStruct* CurrentOuterAsStruct = CurrentOuter.Get<UStruct>() )
						{
							// Walk the struct hierarchy in case we have a sub struct of the expected struct
							FFieldVariant PropertyOuter = CurentProperty->GetOwnerVariant();
							CurrentOuterAsStruct = CurrentOuterAsStruct->GetSuperStruct();
							while ( !bPropertySeamsValid && CurrentOuterAsStruct )
							{
								bPropertySeamsValid = (CurrentOuterAsStruct == PropertyOuter.ToUObject());
								CurrentOuterAsStruct = CurrentOuterAsStruct->GetSuperStruct();
							}
						}
					}

					// The get outer does also the validation for the non supported properties
					CurrentOuter = GetOuterForPropertyFinding( CurentProperty );

					bAbortCacheValidation = !( CurrentPropertyValueAddresss && bPropertySeamsValid && CurrentOuter );
				}
				else
				{
					CurrentPropertyValueAddresss = nullptr;
				}

				if ( bAbortCacheValidation )
				{
					Level--;
					OutPropertyValueAddress = LastValidValueAddress;
					return Level;
				}

				LastValidValueAddress = CurrentPropertyValueAddresss;
			}

			// The cached properties were all valid
			OutPropertyValueAddress = LastValidValueAddress;
			return  PropertyChain.Num() - 1;

		}

		OutPropertyValueAddress = nullptr;
		return INDEX_NONE;
	}


	/**
	 * Try to get the property from a binding
	 * @param OutPropertyValueAddress The address of where we can find the value of the property
	 * @return Return nullptr if the binding is invalid
	 */
	FProperty* GetPropertyFromBinding(FDataprepParameterizationBinding& Binding, void*& OutPropertyValueAddress)
	{
		if ( !Binding.ObjectBinded )
		{
			return nullptr;
		}

		// Get the last valid level from the cache
		int32 LevelIndex = GetDeepestLevelOfValidCache( Binding, OutPropertyValueAddress );


		FFieldVariant CurrentOuter = Binding.ObjectBinded->GetClass();
		if ( LevelIndex != INDEX_NONE )
		{
			CurrentOuter = GetOuterForPropertyFinding( Binding.PropertyChain[LevelIndex].CachedProperty.Get() );
		}

		// We start updating the binding from the first invalid level this is always the one after
		LevelIndex++;

		// Todo (what happen if the bottom property changed its type)

		// Find missing or new properties and update the cache of the property link
		FProperty* PropertyAtCurrentLevel = nullptr;
		TArray<FDataprepPropertyLink>& PropertyChain = Binding.PropertyChain;
		while ( LevelIndex < PropertyChain.Num() && CurrentOuter )
		{
			FDataprepPropertyLink& PropertyLink = Binding.PropertyChain[LevelIndex];
			PropertyAtCurrentLevel = FindFProperty<FProperty>(CurrentOuter.Get<UStruct>(), PropertyLink.PropertyName);
			if (!PropertyAtCurrentLevel)
			{
				if ( UStruct* OuterAsStruct = CurrentOuter.Get<UStruct>() )
				{
					OuterAsStruct = OuterAsStruct->GetSuperStruct();
					while ( !PropertyAtCurrentLevel && OuterAsStruct )
					{
						PropertyAtCurrentLevel = FindFProperty<FProperty>(OuterAsStruct, PropertyLink.PropertyName);
						OuterAsStruct = OuterAsStruct->GetSuperStruct();
					}
				}
			}

			PropertyLink.CachedProperty = PropertyAtCurrentLevel;
			CurrentOuter = GetOuterForPropertyFinding( PropertyAtCurrentLevel);
			LevelIndex++;
		}

		// If the current outer is null it's because the last property we checked is not supported
		if ( LevelIndex == PropertyChain.Num() && CurrentOuter )
		{
			PropertyAtCurrentLevel = PropertyChain.Last().CachedProperty.Get();
			FValueTypeValidationData ValueTypeValidationData;
			PopulateValueTypeValidationData( PropertyAtCurrentLevel, ValueTypeValidationData );

			// 4.25 development hotfix
			if ( Binding.ValueTypeValidationData.Num() == 0 )
			{
				Binding.ValueTypeValidationData = ValueTypeValidationData;
			}

			// Perf Note: We might be able to cache this validation and some part of this function at some point
			if ( ValueTypeValidationData == Binding.ValueTypeValidationData )
			{
				return PropertyAtCurrentLevel;
			}
			else
			{
				UE_LOGF( LogDataprepCore, Warning, "A binding was invalid because it's type changed" );
				OutPropertyValueAddress = nullptr;
			}
		}

		return nullptr;
	}

	/**
	 * Try to get the property from a binding
	 * Return nullptr if the binding is invalid
	 */
	FProperty* GetPropertyFromBinding(FDataprepParameterizationBinding& Binding)
	{
		void* DummyPointer;
		return GetPropertyFromBinding( Binding, DummyPointer );
	}

	void CopyValue(FProperty& DestinationProperty, void* DestinationAddress, FProperty& SourceProperty, void* SourceAddress)
	{
		FFieldClass* PropertyClass = DestinationProperty.GetClass();
		// We only support copying value of properties when they are from the same class (this is not a warranty that this is safe, it's only a validation heuristic)
		check(PropertyClass == SourceProperty.GetClass() );

		// Bool properties are special because each property can have their own mask and offset from there base address (probably to support bitfields)
		if (PropertyClass == FBoolProperty::StaticClass())
		{
			const bool bSourceValue = static_cast<FBoolProperty&>( SourceProperty ).GetPropertyValue( SourceAddress );
			static_cast<FBoolProperty&>( DestinationProperty ).SetPropertyValue( DestinationAddress, bSourceValue );
		}
		else if ( DestinationProperty.ArrayDim != SourceProperty.ArrayDim )
		{
			FProperty& SmallerProperty = DestinationProperty.ArrayDim > SourceProperty.ArrayDim ? SourceProperty : DestinationProperty;
			SmallerProperty.CopySingleValue( DestinationAddress, SourceAddress );
		}
		else
		{
			DestinationProperty.CopyCompleteValue( DestinationAddress, SourceAddress );
		}
	}
};

bool FValueTypeValidationData::Serialize(FArchive& Ar)
{
	Ar << ObjectData;
	Ar << PropertyData;
	Ar << PropertyTypeData;
	return true;
}

void FValueTypeValidationData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ObjectData);
	for (TFieldPath<FProperty>& Prop : PropertyData)
	{
		Prop->AddReferencedObjects(Collector);
	}
}

FDataprepParameterizationBinding::FDataprepParameterizationBinding(UDataprepParameterizableObject* InObjectBinded, TArray<FDataprepPropertyLink> InPropertyChain)
	: ObjectBinded( InObjectBinded )
	, PropertyChain( MoveTemp( InPropertyChain ) )
	, ValueTypeValidationData()
{
	if ( PropertyChain.Num() > 0 )
	{
		if ( FProperty* Property = PropertyChain.Last().CachedProperty.Get() )
		{
			DataprepParameterization::PopulateValueTypeValidationData( Property, ValueTypeValidationData );
		}
	}
}

bool FDataprepParameterizationBinding::operator==(const FDataprepParameterizationBinding& Other) const
{
	// The value type validation data shouldn't matter when comparing binding
	return ObjectBinded == Other.ObjectBinded && PropertyChain == Other.PropertyChain;
}

uint32 GetTypeHash(const FDataprepParameterizationBinding& Binding)
{
	// The value type validation data shouldn't matter for the hash of a binding
	return HashCombine(  GetTypeHash( Binding.ObjectBinded ), GetTypeHash( Binding.PropertyChain ) );

}

uint32 GetTypeHash(const TArray<FDataprepPropertyLink>& PropertyLinks)
{
	uint32 Hash = GetTypeHash( PropertyLinks.Num() );
	for ( const FDataprepPropertyLink& PropertyLink : PropertyLinks )
	{
		Hash = HashCombine( Hash, GetTypeHash( PropertyLink ) );
	}

	return Hash;
}

bool UDataprepParameterizationBindings::ContainsBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const
{
	return BindingToParameterName.Contains( Binding );
}

FName UDataprepParameterizationBindings::GetParameterNameForBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const
{
	return BindingToParameterName.FindRef( Binding );
}

const UDataprepParameterizationBindings::FSetOfBinding* UDataprepParameterizationBindings::GetBindingsFromObject(const UDataprepParameterizableObject* Object) const
{
	return ObjectToBindings.Find( Object );
}

const UDataprepParameterizationBindings::FSetOfBinding* UDataprepParameterizationBindings::GetBindingsFromParameter(const FName& ParameterName) const
{
	return NameToBindings.Find( ParameterName );
}

bool UDataprepParameterizationBindings::HasBindingsForParameter(const FName& ParameterName) const
{
	return NameToBindings.Contains( ParameterName );
}

bool UDataprepParameterizationBindings::HasBindingsFromObject(UDataprepParameterizableObject* Object) const
{
	return ObjectToBindings.Contains( Object );
}

void UDataprepParameterizationBindings::Add(const TSharedRef<FDataprepParameterizationBinding>& Binding, const FName& ParamerterName, FSetOfBinding& OutBindingsContainedByNewBinding)
{
	Modify();

	uint32 BindingHash = GetTypeHash( Binding.Get() );

	if ( FName* ExistingParameterName = BindingToParameterName.FindByHash( BindingHash, Binding ) )
	{
		if ( *ExistingParameterName != ParamerterName )
		{
			// Remove the trace of the old mapping
			NameToBindings.FindRef( *ExistingParameterName ).RemoveByHash( BindingHash, Binding );
		}
	}

	BindingToParameterName.AddByHash( BindingHash, Binding, ParamerterName );
	NameToBindings.FindOrAdd( ParamerterName ).AddByHash( BindingHash, Binding );

	FSetOfBinding& BindingsFromSameObject = ObjectToBindings.FindOrAdd( Binding->ObjectBinded );

	for ( TSharedRef<FDataprepParameterizationBinding>& PossibleSubBinding : BindingsFromSameObject )
	{
		if ( PossibleSubBinding->PropertyChain.Num() >= Binding->PropertyChain.Num() )
		{
			TArray<FDataprepPropertyLink> PropertyChain = PossibleSubBinding->PropertyChain;
			while ( PropertyChain.Num() > Binding->PropertyChain.Num() )
			{
				PropertyChain.RemoveAt( PropertyChain.Num() - 1, EAllowShrinking::No );
			}

			if ( PropertyChain == Binding->PropertyChain )
			{
				OutBindingsContainedByNewBinding.Add( PossibleSubBinding );
			}
			else if ( PropertyChain.Num() > 0 )
			{
				PropertyChain.Last().ContainerIndex = INDEX_NONE;
				if ( PropertyChain == Binding->PropertyChain )
				{
					OutBindingsContainedByNewBinding.Add( PossibleSubBinding );
				}
			}
		}

	}

	BindingsFromSameObject.AddByHash( BindingHash, Binding );


}

FName UDataprepParameterizationBindings::RemoveBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding)
{
	uint32 BindingHash = GetTypeHash( Binding.Get() );
	if ( FName* ParameterName = BindingToParameterName.FindByHash( BindingHash, Binding ) )
	{
		Modify();

		// 1) Remove from the map binding to parameter
		BindingToParameterName.RemoveByHash( BindingHash, Binding );

		// 2) Remove from the map parameter to bindings
		{
			if ( FSetOfBinding* BindingsMappedToParameter = NameToBindings.Find( *ParameterName ) )
			{
				if ( BindingsMappedToParameter->Num() <= 1 )
				{
					NameToBindings.Remove( *ParameterName );
				}
				else
				{
					BindingsMappedToParameter->RemoveByHash( BindingHash, Binding );
				}
			}
		}

		// 3) Remove from the map object to bindings
		{
			UDataprepParameterizableObject* Object = Binding->ObjectBinded;
			uint32 ObjectHash = GetTypeHash( Object );
			if ( FSetOfBinding* BindingsMappedToObject = ObjectToBindings.FindByHash( ObjectHash, Object ) )
			{
				if ( BindingsMappedToObject->Num() <= 1 )
				{
					ObjectToBindings.RemoveByHash( ObjectHash, Object );
				}
				else
				{
					BindingsMappedToObject->RemoveByHash( BindingHash, Binding );
				}
			}
		}

		return *ParameterName;
	}

	return NAME_None;
}

TSet<FName> UDataprepParameterizationBindings::RemoveAllBindingsFromObject(UDataprepParameterizableObject* Object)
{
	if ( FSetOfBinding* Bindings = ObjectToBindings.Find( Object ) )
	{
		Modify();

		// Remove the bindings
		TSet<FName> ParameterNames;
		TArray<TSharedRef<FDataprepParameterizationBinding>> BindingsToRemove( Bindings->Array() );
		for ( TSharedRef<FDataprepParameterizationBinding>& Binding : BindingsToRemove )
		{
			ParameterNames.Add( RemoveBinding( Binding ) );
		}

		return ParameterNames;
	}

	return {};
}

TSharedPtr<FDataprepParameterizationBinding> UDataprepParameterizationBindings::GetContainingBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const
{
	if ( Binding->ObjectBinded )
	{
		 TSharedRef<FDataprepParameterizationBinding> PossibleContainingBinding = MakeShared<FDataprepParameterizationBinding>( Binding.Get() );

		while ( PossibleContainingBinding->PropertyChain.Num() > 0 )
		{
			if ( BindingToParameterName.Contains( PossibleContainingBinding ) )
			{
				PossibleContainingBinding->ValueTypeValidationData.Empty();
				DataprepParameterization::PopulateValueTypeValidationData( PossibleContainingBinding->PropertyChain.Last().CachedProperty.Get(), PossibleContainingBinding->ValueTypeValidationData );
				return PossibleContainingBinding;
			}

			int32& ContainerIndex = PossibleContainingBinding->PropertyChain.Last().ContainerIndex;
			if ( ContainerIndex != INDEX_NONE )
			{
				ContainerIndex = INDEX_NONE;
			}
			else
			{
				PossibleContainingBinding->PropertyChain.Pop( EAllowShrinking::No );
			}
		}
	}

	return {};
}

const UDataprepParameterizationBindings::FBindingToParameterNameMap& UDataprepParameterizationBindings::GetBindingToParameterName() const
{
	return BindingToParameterName;
}

TArray<UDataprepParameterizableObject*> UDataprepParameterizationBindings::GetParameterizedObjects() const
{
	TArray<UDataprepParameterizableObject*> Objects;
	ObjectPtrDecay(ObjectToBindings).GenerateKeyArray( Objects );
	return Objects;
}

void UDataprepParameterizationBindings::Serialize(FArchive& Ar)
{
	Super::Serialize( Ar );

	if ( Ar.IsSaving() )
	{
		Save( Ar );
	}
	else if ( Ar.IsLoading() )
	{
		Load( Ar );
	}
}

void UDataprepParameterizationBindings::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataprepParameterizationBindings* Bindings = CastChecked<UDataprepParameterizationBindings>( InThis );

	for ( auto& Pair : Bindings->ObjectToBindings )
	{
		Collector.AddReferencedObject( Pair.Key );

		FSetOfBinding& BindingSet = Pair.Value;
		for ( TSharedRef<FDataprepParameterizationBinding>& Binding : BindingSet )
		{
			Collector.AddReferencedObject( Binding->ObjectBinded );
			Binding->ValueTypeValidationData.AddReferencedObjects(Collector);
		}
	}

	Super::AddReferencedObjects( InThis, Collector );
}

void UDataprepParameterizationBindings::Save(FArchive& Ar)
{
	// 1) Save the data for the name to bindings
	{
		int32 NumberOfParameter = NameToBindings.Num();
		Ar << NumberOfParameter;
		for ( const TPair<FName, FSetOfBinding>& PairNameAndBindings : NameToBindings )
		{
			FName ParameterName = PairNameAndBindings.Key;
			Ar << ParameterName;
			int32 NumberOfBinding = PairNameAndBindings.Value.Num();
			Ar << NumberOfBinding;
		}
	}

	// 2) Save the data for the object to bindings
	{
		int32 NumberOfObjectBinded = ObjectToBindings.Num();
		Ar << NumberOfObjectBinded;
		for ( const auto& PairObjectAndBindings : ObjectToBindings )
		{
			UDataprepParameterizableObject* Object = PairObjectAndBindings.Key;
			Ar << Object;
			int32 NumberOfBinding = PairObjectAndBindings.Value.Num();
			Ar << NumberOfBinding;
		}
	}

	// 3) Do the actual save of the bindings
	{
		int32 NumberOfBinding = BindingToParameterName.Num();
		Ar << NumberOfBinding;
		UScriptStruct* StaticStruct = FDataprepParameterizationBinding::StaticStruct();

		for (const TPair<TSharedRef<FDataprepParameterizationBinding>, FName>& PairBindingToParameterName : BindingToParameterName)
		{
			FDataprepParameterizationBinding& Binding = PairBindingToParameterName.Key.Get();
			StaticStruct->SerializeItem( Ar, &Binding, nullptr );

			FName ParameterName =  PairBindingToParameterName.Value;
			Ar << ParameterName;
		}
	}
}

void UDataprepParameterizationBindings::Load(FArchive& Ar)
{
	// 1) Load the data for the name to bindings (prepare the structure for the insertion of data later)
	{
		int32 NumOfUniqueParameterName;
		Ar << NumOfUniqueParameterName;
		NameToBindings.Empty( NumOfUniqueParameterName );

		for ( int32 Index = 0; Index < NumOfUniqueParameterName; Index++ )
		{
			FName ParameterName;
			Ar << ParameterName;
			FSetOfBinding& Bindings = NameToBindings.FindOrAdd( ParameterName );

			int32 NumOfBindingForParameter;
			Ar << NumOfBindingForParameter;
			Bindings.Reserve( NumOfBindingForParameter );
		}
	}

	// 2) Load the data for the object to bindings (prepare the structure for the insertion of data later)
	{
		int32 NumOfObjectBinded;
		Ar << NumOfObjectBinded;
		ObjectToBindings.Empty( NumOfObjectBinded );

		for ( int32 Index = 0; Index < NumOfObjectBinded; Index++ )
		{
			UDataprepParameterizableObject* Object = nullptr;
			Ar << Object;

			FSetOfBinding& Bindings = ObjectToBindings.FindOrAdd( Object );

			int32 NumOfBindingForObject;
			Ar << NumOfBindingForObject;
			Bindings.Reserve( NumOfBindingForObject );
		}
	}

	// 3) Do the actual loading of the binding
	{
		int32 NumOfBindings;
		Ar << NumOfBindings;
		BindingToParameterName.Empty( NumOfBindings );

		UScriptStruct* StaticStruct = FDataprepParameterizationBinding::StaticStruct();

		for (int32 Index = 0; Index < NumOfBindings; Index++)
		{
			TSharedRef<FDataprepParameterizationBinding> BindingPtr = MakeShared<FDataprepParameterizationBinding>();
			FDataprepParameterizationBinding& Binding = BindingPtr.Get();
			StaticStruct->SerializeItem( Ar, &Binding, nullptr );

			FName ParameterizationName;
			Ar << ParameterizationName;

			if ( Binding.ObjectBinded )
			{
				uint32 BindingHash = GetTypeHash( Binding );

				BindingToParameterName.AddByHash( BindingHash, BindingPtr, ParameterizationName );

				NameToBindings.FindOrAdd( ParameterizationName ).AddByHash( BindingHash, BindingPtr );

				ObjectToBindings.FindOrAdd( BindingPtr->ObjectBinded ).AddByHash( BindingHash, MoveTemp( BindingPtr ) );
			}
		}
	}
}


UDataprepParameterization::UDataprepParameterization()
{
	OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddUObject( this, &UDataprepParameterization::OnObjectModified );
}

void UDataprepParameterization::PostInitProperties()
{
	Super::PostInitProperties();

	if ( !HasAnyFlags( RF_ClassDefaultObject | RF_NeedLoad ) )
	{
		SetFlags( RF_Public );

		if ( !BindingsContainer )
		{
			BindingsContainer = NewObject<UDataprepParameterizationBindings>( this, NAME_None, RF_Transactional );
		}

		GenerateClass();
	}
}

void UDataprepParameterization::PostLoad()
{
	if ( !HasAnyFlags( RF_ClassDefaultObject | RF_NeedLoad ) )
	{
		SetFlags( RF_Public );

		if ( !BindingsContainer )
		{
			BindingsContainer = NewObject<UDataprepParameterizationBindings>(this, NAME_None, RF_Transactional);
		}

		// ueent_hotfix revisit this code has renaming a object while a linker is active is dangerous (this was put here so that the duplicate object would work properly)
		PrepareCustomClassForNewClassGeneration();
		UClass* OldClass = CustomContainerClass;
		CustomContainerClass = nullptr;

		LoadParameterization();

		if ( OldClass )
		{
			DoReinstancing( OldClass, false );
			OnTellInstancesToReloadTheirSerializedData.Broadcast();
			CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( nullptr );
		}

		TArray<UDataprepParameterizableObject*> ParameterizedObjects = BindingsContainer->GetParameterizedObjects();
		for ( UDataprepParameterizableObject* Object : ParameterizedObjects )
		{
			if ( Object )
			{
				AddBindingToPostEditOfParameterizableObject( *Object, false );
			}
		}
	}
	Super::PostLoad();
}

void UDataprepParameterization::Serialize(FArchive& Ar)
{
	if ( Ar.IsSaving() && !HasAnyFlags( RF_ClassDefaultObject ) )
	{
		// Todo track when the object is changed to avoid rewriting to array each time
		ParameterizationStorage.Empty(CustomContainerClass->GetMinAlignment());
		FDataprepParameterizationWriter Writer(DefaultParameterisation, ParameterizationStorage);
	}

	Super::Serialize( Ar );
}

void UDataprepParameterization::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	if ( TransactionEvent.GetChangedProperties().Contains( GET_MEMBER_NAME_CHECKED( UDataprepParameterization, ParameterizationStorage ) ) )
	{
		// This implementation work on the assumption that all the objects in the transaction were serialized before the calls to post edit undo
		PrepareCustomClassForNewClassGeneration();
		UClass* OldClass = CustomContainerClass;
		CustomContainerClass = nullptr;

		LoadParameterization();

		DoReinstancing( OldClass, false );

		OnTellInstancesToReloadTheirSerializedData.Broadcast();

		CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( nullptr );
	}

	if ( TransactionEvent.GetChangedProperties().Contains( GET_MEMBER_NAME_CHECKED( UDataprepParameterization, BindingsContainer ) ) )
	{

	}
}

void UDataprepParameterization::FinishDestroy()
{
	FCoreUObjectDelegates::OnObjectModified.Remove( OnObjectModifiedHandle );

	if ( BindingsContainer )
	{
		TArray<UDataprepParameterizableObject*> ParameterizedObjects = BindingsContainer->GetParameterizedObjects();
		for ( UDataprepParameterizableObject* Object : ParameterizedObjects )
		{
			if ( Object )
			{
				RemoveBindingToPostEditOfParameterizableObject( *Object, false );
			}
		}
	}

	Super::FinishDestroy();
}

void UDataprepParameterization::OnObjectModified(UObject* Object)
{
	if ( Object == DefaultParameterisation )
	{
		Modify();
	}
}

UObject* UDataprepParameterization::GetDefaultObject()
{
	return DefaultParameterisation;
}

bool UDataprepParameterization::BindObjectProperty(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain, const FName& Name)
{
	if ( Object && FDataprepParameterizationUtils::IsPropertyChainValid( PropertyChain ) && !Name.IsNone() )
	{
		Modify();

		TSharedRef<FDataprepParameterizationBinding> Binding = MakeShared<FDataprepParameterizationBinding>( Object, PropertyChain );
		void* AddressOfTheValueFromBinding;

		FName OldParameterName = BindingsContainer->GetParameterNameForBinding( Binding );
		if ( OldParameterName == Name )
		{
			return !Name.IsNone();
		}

		bool bClassNeedUpdate = false;
		if ( OldParameterName != Name )
		{
			RemoveBinding( Binding, bClassNeedUpdate );
		}

		bool bBindingWasAdded = false;
		bool bAddingFullProperty = PropertyChain.Last().ContainerIndex == INDEX_NONE;
		bool bIsFirstBindingFromObject = false;

		// We expect the chain to have a valid chain of cached property before inserting the binding
		if ( DataprepParameterization::GetDeepestLevelOfValidCache( Binding.Get(), AddressOfTheValueFromBinding) == PropertyChain.Num() - 1 )
		{
			bIsFirstBindingFromObject = !BindingsContainer->HasBindingsFromObject( Object );
			UDataprepParameterizationBindings::FSetOfBinding BindingsToRemove;

			if ( FProperty** PropertyPtr = NameToParameterizationProperty.Find( Name ) )
			{
				FProperty* PropertyFromParameterization = *PropertyPtr;
				FProperty* PropertyFromBinding = PropertyChain.Last().CachedProperty.Get();
				// Ensure that the properties are compatible
				if ( !bAddingFullProperty || PropertyFromParameterization->ArrayDim == PropertyFromBinding->ArrayDim )
				{
					FValueTypeValidationData ValueTypeValidationData;
					DataprepParameterization::PopulateValueTypeValidationData( PropertyFromParameterization, ValueTypeValidationData );
					if ( ValueTypeValidationData == Binding->ValueTypeValidationData )
					{
						BindingsContainer->Add( Binding, Name, BindingsToRemove );
						bBindingWasAdded = true;
					}
				}
			}
			else
			{
				BindingsContainer->Add( Binding, Name, BindingsToRemove );

				FProperty* PropertyFromBinding = PropertyChain.Last().CachedProperty.Get();

				// The validation we did with GetDeepestLevelOfValidCache ensure us that the property ptr is valid
				FProperty* NewProperty = AddPropertyToClass( Name, *PropertyFromBinding, bAddingFullProperty );

				bClassNeedUpdate = true;
				bBindingWasAdded = true;
			}

			for ( TSharedRef<FDataprepParameterizationBinding>& BindingToRemove : BindingsToRemove )
			{
				RemoveBinding( BindingToRemove, bClassNeedUpdate );
			}
		}

		if ( bClassNeedUpdate )
		{
			UpdateClass();
		}

		if ( bBindingWasAdded )
		{
			UpdateParameterizationFromBinding( Binding );
			
			if ( bIsFirstBindingFromObject )
			{
				AddBindingToPostEditOfParameterizableObject( *Object, true );
			}

			if ( bIsFirstBindingFromObject )
			{
				AddBindingToPostEditOfParameterizableObject( *Object, true );
			}

			TSet<UObject*> Objects;
			Objects.Add( Object );
			CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &Objects );
		}
	}

	return false;
}

bool UDataprepParameterization::IsObjectPropertyBinded(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain) const
{
	TSharedRef<FDataprepParameterizationBinding> Binding= MakeShared<FDataprepParameterizationBinding>( Object, PropertyChain );
	return BindingsContainer->ContainsBinding( Binding );
}

FName UDataprepParameterization::GetNameOfParameterForObjectProperty(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& PropertyChain) const
{
	TSharedRef<FDataprepParameterizationBinding> Binding = MakeShared<FDataprepParameterizationBinding>(Object, PropertyChain);
	return BindingsContainer->GetParameterNameForBinding(Binding);
}

void UDataprepParameterization::RemoveBindedObjectProperty(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain)
{
	TSharedRef<FDataprepParameterizationBinding> Binding = MakeShared<FDataprepParameterizationBinding>( Object, PropertyChain );

	Modify();

	bool bClassWasModified;
	if ( RemoveBinding( Binding, bClassWasModified ) )
	{
		if ( bClassWasModified )
		{
			UpdateClass();
		}

		TSet<UObject*> Objects;
		Objects.Add( Object );
		CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &Objects );
	}
}

void UDataprepParameterization::RemoveBindingFromObjects(const TArrayView<UDataprepParameterizableObject*>& Objects)
{
	Modify();
	TSet<FName> ParameterPotentiallyRemoved;
	TSet<UObject*> UniqueObjects;
	UniqueObjects.Reserve( Objects.Num() );

	for ( UDataprepParameterizableObject* Object : Objects )
	{
		if ( Object )
		{ 
			ParameterPotentiallyRemoved.Append( BindingsContainer->RemoveAllBindingsFromObject( Object ) );
			UniqueObjects.Add( Object );
			RemoveBindingToPostEditOfParameterizableObject( *Object, true );
		}
	}

	bool bClassWasChanged = false;
	for ( const FName& Name : ParameterPotentiallyRemoved )
	{
		if ( !BindingsContainer->HasBindingsForParameter( Name ) )
		{
			NameToParameterizationProperty.Remove( Name );
			bClassWasChanged = true;
		}
	}

	if ( bClassWasChanged )
	{
		UpdateClass();
	}

	CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &UniqueObjects );
}

void UDataprepParameterization::UpdateParameterizationFromBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding)
{
	FName ParameterModified = BindingsContainer->GetParameterNameForBinding( Binding );
	FProperty* ParameterizationProperty = NameToParameterizationProperty.FindRef( ParameterModified );
	if ( ParameterizationProperty )
	{
		void* AddressOfObjectValue = nullptr;
		if ( FProperty * ObjectProperty = DataprepParameterization::GetPropertyFromBinding( Binding.Get(), AddressOfObjectValue ) )
		{
			Modify();
			void* AddressOfParameterizationValue = DataprepParameterization::GetAddressOf( *ParameterizationProperty, DefaultParameterisation, INDEX_NONE );
			DataprepParameterization::CopyValue( *ParameterizationProperty, AddressOfParameterizationValue, *ObjectProperty, AddressOfObjectValue );

			// Post edit the default parameterization
			FEditPropertyChain EditChain;
			EditChain.AddHead( ParameterizationProperty );
			EditChain.SetActivePropertyNode( ParameterizationProperty );
			FPropertyChangedEvent EditPropertyChangeEvent( ParameterizationProperty, EPropertyChangeType::ValueSet );
			FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
			DefaultParameterisation->PostEditChangeChainProperty( EditChangeChainEvent );

			TSet<UObject*> Objects;
			Objects.Add( DefaultParameterisation );
			CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &Objects );
		}
	}
}

void UDataprepParameterization::GetExistingParameterNamesForType(FProperty* Property, bool bIsDescribingFullProperty, TSet<FString>& OutValidExistingNames, TSet<FString>& OutInvalidNames) const
{
	OutValidExistingNames.Empty( NameToParameterizationProperty.Num() );
	OutInvalidNames.Empty( NameToParameterizationProperty.Num() );

	FValueTypeValidationData ValidationData;
	DataprepParameterization::PopulateValueTypeValidationData( Property, ValidationData );

	for ( const TPair<FName, FProperty*>& Pair : NameToParameterizationProperty )
	{
		if ( FProperty* ParameterizationProperty = Pair.Value )
		{
			bool bWasAdded = false;
			if ( Property && ParameterizationProperty->GetClass() == Property->GetClass() && ( !bIsDescribingFullProperty || ParameterizationProperty->ArrayDim == Property->ArrayDim ) )
			{
				FValueTypeValidationData ValidationDataForParameterizationProperty;
				DataprepParameterization::PopulateValueTypeValidationData( ParameterizationProperty, ValidationDataForParameterizationProperty );
				if ( ValidationDataForParameterizationProperty == ValidationData )
				{
					bWasAdded = true;
					OutValidExistingNames.Add( Pair.Key.ToString() );
				}
			}

			if ( !bWasAdded )
			{
				OutInvalidNames.Add( Pair.Key.ToString() );
			}
		}
	}
}

void UDataprepParameterization::DuplicateObjectParamaterization(const UDataprepParameterizableObject* InObject, UDataprepParameterizableObject* OutObject)
{
	if (const UDataprepParameterizationBindings::FSetOfBinding* BindingsSet = BindingsContainer->GetBindingsFromObject(InObject))
	{
		TArray<TSharedRef<FDataprepParameterizationBinding>> OriginalBindings = BindingsSet->Array();

		for (TSharedRef<FDataprepParameterizationBinding> Binding : OriginalBindings)
		{
			const FName ParamName = BindingsContainer->GetParameterNameForBinding(Binding);
			BindObjectProperty(OutObject, Binding->PropertyChain, ParamName);
		}
	}
}

void UDataprepParameterization::GenerateClass()
{
	if ( !CustomContainerClass )
	{
		CreateClassObject();

		// Make the properties appear in a alphabetically order (for that we must add the properties to the class in the reverse order)
		NameToParameterizationProperty.KeySort([](const FName& First, const FName& Second)
		{
			return !First.LexicalLess(Second);
		});

		for ( TPair<FName, FProperty*>& Pair : NameToParameterizationProperty )
		{
			FProperty* NewProperty = CastField<FProperty>(FField::Duplicate(Pair.Value, CustomContainerClass, Pair.Key));
			NewProperty->PropertyFlags = CPF_Edit;

			// Need to manually call Link to fix-up some data (such as the C++ property flags) that are only set during Link
			{
				FArchive Ar;
				NewProperty->LinkWithoutChangingOffset( Ar );
			}

			CustomContainerClass->AddCppProperty( NewProperty );

			Pair.Value = NewProperty;
		}

		CustomContainerClass->Bind();
		CustomContainerClass->StaticLink( true );
		CustomContainerClass->AssembleReferenceTokenStream( true );

		DefaultParameterisation = static_cast<UDataprepParameterizableObject*>( CustomContainerClass->GetDefaultObject( true ) );
		OnParameterizationDefaultObjectPostEditHandle = DefaultParameterisation->GetOnPostEdit().AddUObject( this, &UDataprepParameterization::OnParameterizationDefaultObjectPostEdit );
	}
}

void UDataprepParameterization::UpdateClass()
{
	OnCustomClassAboutToBeUpdated.Broadcast();

	// Move away the old class
	PrepareCustomClassForNewClassGeneration();

	UClass* OldClass = CustomContainerClass;

	CustomContainerClass = nullptr;

	// Generate the new class
	GenerateClass();

	DoReinstancing( OldClass );
}

void UDataprepParameterization::LoadParameterization()
{
	if ( !CustomContainerClass )
	{
		CreateClassObject();

		TSet<TSharedRef<FDataprepParameterizationBinding>> BindingToRemove;
		NameToParameterizationProperty.Empty( NameToParameterizationProperty.Num() );

		for ( const TPair<TSharedRef<FDataprepParameterizationBinding>, FName>& Binding : BindingsContainer->GetBindingToParameterName() )
		{
			const FName& BindingName = Binding.Value;

			FProperty* PropertyFromChain = DataprepParameterization::GetPropertyFromBinding( Binding.Key.Get() );
			FProperty** PropertyFromParameterizationClass = NameToParameterizationProperty.Find( BindingName );

			if ( PropertyFromChain && !PropertyFromParameterizationClass )
			{
				FProperty* NewProperty = AddPropertyToClass( BindingName, *PropertyFromChain, Binding.Key->PropertyChain.Last().ContainerIndex == INDEX_NONE );
			}
			else if ( !PropertyFromChain || PropertyFromChain->GetClass() != (*PropertyFromParameterizationClass)->GetClass() )
			{
				BindingToRemove.Add( Binding.Key ) ;
			}
		}

		// Remove the invalid bindings
		TSet<UObject*> ObjectsToNotify;
		for ( const TSharedRef<FDataprepParameterizationBinding>& InvalidBinding : BindingToRemove )
		{
			BindingsContainer->RemoveBinding( InvalidBinding );
			ObjectsToNotify.Add( InvalidBinding->ObjectBinded );
		}

		// Make the properties appear in a alphabetically order (for that we must add the properties to the class in the reverse order)
		NameToParameterizationProperty.KeySort( [](const FName& First, const FName& Second)
			{
				return !First.LexicalLess( Second );
			});

		for ( const TPair<FName, FProperty*>& Pair : NameToParameterizationProperty )
		{
			CustomContainerClass->AddCppProperty( Pair.Value );
		}

		CustomContainerClass->Bind();
		CustomContainerClass->StaticLink(true);
		CustomContainerClass->AssembleReferenceTokenStream(true);

		DefaultParameterisation = static_cast<UDataprepParameterizableObject*>( CustomContainerClass->GetDefaultObject(true) );
		FDataprepParameterizationReader Reader( DefaultParameterisation, ParameterizationStorage );
		OnParameterizationDefaultObjectPostEditHandle = DefaultParameterisation->GetOnPostEdit().AddUObject( this, &UDataprepParameterization::OnParameterizationDefaultObjectPostEdit );

		if ( ObjectsToNotify.Num() > 0 )
		{
			CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &ObjectsToNotify );
		}
	}
}

void UDataprepParameterization::PrepareCustomClassForNewClassGeneration()
{
	if ( CustomContainerClass )
	{
		const FString OldClassName = MakeUniqueObjectName( GetTransientPackage(), CustomContainerClass->GetClass(), *FString::Printf(TEXT("%s_REINST"), *CustomContainerClass->GetName()) ).ToString();
		CustomContainerClass->ClassFlags |= CLASS_NewerVersionExists;
		CustomContainerClass->ClearFlags( RF_Public | RF_Standalone );
		CustomContainerClass->Rename( *OldClassName, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty );
		CustomContainerClass->SetFlags( RF_Transient | RF_NewerVersionExists );
		CustomContainerClass->GetDefaultObject()->SetFlags( RF_Transient | RF_NewerVersionExists );

		// Undo the hook for the post edit
		DefaultParameterisation->GetOnPostEdit().Remove( OnParameterizationDefaultObjectPostEditHandle );
	}
}

void UDataprepParameterization::CreateClassObject()
{
	check( !CustomContainerClass );

	CustomContainerClass = NewObject<UClass>(GetOutermost(), FName(TEXT("Parameterization")), RF_Transient);
	CustomContainerClass->SetSuperStruct( UDataprepParameterizableObject::StaticClass() );
}

void UDataprepParameterization::DoReinstancing(UClass* OldClass, bool bMigrateData)
{
	if ( OldClass && CustomContainerClass )
	{
		// For the CDO
		UObject* OldCDO = OldClass->GetDefaultObject();
		UObject* NewCDO = CustomContainerClass->GetDefaultObject();

		if ( bMigrateData )
		{
			UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
			Options.bNotifyObjectReplacement = true;
			UEngine::CopyPropertiesForUnrelatedObjects( OldClass->GetDefaultObject(), CustomContainerClass->GetDefaultObject(), Options );
		}

		// For the instances
		TArray<UObject*> Objects;
		constexpr bool bIncludeDerivedClasses = false;
		GetObjectsOfClass( OldClass, Objects, bIncludeDerivedClasses );
		TMap<UObject*, UObject*> OldToNew;
		OldToNew.Reserve( Objects.Num() + 1 );
		for (UObject* OldObject : Objects)
		{
			if ( OldObject && OldObject->IsValidLowLevel() )
			{
				FName ObjectName = OldObject->GetFName();
				UObject* Outer = OldObject->GetOuter();
				OldObject->Rename( nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors );

				ObjectName = MakeUniqueObjectName( Outer, CustomContainerClass, ObjectName );
				UObject* Object = NewObject<UObject>( Outer, CustomContainerClass, ObjectName, OldObject->GetFlags() );

				if ( bMigrateData )
				{
					UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
					Options.bNotifyObjectReplacement = true;
					UEngine::CopyPropertiesForUnrelatedObjects( OldObject, Object, Options );
				}

				OldToNew.Add( OldObject, Object );
			}
		}

		OldToNew.Add( OldCDO, NewCDO );

		/**
		 * Notify the tools
		 * If we did the data migration the tools were already notify of the change by the copy properties for unrelated objects by GEngine::CopyPropertiesForUnrelatedObjects
		 */
		if ( !bMigrateData && GEngine)
		{
			GEngine->NotifyToolsOfObjectReplacement(OldToNew);
		}

		OnCustomClassWasUpdated.Broadcast( OldToNew );
	}
}

FProperty* UDataprepParameterization::AddPropertyToClass(const FName& ParameterisationPropertyName, FProperty& Property, bool bAddFullProperty)
{
	if ( !NameToParameterizationProperty.Find( ParameterisationPropertyName ) )
	{
		FProperty* NewProperty = CastFieldChecked<FProperty>(FField::Duplicate(&Property, CustomContainerClass, ParameterisationPropertyName));
		NewProperty->PropertyFlags = CPF_Edit | CPF_NonTransactional;

		if ( !bAddFullProperty )
		{
			NewProperty->ArrayDim = 1;
		}

		// Need to manually call Link to fix-up some data (such as the C++ property flags) that are only set during Link
		{
			FArchive Ar;
			NewProperty->LinkWithoutChangingOffset( Ar );
		}

		NameToParameterizationProperty.Add( ParameterisationPropertyName, NewProperty );

		return NewProperty;
	}

	return nullptr;
}


// Just a macro to insert the definition in the right scope but to also keep the concerns separated of making the property change chain event from the function PushParametrizationValueToBindings
#define MAKEPROPERTYCHANGEDCHAINEVENT( BINDING ) \
	check( BINDING.PropertyChain.Num() > 0 ); \
	TArray<UObject*, TInlineAllocator<1>> CurrentObject; \
	CurrentObject.Add( BINDING.ObjectBinded ); \
	FPropertyChangedEvent PropertyChangedEvent( BINDING.PropertyChain.Last().CachedProperty.Get(), EPropertyChangeType::Redirected, CurrentObject ); \
	\
	FEditPropertyChain EditPropertyChain; \
	TMap<FString, int32> PropertiesIndex; \
	PropertiesIndex.Reserve( BINDING.PropertyChain.Num() ); \
	for ( const FDataprepPropertyLink& PropertyLink : BINDING.PropertyChain ) \
	{ \
		FProperty* Property = PropertyLink.CachedProperty.Get(); \
		check( Property ); \
		EditPropertyChain.AddTail( Property ); \
		PropertiesIndex.Add( Property->GetName(), PropertyLink.ContainerIndex ); \
	} \
	TArray<TMap<FString, int32>, TInlineAllocator<1>> ArrayIndexPerObject; \
	ArrayIndexPerObject.Add( PropertiesIndex ); \
	PropertyChangedEvent.SetArrayIndexPerObject( ArrayIndexPerObject ); \
	\
	TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* TailNode = EditPropertyChain.GetTail(); \
	check( TailNode ); \
	EditPropertyChain.SetActivePropertyNode( TailNode->GetValue() ); \
	\
	if ( TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* NodeBeforeTail = TailNode->GetPrevNode() ) \
	{ \
		EditPropertyChain.SetActiveMemberPropertyNode( NodeBeforeTail->GetValue() ); \
		PropertyChangedEvent.SetActiveMemberProperty( NodeBeforeTail->GetValue() ); \
	} \
	\
	FPropertyChangedChainEvent PropertyChangedChainEvent(EditPropertyChain, PropertyChangedEvent);

void UDataprepParameterization::PushParametrizationValueToBindings(FName ParameterName)
{
	if ( FProperty* ParameterizationProperty = NameToParameterizationProperty.FindRef( ParameterName ) )
	{
		void* AddressOfParameterValue = DataprepParameterization::GetAddressOf( *ParameterizationProperty, DefaultParameterisation, INDEX_NONE );

		if ( AddressOfParameterValue )
		{
			if ( const UDataprepParameterizationBindings::FSetOfBinding* Bindings = BindingsContainer->GetBindingsFromParameter( ParameterName ) )
			{
				TSet<TSharedRef<FDataprepParameterizationBinding>> BindingToRemove;

				TSet<UObject*> ObjectsModified;
				for ( const TSharedRef<FDataprepParameterizationBinding>& Binding : *Bindings )
				{
					void* AddressOfBindingValue = nullptr;
					if ( FProperty* BindingProperty = DataprepParameterization::GetPropertyFromBinding( Binding.Get(), AddressOfBindingValue ) )
					{
						UDataprepParameterizableObject* ObjectBinded = Binding->ObjectBinded;
						MAKEPROPERTYCHANGEDCHAINEVENT( Binding.Get() )
						ObjectBinded->PreEditChange( PropertyChangedChainEvent.PropertyChain );
						ObjectsModified.Add( Binding->ObjectBinded );
						ObjectBinded->Modify();
						DataprepParameterization::CopyValue( *BindingProperty, AddressOfBindingValue, *ParameterizationProperty, AddressOfParameterValue );
						ObjectBinded->PostEditChangeChainProperty( PropertyChangedChainEvent );
					}
					else
					{
						BindingToRemove.Add( Binding );
					}
				}


				CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &ObjectsModified );

				TSet<UObject*> ObjectsToNotify;
				bool bClassNeedUpdate = false;

				// Remove the invalid bindings
				for ( const TSharedRef<FDataprepParameterizationBinding>& Binding : BindingToRemove )
				{
					bool bModifiedClass;
					if ( RemoveBinding( Binding, bModifiedClass ) )
					{
						bClassNeedUpdate |= bModifiedClass;
						ObjectsToNotify.Add( Binding->ObjectBinded );
					}
				}

				if ( bClassNeedUpdate )
				{
					Modify();
					UpdateClass();
				}

				if ( ObjectsToNotify.Num() > 0 )
				{
					CastChecked<UDataprepAsset>( GetOuter() )->OnParameterizedObjectsStatusChanged.Broadcast( &ObjectsToNotify );
				}
			}
		}
	}
}

#undef MAKEPROPERTYCHANGEDCHAINEVENT


bool UDataprepParameterization::RemoveBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding, bool& bOutClassNeedUpdate)
{
	FName ParameterOfRemovedBinding = BindingsContainer->RemoveBinding( Binding );
	if ( !BindingsContainer->HasBindingsForParameter( ParameterOfRemovedBinding ) )
	{
		bOutClassNeedUpdate = true;
		NameToParameterizationProperty.Remove(ParameterOfRemovedBinding);
	}

	bool bBindingWasRemoved = !ParameterOfRemovedBinding.IsNone();

	if ( bBindingWasRemoved )
	{ 
		UDataprepParameterizableObject* Object = Binding->ObjectBinded;
		if ( !BindingsContainer->HasBindingsFromObject( Object ) && Object )
		{
			RemoveBindingToPostEditOfParameterizableObject( *Object, true );
		}
	}

	return bBindingWasRemoved;
}

void UDataprepParameterization::OnParameterizationDefaultObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	// When the parameterization push a new value to it's binding 
	if ( !bool( PropertyChangedChainEvent.ChangeType & EPropertyChangeType::Redirected ) )
	{
		TArray<FDataprepPropertyLink> PropertyChain = FDataprepParameterizationUtils::MakePropertyChain( PropertyChangedChainEvent );
		if ( PropertyChain.Num() > 0 )
		{
			PushParametrizationValueToBindings( PropertyChain[0].PropertyName );
		}
	}
}

void UDataprepParameterization::OnParameterizedObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	if ( !bool( PropertyChangedChainEvent.ChangeType & EPropertyChangeType::Redirected ) )
	{
		TArray<FDataprepPropertyLink> PropertyChain = FDataprepParameterizationUtils::MakePropertyChain( PropertyChangedChainEvent );
		if (PropertyChain.Num() > 0)
		{
			TSharedRef<FDataprepParameterizationBinding> BindingForModifiedProperty = MakeShared<FDataprepParameterizationBinding>(&Object, PropertyChain);
			TSharedPtr<FDataprepParameterizationBinding> Binding = BindingsContainer->GetContainingBinding(BindingForModifiedProperty);
			if ( Binding )
			{
				UpdateParameterizationFromBinding(Binding.ToSharedRef());
			}
		}
	}
}

void UDataprepParameterization::OnAddedBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, const FDelegateHandle& Handle)
{
	ObservedObjects.Add( &Object, Handle );
}

void UDataprepParameterization::OnRemovedBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, const FDelegateHandle& Handle)
{
	ObservedObjects.Remove( &Object );
}

void UDataprepParameterization::AddBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, bool bShouldAddToTransaction)
{
	UDataprepParameterizableObject::FOnPostEdit& OnPostEdit = Object.GetOnPostEdit();

	using FPostEditDelegate = UDataprepParameterizableObject::FOnPostEdit::FDelegate;
	FPostEditDelegate PostEditDelegate = FPostEditDelegate::CreateUObject( this, &UDataprepParameterization::OnParameterizedObjectPostEdit );

	FDelegateHandle Handle = OnPostEdit.Add( PostEditDelegate );
	OnAddedBindingToPostEditOfParameterizableObject( Object, Handle );

	using FBindingCommandChange = TDataprepBindingCommandChange<UDataprepParameterizableObject, UDataprepParameterizableObject::FOnPostEdit>;
	using FOnBindingChange = FBindingCommandChange::OnUndoRedoDelegateType;

	if ( GUndo && bShouldAddToTransaction )
	{
		constexpr bool bIsAddingBinding = true;
		GUndo->StoreUndo( &Object
			, MakeUnique<FBindingCommandChange>( bIsAddingBinding
				, Object
				, MoveTemp( PostEditDelegate )
				, FBindingCommandChange::FDelegateAccessor::CreateUObject( &Object, &UDataprepParameterizableObject::GetOnPostEdit )
				, FOnBindingChange::CreateUObject( this, &UDataprepParameterization::OnAddedBindingToPostEditOfParameterizableObject )
				, FOnBindingChange::CreateUObject( this, &UDataprepParameterization::OnRemovedBindingToPostEditOfParameterizableObject )
				, Handle
				)
			);
	}
}

void UDataprepParameterization::RemoveBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, bool bShouldAddToTransaction)
{
	if ( FDelegateHandle* HandlePtr = ObservedObjects.Find( &Object ) )
	{
		FDelegateHandle Handle = *HandlePtr;
		UDataprepParameterizableObject::FOnPostEdit& OnPostEdit = Object.GetOnPostEdit();
		OnPostEdit.Remove( Handle );
		OnRemovedBindingToPostEditOfParameterizableObject( Object, Handle );

		using FBindingCommandChange = TDataprepBindingCommandChange<UDataprepParameterizableObject, UDataprepParameterizableObject::FOnPostEdit>;
		using FOnBindingChange = FBindingCommandChange::OnUndoRedoDelegateType;

		if ( GUndo && bShouldAddToTransaction )
		{
			using FPostEditDelegate = UDataprepParameterizableObject::FOnPostEdit::FDelegate;
			FPostEditDelegate PostEditDelegate = FPostEditDelegate::CreateUObject( this, &UDataprepParameterization::OnParameterizedObjectPostEdit );

			constexpr bool bIsAddingBinding = false;
			GUndo->StoreUndo( &Object
				, MakeUnique<FBindingCommandChange>( bIsAddingBinding
					, Object
					, MoveTemp( PostEditDelegate )
					, FBindingCommandChange::FDelegateAccessor::CreateUObject( &Object, &UDataprepParameterizableObject::GetOnPostEdit )
					, FOnBindingChange::CreateUObject( this, &UDataprepParameterization::OnRemovedBindingToPostEditOfParameterizableObject )
					, FOnBindingChange::CreateUObject( this, &UDataprepParameterization::OnAddedBindingToPostEditOfParameterizableObject )
					, Handle
					)
				);
		}
	}
}

bool UDataprepParameterization::OnAssetRename(ERenameFlags Flags)
{
	if ( CustomContainerClass )
	{
		const FString NewClassName = MakeUniqueObjectName( GetOutermost(), CustomContainerClass->GetClass(), *CustomContainerClass->GetName() ).ToString();
		return CustomContainerClass->Rename( *NewClassName, GetOutermost(), Flags );
	}

	return true;
}


UDataprepParameterizationInstance::UDataprepParameterizationInstance()
{
	OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &UDataprepParameterizationInstance::OnObjectModified );
}

UDataprepParameterizationInstance::~UDataprepParameterizationInstance()
{
	FCoreUObjectDelegates::OnObjectModified.Remove( OnObjectModifiedHandle );
}

void UDataprepParameterization::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	UDataprepParameterization* This = CastChecked<UDataprepParameterization>(InThis);

	for ( TPair<FName, FProperty*>& Pair : This->NameToParameterizationProperty )
	{
		if ( Pair.Value )
		{
			Pair.Value->AddReferencedObjects( Collector );
		}
	}

	for ( auto& Pair : This->ObservedObjects)
	{
		if ( Pair.Key )
		{
			Collector.AddReferencedObject( Pair.Key );
		}
	}
}

void UDataprepParameterizationInstance::PostLoad()
{
	if ( !HasAnyFlags( RF_ClassDefaultObject | RF_NeedLoad ) )
	{
		// #ueent_hotfix: If source is null, the parent of the DataprepAssetInstance is null. So recreate a temporary source parameterization
		if(SourceParameterization == nullptr)
		{
			ensure( Cast<UDataprepAssetInstance>( GetOuter()) && Cast<UDataprepAssetInstance>( GetOuter())->GetParent() == nullptr );
			SourceParameterization = NewObject<UDataprepParameterization>( GetTransientPackage(), FName(), RF_Public );
		}
		SetFlags( RF_Public );
		LoadParameterization();
		SetupCallbacksFromSourceParameterisation();
	}
	Super::PostLoad();
}

void UDataprepParameterizationInstance::Serialize(FArchive& Ar)
{
	if ( Ar.IsSaving() && !HasAnyFlags( RF_ClassDefaultObject ) )
	{
		check( SourceParameterization );
		// Todo track when the object is changed to avoid rewriting to array each time
		ParameterizationInstanceStorage.Empty( ParameterizationInstanceStorage.Num() );
		FDataprepParameterizationWriter Writer( ParameterizationInstance, ParameterizationInstanceStorage );
	}

	Super::Serialize( Ar );
}

void UDataprepParameterizationInstance::PostEditUndo()
{
	LoadParameterization();
}

void UDataprepParameterizationInstance::OnObjectModified(UObject* Object)
{
	if ( Object == ParameterizationInstance )
	{
		Modify();
	}
}

void UDataprepParameterizationInstance::ApplyParameterization(const TMap<UObject*, UObject*>& SourceToCopy)
{
	check( SourceParameterization );
	for ( const TPair<TSharedRef<FDataprepParameterizationBinding>, FName>& BindingPair : SourceParameterization->BindingsContainer->GetBindingToParameterName() )
	{
		const TSharedRef<FDataprepParameterizationBinding>& Binding = BindingPair.Key;
		if ( UDataprepParameterizableObject* Object = Cast<UDataprepParameterizableObject>( SourceToCopy.FindRef( Binding->ObjectBinded ) ) )
		{
			TGuardValue<decltype(Binding->ObjectBinded)> GuardObjectBinded( Binding->ObjectBinded, Object );

			void* DestinationAddress = nullptr;
			if ( FProperty* DestinationProperty = DataprepParameterization::GetPropertyFromBinding( Binding.Get(), DestinationAddress ) )
			{
				FProperty* ParameterizationProperty = FindFProperty<FProperty>( SourceParameterization->CustomContainerClass, BindingPair.Value );
				void* ParameterizationAddress =  DataprepParameterization::GetAddressOf( *ParameterizationProperty, ParameterizationInstance, INDEX_NONE );
				DataprepParameterization::CopyValue( *DestinationProperty, DestinationAddress, *ParameterizationProperty, ParameterizationAddress );
			}
		}
	}
}

void UDataprepParameterizationInstance::CustomClassAboutToBeUpdated()
{
	// The instance is about to be modified
	Modify();
}

void UDataprepParameterizationInstance::CustomClassWasUpdated(const TMap<UObject*, UObject*>& OldToNew)
{
	if ( UObject* NewInstance = OldToNew.FindRef( ParameterizationInstance ) )
	{
		ParameterizationInstance = NewInstance;
	}
}

void UDataprepParameterizationInstance::LoadParameterization()
{
	check( SourceParameterization );

	if ( !SourceParameterization->CustomContainerClass )
	{
		SourceParameterization->ConditionalPostLoad();
	}

	if ( !ParameterizationInstance )
	{
		ParameterizationInstance = NewObject<UObject>( this, SourceParameterization->CustomContainerClass, FName(TEXT("Parameterization")), RF_Transient );
	}

	FDataprepParameterizationReader Reader( ParameterizationInstance, ParameterizationInstanceStorage );
}

void UDataprepParameterizationInstance::SetParameterizationSource(UDataprepParameterization& Parameterization)
{
	UndoSetupForCallbacksFromParameterization();

	SourceParameterization = &Parameterization;
	SetupCallbacksFromSourceParameterisation();

	// Reload the parameterization (this act as a sort of data migration process)
	LoadParameterization();
}

void UDataprepParameterizationInstance::SetupCallbacksFromSourceParameterisation()
{
	check( SourceParameterization );

	SourceParameterization->OnCustomClassAboutToBeUpdated.AddUObject( this, &UDataprepParameterizationInstance::CustomClassAboutToBeUpdated );
	SourceParameterization->OnCustomClassWasUpdated.AddUObject( this, &UDataprepParameterizationInstance::CustomClassWasUpdated );
	SourceParameterization->OnTellInstancesToReloadTheirSerializedData.AddUObject( this, &UDataprepParameterizationInstance::LoadParameterization );
}

void UDataprepParameterizationInstance::UndoSetupForCallbacksFromParameterization()
{
	if (SourceParameterization)
	{
		SourceParameterization->OnCustomClassAboutToBeUpdated.RemoveAll( this );
		SourceParameterization->OnCustomClassWasUpdated.RemoveAll( this );
		SourceParameterization->OnTellInstancesToReloadTheirSerializedData.RemoveAll( this );
	}
}
