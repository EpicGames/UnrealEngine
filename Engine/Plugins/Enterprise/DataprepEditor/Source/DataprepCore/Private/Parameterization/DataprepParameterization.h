// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Parameterization/DataprepParameterizationUtils.h"


#include "DataprepParameterization.generated.h"

class FFieldClass;
class FProperty;
class FTransactionObjectEvent;
class UDataprepParameterizableObject;
class UDataprepActionAsset;

struct FValueTypeValidationData
{
	TArray<TObjectPtr<UObject>> ObjectData;
	TArray<TFieldPath<FProperty>> PropertyData;
	TArray<FFieldClass*> PropertyTypeData;
	
	bool Serialize(FArchive& Ar);
	void AddReferencedObjects(FReferenceCollector& Collector);

	void Add(UObject* InObj)
	{
		ObjectData.Add(InObj);
	}
	void Add(FProperty* InProp)
	{
		PropertyData.Add(TFieldPath<FProperty>(InProp));
	}
	void Add(FFieldClass* InClass)
	{
		PropertyTypeData.Add(InClass);
	}
	void Reserve(int32 Count)
	{
		// Do nothing or TBD
	}
	int32 Num() const
	{
		return ObjectData.Num() + PropertyData.Num() + PropertyTypeData.Num();
	}
	void Empty()
	{
		ObjectData.Empty();
		PropertyData.Empty();
		PropertyTypeData.Empty();
	}
	bool operator == (const FValueTypeValidationData& Other) const
	{
		return ObjectData == Other.ObjectData && PropertyData == Other.PropertyData && PropertyTypeData == Other.PropertyTypeData;
	}
};

/**
 * The parameterization binding is a struct that hold an object and the property path to the parameterized property
 * It also hold a array to validate the that value type of the parameterized property didn't change since it's creation
 */
USTRUCT()
struct FDataprepParameterizationBinding
{
	GENERATED_BODY()

	FDataprepParameterizationBinding()
		: ValueTypeValidationData()
	{}

	FDataprepParameterizationBinding(UDataprepParameterizableObject* InObjectBinded, TArray<FDataprepPropertyLink> InPropertyChain);

	FDataprepParameterizationBinding(FDataprepParameterizationBinding&&) = default;
	FDataprepParameterizationBinding(const FDataprepParameterizationBinding&) = default;
	FDataprepParameterizationBinding& operator=(FDataprepParameterizationBinding&&) = default;
	FDataprepParameterizationBinding& operator=(const FDataprepParameterizationBinding&) = default;

	bool operator==(const FDataprepParameterizationBinding& Other) const;

	UPROPERTY()
	TObjectPtr<UDataprepParameterizableObject> ObjectBinded = nullptr;

	UPROPERTY()
	TArray<FDataprepPropertyLink> PropertyChain;

	// Value Type Validation Array. This is the result of a depth first search on the parametrized property
	//UPROPERTY() // @todo FProp: do we need this to be an FProp? (is this struct being serialized)
	FValueTypeValidationData ValueTypeValidationData;
};

uint32 GetTypeHash(const FDataprepParameterizationBinding& Binding);

uint32 GetTypeHash(const TArray<FDataprepPropertyLink>& PropertyLinks);

/** 
 * A override to defines how the map's pairs are hashed and compared.
 * This allow us to have some map that work with TSharedRef but compare the object value instead of the pointer
 */
struct FDataprepParametrizationBindingMapKeyFuncs : TDefaultMapHashableKeyFuncs<TSharedRef<FDataprepParameterizationBinding>, FName, false>
{
	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.Get() == B.Get();
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash( Key.Get() );
	}
};

/** 
 * A override to defines how the set values are hashed and compared.
 * This allow us to have some set that work with TSharedRef but compare the object value instead of the pointer
 */
struct FDataprepParametrizationBindingSetKeyFuncs : DefaultKeyFuncs<TSharedRef<FDataprepParameterizationBinding>>
{
	template<typename ComparableKey>
	static FORCEINLINE bool Matches(KeyInitType A, ComparableKey B)
	{
		return A.Get() == B.Get();
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key.Get());
	}
};

/**
 * Encapsulate the unidirectionality necessary for a constant cost of access to the data related to the bindings
 */
UCLASS(MinimalAPI)
class UDataprepParameterizationBindings : public UObject
{
public:

	GENERATED_BODY()

	using FBindingToParameterNameMap = TMap<TSharedRef<FDataprepParameterizationBinding>, FName, FDefaultSetAllocator, FDataprepParametrizationBindingMapKeyFuncs>;
	using FSetOfBinding = TSet<TSharedRef<FDataprepParameterizationBinding>, FDataprepParametrizationBindingSetKeyFuncs>;

	/**
	 * Does the data structure contains this binding
	 */
	bool ContainsBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const;

	/**
	 * Return the name of the parameter for a binding
	 */
	FName GetParameterNameForBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const;

	/**
	 * Return a valid ptr if the object had some bindings
	 */
	const FSetOfBinding* GetBindingsFromObject(const UDataprepParameterizableObject* Object) const;

	/**
	 * Get the bindings from a parameter
	 * Return nullptr if the parameter doesn't exist
	 */
	const FSetOfBinding* GetBindingsFromParameter(const FName& ParameterName) const;

	/**
	 * Does the data structure has some bindings for the parameter name
	 */
	bool HasBindingsForParameter(const FName& ParameterName) const;

	/**
	 * Does the data structure has some bindings from the specified object
	 */
	bool HasBindingsFromObject(UDataprepParameterizableObject* Object) const;

	/**
	 * Add a binding and map it to the parameter
	 */
	void Add(const TSharedRef<FDataprepParameterizationBinding>& Binding, const FName& ParameterName, FSetOfBinding& OutBindingsContainedByNewBinding);

	/**
	 * Remove a binding.
	 * @return The name of the parameter the binding was associated with
	 */
	FName RemoveBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding);

	/**
	 * Remove all the bindings from a object
	 * @return The name of the parameters that were associated to the binding of the object
	 */
	TSet<FName> RemoveAllBindingsFromObject(UDataprepParameterizableObject* Object);

	/**
	 * @return A valid pointer if the binding is a part of an existing binding
	 */
	TSharedPtr<FDataprepParameterizationBinding> GetContainingBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding) const;

	const FBindingToParameterNameMap& GetBindingToParameterName() const;

	TArray<UDataprepParameterizableObject*> GetParameterizedObjects() const;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// End of UObject interface

private:
	/**
	 * Do the actual serialization when saving
	 */
	void Save(FArchive& Ar);

	/**
	 * Do the actual serialization when reloading
	 */
	void Load(FArchive& Ar);

	/** Core storage also track a binding to it's parameter name */
	FBindingToParameterNameMap BindingToParameterName;

	/** Track the name usage for parameters */
	TMap<FName, FSetOfBinding> NameToBindings;

	/** Track which binding a object has */
	TMap<TObjectPtr<UDataprepParameterizableObject>, FSetOfBinding> ObjectToBindings;
};


/** 
 * The DataprepParameterization contains the data for the parameterization of a pipeline
 */
UCLASS(MinimalAPI)
class UDataprepParameterization : public UObject
{
public:
	GENERATED_BODY()

	UDataprepParameterization();

	// UObject interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	virtual void FinishDestroy() override;
	// End of UObject interface

	void OnObjectModified(UObject* Object);

	UObject* GetDefaultObject();

	bool BindObjectProperty(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain, const FName& Name);

	bool IsObjectPropertyBinded(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain) const;

	FName GetNameOfParameterForObjectProperty(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& PropertyChain) const;

	void RemoveBindedObjectProperty(UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain);

	void RemoveBindingFromObjects(const TArrayView<UDataprepParameterizableObject*>& Objects);

	void GetExistingParameterNamesForType(FProperty* Property, bool bIsDescribingFullProperty, TSet<FString>& OutValidExistingNames, TSet<FString>& OutInvalidNames) const;

	void DuplicateObjectParamaterization(const UDataprepParameterizableObject* InObject, UDataprepParameterizableObject* OutObject);

private:

	/**
	 * Generate the Custom Container Class
	 */
	void GenerateClass();

	/**
	 * Update the Custom Container Class to a newer version
	 */
	void UpdateClass();

	/**
	 * Do the process of regenerating the Custom Container Class and the data of its default object from the serialized data
	 */
	void LoadParameterization();

	/**
	 * Remove the current Custom Container Class so that we can create a new one
	 */
	void PrepareCustomClassForNewClassGeneration();

	/**
	 * Do the actual creation of the class object
	 */
	void CreateClassObject();

	/**
	 * Do reinstancing of the objects created from the Custom Container Class
	 * @param OldClass The previous Custom Constainer Class
	 * @param bMigrateData Should we migrate the data from the old instances to the new instances
	 */
	void DoReinstancing(UClass* OldClass, bool bMigrateData = true);

	/**
	 * Try adding a binded property to the parameterization class
	 */
	FProperty* AddPropertyToClass(const FName& ParameterisationPropertyName, FProperty& Property, bool bAddFullProperty);

	/**
	 * Get a new value for the parameterization from it's associated binding
	 */
	void UpdateParameterizationFromBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding);

	/**
	 * Push the value of the parametrization to the bindings
	 */
	void PushParametrizationValueToBindings(FName ParameterName);

	/**
	 * Do the actual removing of a binding
	 * @return True if binding was remove
	 */
	bool RemoveBinding(const TSharedRef<FDataprepParameterizationBinding>& Binding, bool& bOutClassNeedUpdate);

	/**
	 * Functions for the tracking of the post edit change
	 */
	void OnParameterizationDefaultObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent);
	void OnParameterizedObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent);
	void OnAddedBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, const FDelegateHandle& Handle);
	void OnRemovedBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, const FDelegateHandle& Handle);
	void AddBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, bool bShouldAddToTransaction);
	void RemoveBindingToPostEditOfParameterizableObject(UDataprepParameterizableObject& Object, bool bShouldAddToTransaction);

public:

	/**
	 * Update the package of the generated class
	 * Return if it can be rename
	 */
	bool OnAssetRename(ERenameFlags Flags);

private:
	// The containers for the bindings
	UPROPERTY()
	TObjectPtr<UDataprepParameterizationBindings> BindingsContainer;

	/**
	* Mapping of the observed object to their delegate handle
	* The name are relative to the dataprep asset
	*/
	TMap<TObjectPtr<UDataprepParameterizableObject>, FDelegateHandle> ObservedObjects;

	//TMap<UDataprepActionAsset*, TPair<FDelegateHandle, uint32> ObservedAction;

	TMap<FName, FProperty*> NameToParameterizationProperty;

	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UClass> CustomContainerClass;

	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UDataprepParameterizableObject> DefaultParameterisation;

	/** 
	 * This is used only to store a serialization of the values of the parameterization since we can't save our custom container class
	 */
	UPROPERTY()
	TArray<uint8> ParameterizationStorage;

	DECLARE_EVENT(UDataprepParameterization, FOnCustomClassAboutToBeUpdated);
	FOnCustomClassAboutToBeUpdated OnCustomClassAboutToBeUpdated;

	using FMapOldToNewObjects = TMap<UObject*, UObject*>;
	DECLARE_EVENT_OneParam(UDataprepParameterization, FOnCustomClassWasUpdated, const FMapOldToNewObjects& /** OldToNew */);
	FOnCustomClassWasUpdated OnCustomClassWasUpdated;

	DECLARE_EVENT(UDataprepParameterization,FOnTellInstancesToReloadTheirSerializedData);
	FOnTellInstancesToReloadTheirSerializedData OnTellInstancesToReloadTheirSerializedData;

	// the dataprep instance need some special access to the dataprep parameterization
	friend class UDataprepParameterizationInstance;

	FDelegateHandle OnObjectModifiedHandle;

	FDelegateHandle OnParameterizationDefaultObjectPostEditHandle;
};


UCLASS(MinimalAPI)
class UDataprepParameterizationInstance : public UObject
{
public:
	GENERATED_BODY()

	UDataprepParameterizationInstance();
	virtual ~UDataprepParameterizationInstance();

	// UObject interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditUndo() override;
	// End of UObject interface

	void OnObjectModified(UObject* Object);

	// Apply the parameterization to a copy of the source pipeline
	void ApplyParameterization(const TMap<UObject*, UObject*>& SourceToCopy);

	void SetParameterizationSource(UDataprepParameterization& Parameterization);

	UObject* GetParameterizationInstance() { return ParameterizationInstance; }

private:

	void CustomClassAboutToBeUpdated();

	/**
	 * Used as call back for event coming from the source parameterization
	 * Change the parametrization instance to the new object after a reinstancing
	 */
	void CustomClassWasUpdated(const TMap<UObject*, UObject*>& OldToNew);

	/**
	 * Load the parameterization data on the instance from the ParameterizationInstanceStorage
	 */
	void LoadParameterization();

	/**
	 * Setup the parameterization instance so that we can react to event coming from the source parameterization
	 */
	void SetupCallbacksFromSourceParameterisation();

	/**
	 * Clean the parameterization instance so that we can bind to a new source parameterization
	 */
	void UndoSetupForCallbacksFromParameterization();

	// The parameterization from which this instance was constructed
	UPROPERTY()
	TObjectPtr<UDataprepParameterization> SourceParameterization;

	// The actual object on which the parameterization data is stored
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UObject> ParameterizationInstance;

	// This is used only to store a serialization of the values of the parameterization since we can't save the custom class
	UPROPERTY()
	TArray<uint8> ParameterizationInstanceStorage;

	FDelegateHandle OnObjectModifiedHandle;
};
