// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/DataprepParameterizationUtils.h"

#include "DataprepAsset.h"

#include "PropertyHandle.h"
#include "UObject/TextProperty.h"

namespace DataprepParameterizationUtils
{
	bool IsAContainerProperty(FProperty* Property)
	{
		if ( Property )
		{
			if ( FFieldClass* PropertyClass = Property->GetClass() )
			{
				if ( PropertyClass == FArrayProperty::StaticClass()
					|| PropertyClass == FSetProperty::StaticClass()
					|| PropertyClass == FMapProperty::StaticClass() )
				{
					return true;
				}
			}
		}

		return false;
	}

	bool IsASupportedClassForParameterization(UClass* Class)
	{
		return Class->IsChildOf<UDataprepParameterizableObject>();
	}

}

bool operator==(const FDataprepPropertyLink& A, const FDataprepPropertyLink& B)
{
	return A.PropertyName == B.PropertyName && A.ContainerIndex == B.ContainerIndex;
}

uint32 GetTypeHash(const FDataprepPropertyLink& PropertyLink)
{
	return HashCombine( GetTypeHash( PropertyLink.PropertyName ), GetTypeHash( PropertyLink.ContainerIndex ) );
}

TArray<FDataprepPropertyLink> FDataprepParameterizationUtils::MakePropertyChain(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TArray<FDataprepPropertyLink> PropertyChain;
	
	if ( PropertyHandle.IsValid() )
	{ 
		TSharedPtr<IPropertyHandle> CurrentHandle = PropertyHandle;
		FProperty* Property = CurrentHandle->GetProperty();

		TSharedPtr<IPropertyHandle> ParentHandle;

		while ( CurrentHandle.IsValid() &&  Property )
		{
			bool bWasProccess = false;
			ParentHandle = CurrentHandle->GetParentHandle();
				
			if ( ParentHandle.IsValid() )
			{ 
				FProperty* ParentProperty = ParentHandle->GetProperty();

				// We manipulate a bit the chain to store the property inside a container in a special way. So that we can 
				if ( DataprepParameterizationUtils::IsAContainerProperty( ParentProperty ) )
				{
					PropertyChain.Emplace( Property, Property->GetFName(), INDEX_NONE );
					PropertyChain.Emplace( ParentProperty, ParentProperty->GetFName(), CurrentHandle->GetIndexInArray() );
					CurrentHandle = ParentHandle->GetParentHandle();
					bWasProccess = true;
				}
			}

			if ( !bWasProccess )
			{
				PropertyChain.Emplace( Property, Property->GetFName(), CurrentHandle->GetIndexInArray() );
				CurrentHandle = CurrentHandle->GetParentHandle();
			}

			if ( CurrentHandle )
			{
				FProperty* NextProperty = CurrentHandle->GetProperty();

				// Deal with the case of a property of a raw c++ array
				while ( CurrentHandle && NextProperty == Property )
				{
					// Skip the new current handle as it point on the same property
					CurrentHandle = CurrentHandle->GetParentHandle();
					if ( CurrentHandle.IsValid() )
					{
						NextProperty = CurrentHandle->GetProperty();
					}
					else
					{
						NextProperty = nullptr;
					}
				}

				Property = NextProperty;
			}
		}
		
	}

	// Reverse the array to be from top property to bottom
	int32 Lower = 0;
	int32 Top = PropertyChain.Num() - 1;
	while ( Lower < Top )
	{
		PropertyChain.Swap( Lower, Top );
		Lower++;
		Top--;
	}

	return PropertyChain;
}

TArray<FDataprepPropertyLink> FDataprepParameterizationUtils::MakePropertyChain(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// This implementation is base on what the property editor does in general
	const FEditPropertyChain& EditPropertyChain = PropertyChangedEvent.PropertyChain;

	TArray<FDataprepPropertyLink> DataprepPropertyChain;
	DataprepPropertyChain.Reserve( EditPropertyChain.Num() + 1 );

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* CurrentNode = EditPropertyChain.GetHead();

	FProperty* Property = nullptr;
	while( CurrentNode )
	{
		Property = CurrentNode->GetValue();
		if ( Property )
		{ 
			int32 ContainerIndex = PropertyChangedEvent.GetArrayIndex( Property->GetName() );
			DataprepPropertyChain.Emplace( Property, Property->GetFName(), ContainerIndex);

			// Used to validate if we should skip the next property
			FProperty* PossibleNextProperty = nullptr;

			if ( Property->GetClass() == FArrayProperty::StaticClass() )
			{
				FArrayProperty* ArrayProperty  = static_cast<FArrayProperty*>( Property );
				FProperty* ArrayTypeProperty = ArrayProperty->Inner;
				DataprepPropertyChain.Emplace( ArrayTypeProperty, ArrayTypeProperty->GetFName(), INDEX_NONE );
				PossibleNextProperty = ArrayTypeProperty;
			}
			else if ( Property->GetClass() == FSetProperty::StaticClass() )
			{
				FSetProperty* SetProperty = static_cast<FSetProperty*>(Property);
				FProperty* SetTypeProperty = SetProperty->ElementProp;
				DataprepPropertyChain.Emplace(SetTypeProperty, SetTypeProperty->GetFName(), INDEX_NONE);
				PossibleNextProperty = SetTypeProperty;
			}

			// We can't deal with map yet du to a lack of information from the property chain

			
			if ( PossibleNextProperty )
			{ 
				const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* NextNode = CurrentNode->GetNextNode();
				if ( NextNode && PossibleNextProperty == NextNode->GetValue() )
				{
					//skip the next property
					CurrentNode = NextNode->GetNextNode();
				}
			}
		}
		else
		{
			return {};
		}

		CurrentNode = CurrentNode->GetNextNode();
	}

	if ( Property != PropertyChangedEvent.Property )
	{
		DataprepPropertyChain.Emplace( PropertyChangedEvent.Property, PropertyChangedEvent.Property->GetFName(), INDEX_NONE );
	}

	return DataprepPropertyChain;
}

FDataprepParameterizationContext FDataprepParameterizationUtils::CreateContext(TSharedPtr<IPropertyHandle> PropertyHandle, const FDataprepParameterizationContext& ParameterisationContext)
{
	FDataprepParameterizationContext NewContext;
	NewContext.State = ParameterisationContext.State;

	if ( NewContext.State == EParametrizationState::CanBeParameterized || NewContext.State == EParametrizationState::InvalidForParameterization )
	{
		// This implementation could be improved by incrementally expending the property chain.
		NewContext.PropertyChain = MakePropertyChain( PropertyHandle );
		NewContext.State = IsPropertyChainValid( NewContext.PropertyChain ) ? EParametrizationState::CanBeParameterized : EParametrizationState::InvalidForParameterization;
	}
	else if ( NewContext.State == EParametrizationState::IsParameterized )
	{
		NewContext.State = EParametrizationState::ParentIsParameterized;
	}

	return NewContext;
}

UDataprepAsset* FDataprepParameterizationUtils::GetDataprepAssetForParameterization(UObject* Object)
{
	if ( Object )
	{
		// 1. Check if the object class is part the dataprep parameterization ecosystem
		UClass* Class = Object->GetClass();
		while ( Class )
		{
			if ( DataprepParameterizationUtils::IsASupportedClassForParameterization( Class ) )
			{
				break;
			}

			Class = Class->GetSuperClass();
		}

		// 2. Check if the object inside a dataprep asset
		if ( Class )
		{
			Object = Object->GetOuter();
			const UClass* DataprepAssetClass = UDataprepAsset::StaticClass();
			while ( Object )
			{
				if ( Object->GetClass() == DataprepAssetClass )
				{
					// 3. Return the dataprep asset that own the object
					return static_cast<UDataprepAsset*>( Object );
				}

				Object = Object->GetOuter();
			}
		}
	}

	return nullptr;
}

bool FDataprepParameterizationUtils::IsPropertyChainValid(const TArray<FDataprepPropertyLink>& PropertyChain)
{
	if ( PropertyChain.Num() == 0 )
	{
		return false;
	}

	bool bParentWasAContainer = false;
	for ( const FDataprepPropertyLink& PropertyLink : PropertyChain )
	{
		FProperty* Property = PropertyLink.CachedProperty.Get();
		if ( !Property )
		{
			return false;
		}

		// Ensure that the properties are editable
		if ( !bParentWasAContainer && !bool( Property->PropertyFlags & CPF_Edit ) )
		{
			return false;
		}

		if ( Property->ArrayDim <= PropertyLink.ContainerIndex )
		{
			return false;
		}

		// Temporary unsupported properties check
		{
			// We don't support properties chain that are a subpart of a container property
			if ( bParentWasAContainer )
			{
				return false;
			}

			// We are not able to serialize the text properties yet
			if ( Property->GetClass() == FTextProperty::StaticClass() )
			{
				return false;
			}

			bParentWasAContainer = DataprepParameterizationUtils::IsAContainerProperty( Property );
		}
	}

	return true;
}
