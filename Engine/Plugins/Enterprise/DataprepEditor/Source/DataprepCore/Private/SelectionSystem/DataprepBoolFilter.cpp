// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepBoolFilter.h"

#include "DataprepCoreLogCategory.h"
#include "SelectionSystem/DataprepBoolFetcher.h"
#include "SelectionSystem/DataprepSelectionSystemUtils.h"


bool UDataprepBoolFilter::Filter(const bool bResult) const
{
	// Simply exist to reuse the templated function in FilterObjects
	return bResult;
}

TArray<UObject*> UDataprepBoolFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	if ( BoolFetcher )
	{
		return DataprepSelectionSystemUtils::FilterObjects< UDataprepBoolFilter, UDataprepBoolFetcher, bool >( *this, * BoolFetcher, Objects );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepBoolFilter::FilterObjects: There was no Fetcher" );
	return {};
}

void UDataprepBoolFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	if ( BoolFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndGatherInfo< UDataprepBoolFilter, UDataprepBoolFetcher, bool >( *this, *BoolFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepBoolFilter::FilterAndGatherInfo: There was no Fetcher" );
}

void UDataprepBoolFilter::FilterAndStoreInArrayView(const TArrayView<UObject *>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	if ( BoolFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndStoreInArrayView< UDataprepBoolFilter, UDataprepBoolFetcher, bool >( *this, *BoolFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepBoolFilter::FilterAndStoreInArrayView: There was no Fetcher" );
}

FText UDataprepBoolFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepBoolFilter", "BoolFilterCategory", "Condition");
}

TSubclassOf<UDataprepFetcher> UDataprepBoolFilter::GetAcceptedFetcherClass() const
{
	return UDataprepBoolFetcher::StaticClass();
}

void UDataprepBoolFilter::SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass)
{
		UClass* NewFetcherClass = FetcherClass;

	if ( NewFetcherClass && NewFetcherClass->IsChildOf( GetAcceptedFetcherClass() ) )
	{
		UClass* OldFetcherClass = BoolFetcher ? BoolFetcher->GetClass() : nullptr;
		if ( NewFetcherClass != OldFetcherClass )
		{
			Modify();
			BoolFetcher = NewObject< UDataprepBoolFetcher >( this, NewFetcherClass, NAME_None, RF_Transactional );
		}
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepBoolFilter::SetFetcher: The Fetcher Class is not compatible" );
	}
}

const UDataprepFetcher* UDataprepBoolFilter::GetFetcherImplementation() const
{
	return BoolFetcher;
}
