// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepFetcher.h"

FText UDataprepFetcher::GetDisplayFetcherName_Implementation() const 
{
	return GetClass()->GetDisplayNameText();
}

FText UDataprepFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return GetClass()->GetDisplayNameText();
}

FText UDataprepFetcher::GetTooltipText_Implementation() const
{
	return GetClass()->GetToolTipText();
}

FText UDataprepFetcher::GetAdditionalKeyword_Implementation() const
{
	return {};
}
