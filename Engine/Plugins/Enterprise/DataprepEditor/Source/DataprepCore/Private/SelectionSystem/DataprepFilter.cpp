// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepFilter.h"

TMap<UClass*, UClass*> UDataprepFilter::FetcherClassToFilterClass;

void UDataprepFilter::PostCDOContruct()
{
	Super::PostCDOContruct();
	UClass* ThisClass = GetClass();
	if ( !ThisClass->HasAnyClassFlags( CLASS_Abstract ) )
	{
		// Todo investigate the potential issue of this with the hot reloading
		UClass* FetcherClass = GetAcceptedFetcherClass().Get();
		uint32 ClassHash = GetTypeHash(FetcherClass);
		FetcherClassToFilterClass.AddByHash(ClassHash, FetcherClass, ThisClass);
	}
}

UClass* UDataprepFilter::GetFilterTypeForFetcherType(UClass* FetcherClass)
{
	while (FetcherClass)
	{
		if (UClass** FilterClass = FetcherClassToFilterClass.Find(FetcherClass))
		{
			return *FilterClass;
		}

		FetcherClass = FetcherClass->GetSuperClass();
	}

	return nullptr;
}

FText UDataprepFilterNoFetcher::GetDisplayFilterName_Implementation() const
{
	return GetClass()->GetDisplayNameText();
}

FText UDataprepFilterNoFetcher::GetNodeDisplayFilterName_Implementation() const
{
	return GetClass()->GetDisplayNameText();
}

FText UDataprepFilterNoFetcher::GetTooltipText_Implementation() const
{
	return GetClass()->GetToolTipText();
}

FText UDataprepFilterNoFetcher::GetAdditionalKeyword_Implementation() const
{
	return {};
}
