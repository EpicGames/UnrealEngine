// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepFloatFilter.h"

#include "DataprepCoreLogCategory.h"
#include "Misc/TVariantMeta.h"
#include "SelectionSystem/DataprepSelectionSystemUtils.h"
#include "SelectionSystem/DataprepFloatFetcher.h"

bool UDataprepFloatFilter::Filter(float Float) const
{
	switch ( FloatMatchingCriteria )
	{
	case EDataprepFloatMatchType::LessThan:
		return Float < EqualValue;
	case EDataprepFloatMatchType::GreatherThan:
		return Float > EqualValue;
	case EDataprepFloatMatchType::IsNearlyEqual:
		return FMath::IsNearlyEqual( Float, EqualValue, Tolerance );
	default:
		check( false );
		break;
	}

	return false;
}

TArray<UObject*> UDataprepFloatFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	if ( FloatFetcher )
	{
		return DataprepSelectionSystemUtils::FilterObjects< UDataprepFloatFilter, UDataprepFloatFetcher, float >( *this, *FloatFetcher, Objects );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepFloatFilter::FilterObjects: There was no Fetcher" );
	return {};
}

void UDataprepFloatFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	if ( FloatFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndGatherInfo< UDataprepFloatFilter, UDataprepFloatFetcher, float >( *this, *FloatFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepFloatFilter::FilterAndGatherInfo: There was no Fetcher" );
}

void UDataprepFloatFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	if ( FloatFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndStoreInArrayView< UDataprepFloatFilter, UDataprepFloatFetcher, float >( *this, *FloatFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepFloatFilter::FilterAndStoreInArrayView: There was no Fetcher" );
}

FText UDataprepFloatFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepFloatFilter", "FloatFilterCategory", "Float");
}

TSubclassOf<UDataprepFetcher> UDataprepFloatFilter::GetAcceptedFetcherClass() const
{
	return UDataprepFloatFetcher::StaticClass();
}

void UDataprepFloatFilter::SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass)
{
	UClass* NewFetcherClass = FetcherClass;
	if ( NewFetcherClass && NewFetcherClass->IsChildOf( GetAcceptedFetcherClass() ) )
	{
		UClass* OldFetcherClass = FloatFetcher ? FloatFetcher->GetClass() : nullptr;
		if ( NewFetcherClass != OldFetcherClass )
		{
			Modify();
			FloatFetcher = NewObject< UDataprepFloatFetcher >( this, NewFetcherClass, NAME_None, RF_Transactional );
		}
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepFloatFilter::SetFetcher: The Fetcher Class is not compatible" );
	}
}

const UDataprepFetcher* UDataprepFloatFilter::GetFetcherImplementation() const
{
	return FloatFetcher;
}

EDataprepFloatMatchType UDataprepFloatFilter::GetFloatMatchingCriteria() const
{
	return FloatMatchingCriteria;
}

float UDataprepFloatFilter::GetEqualValue() const
{
	return EqualValue;
}

float UDataprepFloatFilter::GetTolerance() const
{
	return Tolerance;
}

void UDataprepFloatFilter::SetFloatMatchingCriteria(EDataprepFloatMatchType InFloatMatchingCriteria)
{
	if ( FloatMatchingCriteria != InFloatMatchingCriteria )
	{
		Modify();
		FloatMatchingCriteria = InFloatMatchingCriteria;
	}
}

void UDataprepFloatFilter::SetEqualValue(float InEqualValue)
{
	if ( EqualValue != InEqualValue )
	{
		Modify();
		EqualValue = InEqualValue;
	}
}

void UDataprepFloatFilter::SetTolerance(float InTolerance)
{
	if ( Tolerance != InTolerance )
	{
		Modify();
		Tolerance = InTolerance;
	}
}
