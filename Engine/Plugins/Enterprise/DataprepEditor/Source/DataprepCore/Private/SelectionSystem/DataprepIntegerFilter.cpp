// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepIntegerFilter.h"

#include "DataprepCoreLogCategory.h"
#include "Misc/TVariantMeta.h"
#include "SelectionSystem/DataprepSelectionSystemUtils.h"
#include "SelectionSystem/DataprepIntegerFetcher.h"

bool UDataprepIntegerFilter::Filter(int Integer) const
{
	switch ( IntegerMatchingCriteria )
	{
	case EDataprepIntegerMatchType::LessThan:
		return Integer < EqualValue;
	case EDataprepIntegerMatchType::GreatherThan:
		return Integer > EqualValue;
	case EDataprepIntegerMatchType::IsEqual:
		return Integer == EqualValue;
	case EDataprepIntegerMatchType::InBetween:
		return Integer >= FromValue &&  Integer <= ToValue;
	default:
		check( false );
		break;
	}

	return false;
}

TArray<UObject*> UDataprepIntegerFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	if ( IntFetcher )
	{
		return DataprepSelectionSystemUtils::FilterObjects< UDataprepIntegerFilter, UDataprepIntegerFetcher, int32 >( *this, *IntFetcher, Objects );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepIntegerFilter::FilterObjects: There was no Fetcher" );
	return {};
}

void UDataprepIntegerFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	if ( IntFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndGatherInfo< UDataprepIntegerFilter, UDataprepIntegerFetcher, int32 >( *this, *IntFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepIntegerFilter::FilterAndGatherInfo: There was no Fetcher" );
}

void UDataprepIntegerFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	if ( IntFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndStoreInArrayView< UDataprepIntegerFilter, UDataprepIntegerFetcher, int32 >( *this, *IntFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepIntegerFilter::FilterAndStoreInArrayView: There was no Fetcher" );
}

FText UDataprepIntegerFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepIntegerFilter", "IntegerFilterCategory", "Integer");
}

TSubclassOf<UDataprepFetcher> UDataprepIntegerFilter::GetAcceptedFetcherClass() const
{
	return UDataprepIntegerFetcher::StaticClass();
}

void UDataprepIntegerFilter::SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass)
{
	UClass* NewFetcherClass = FetcherClass;
	if ( NewFetcherClass && NewFetcherClass->IsChildOf( GetAcceptedFetcherClass() ) )
	{
		UClass* OldFetcherClass = IntFetcher ? IntFetcher->GetClass() : nullptr;
		if ( NewFetcherClass != OldFetcherClass )
		{
			Modify();
			IntFetcher = NewObject< UDataprepIntegerFetcher >( this, NewFetcherClass, NAME_None, RF_Transactional );
		}
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepIntegerFilter::SetFetcher: The Fetcher Class is not compatible" );
	}
}

const UDataprepFetcher* UDataprepIntegerFilter::GetFetcherImplementation() const
{
	return IntFetcher;
}

EDataprepIntegerMatchType UDataprepIntegerFilter::GetIntegerMatchingCriteria() const
{
	return IntegerMatchingCriteria;
}

int UDataprepIntegerFilter::GetEqualValue() const
{
	return EqualValue;
}

int UDataprepIntegerFilter::GetFromValue() const
{
	return FromValue;
}

int UDataprepIntegerFilter::GetToValue() const
{
	return ToValue;
}

void UDataprepIntegerFilter::SetIntegerMatchingCriteria(EDataprepIntegerMatchType InIntegerMatchingCriteria)
{
	if ( IntegerMatchingCriteria != InIntegerMatchingCriteria )
	{
		Modify();
		IntegerMatchingCriteria = InIntegerMatchingCriteria;
	}
}

void UDataprepIntegerFilter::SetEqualValue(int InValue)
{
	if ( EqualValue != InValue )
	{
		Modify();
		EqualValue = InValue;
	}
}

void UDataprepIntegerFilter::SetFromValue(int InValue)
{
	if (FromValue != InValue)
	{
		Modify();
		FromValue = InValue;
	}
}

void UDataprepIntegerFilter::SetToValue(int InValue)
{
	if (ToValue != InValue)
	{
		Modify();
		ToValue = InValue;
	}
}
