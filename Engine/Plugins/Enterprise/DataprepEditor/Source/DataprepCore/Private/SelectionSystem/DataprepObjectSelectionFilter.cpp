// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepObjectSelectionFilter.h"
#include "Engine/Level.h"
#include "SelectionSystem/DataprepSelectionSystemStructs.h"

#include "Engine/Texture.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "GameFramework/Actor.h"

TArray<UObject*> UDataprepObjectSelectionFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	TArray<UObject*> FilteredObjects;

	RunFilter(Objects, FilteredObjects, nullptr);

	return FilteredObjects;
}

void UDataprepObjectSelectionFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	TArray<UObject*> FilteredObjects;
	TArray<bool> FilterResults;
	FilterResults.Init(false, InObjects.Num());
	TArrayView<bool> FilterResultsView(FilterResults);

	RunFilter(InObjects, FilteredObjects, &FilterResultsView);

	for (int Index = 0; Index < InObjects.Num(); ++Index)
	{
		FDataprepSelectionInfo& SelectionInfo = OutFilterResults[Index];
		SelectionInfo.bHasPassFilter = FilterResultsView[Index];
		SelectionInfo.bWasDataFetchedAndCached = false;
	}
}

void UDataprepObjectSelectionFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	TArray<UObject*> FilteredObjects;
	RunFilter(InObjects, FilteredObjects, &OutFilterResults);
}

void UDataprepObjectSelectionFilter::SetSelection( const FString& InTransientContentPath, const TArray<UObject*>& InSelectedObjects )
{
	NumAssets = 0;
	NumActors = 0;

	TFunction<bool( UObject* )> IsAsset = []( const UObject* InObject ) -> bool
	{
		return
			Cast<UMaterialInterface>( InObject ) ||
			Cast<UStaticMesh>( InObject ) ||
			Cast<UTexture>( InObject );
	};

	for( UObject* Obj : InSelectedObjects )
	{
		FString ObjectName;

		if( IsAsset( Obj ) )
		{
			++NumAssets;

			// Cutoff transient content folder from full path name
			FString PartialPath = Obj->GetPathName();
			PartialPath.RemoveFromStart( InTransientContentPath );

			SelectedObjectPaths.Add( PartialPath );

			ObjectName = Obj->GetName();

		}
		else if( AActor* Actor = Cast<AActor>( Obj ) )
		{
			// Get path inside level only
			for( UObject* Outer = Obj->GetOuter(); Outer; Outer = Outer->GetOuter() )
			{
				if( ULevel* LevelOuter = Cast<ULevel>( Outer ) )
				{
					++NumActors;
					SelectedObjectPaths.Add( Obj->GetPathName( Outer ) );

					ObjectName = Actor->GetActorLabel();

					break;
				}
			}
		}

		if( ObjectName.Len() > 0 )
		{
			CachedNames.Add( ObjectName );
		}
	}
}

void UDataprepObjectSelectionFilter::RunFilter(const TArrayView<UObject* const>& InputObjects, TArray<UObject*>& FilteredObjects, const TArrayView<bool>* OutFilterResults) const
{
	TArray<AActor*> InputActors;

	FilteredObjects.Empty();

	for (UObject* Object : InputObjects)
	{
		for( const FString& SelPath : SelectedObjectPaths )
		{
			if( Object->GetPathName().EndsWith( SelPath ) )
			{
				FilteredObjects.Add( Object );
				break;
			}
		}
	}

	if (OutFilterResults != nullptr)
	{
		check(OutFilterResults->Num() >= InputObjects.Num());

		for (int ObjectIndex = 0; ObjectIndex < InputObjects.Num(); ++ObjectIndex)
		{
			bool& bFiltered = (*OutFilterResults)[ObjectIndex];
			bFiltered = (INDEX_NONE != FilteredObjects.Find(InputObjects[ObjectIndex]));

			if (IsExcludingResult())
			{
				bFiltered = !bFiltered;
			}
		}
	}
}
