// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepSelectionSystemStructs.h"

#include "Async/ParallelFor.h"
#include "UObject/ObjectMacros.h"

class UDataprepFetcher;
class UDataprepFilter;

namespace DataprepSelectionSystemUtils
{
	template< class FilterClass, class FetcherClass, class FetchedDataType>
	void DoFiltering(const FilterClass& Filter, const FetcherClass& Fetcher, const TArrayView<UObject* const>& Objects, TFunctionRef<void (int32 Index, bool bFetchSucceded, const FetchedDataType& FetchedData)> FilteringFunction)
	{
		if ( Filter.IsThreadSafe() && Fetcher.IsThreadSafe() && bool( Fetcher.GetClass()->ClassFlags & CLASS_Native ) )
		{
			ParallelFor( Objects.Num(), [&Fetcher, &Objects, &FilteringFunction](int32 Index)
			{
				bool bFetchSucceded = false;
				const FetchedDataType FetchedData = Fetcher.Fetch_Implementation( Objects[Index], bFetchSucceded );
				FilteringFunction( Index, bFetchSucceded, FetchedData );
			});
		}

		else if ( Filter.IsThreadSafe() )
		{
			TArray< TPair< bool, FetchedDataType > > FetcherResults;
			FetcherResults.Reserve( Objects.Num() );
			for ( const UObject* Object : Objects )
			{
				bool bFetchSucceded = false;
				const FetchedDataType FetchedData = Fetcher.Fetch( Object, bFetchSucceded );
				FetcherResults.Emplace( bFetchSucceded, FetchedData );
			}

			ParallelFor( Objects.Num(), [&FetcherResults, &FilteringFunction](int32 Index)
			{
				const TPair< bool, FetchedDataType >& FetcherResult = FetcherResults[Index];
				FilteringFunction( Index, FetcherResult.Key, FetcherResult.Value );
			});
		}
		
		else if ( Fetcher.IsThreadSafe() && bool( Fetcher.GetClass()->ClassFlags & CLASS_Native ) )
		{
			TArray< TPair< bool, FetchedDataType > > FetcherResults;
			FetcherResults.AddZeroed( Objects.Num() );

			ParallelFor( Objects.Num(), [&Fetcher, &Objects, &FetcherResults](int32 Index)
			{
				bool bFetchSucceded = false;
				const FetchedDataType FetchedData = Fetcher.Fetch_Implementation( Objects[Index], bFetchSucceded );
				FetcherResults[Index] = TPair< bool, FetchedDataType >( bFetchSucceded, FetchedData );
			});

			for (int32 Index = 0; Index < Objects.Num(); Index++)
			{
				const TPair< bool, FetchedDataType >& FetcherResult = FetcherResults[Index];
				FilteringFunction( Index, FetcherResult.Key, FetcherResult.Value );
			}
		}

		else
		{
			for (int32 Index = 0; Index < Objects.Num(); Index++)
			{
				bool bFetchSucceded = false;
				const FetchedDataType FetchedData = Fetcher.Fetch( Objects[Index], bFetchSucceded );
				FilteringFunction( Index, bFetchSucceded, FetchedData );
			}
		}
	}

	/**
	 * A implementation for the filtering of the objects using a filter and fetcher
	 * This implementation is already multi-threaded when it's possible.
	 */
	template< class FilterClass, class FetcherClass, class FetchedDataType>
	TArray<UObject*> FilterObjects(const FilterClass& Filter, const FetcherClass& Fetcher, const TArrayView<UObject* const>& Objects)
	{
		static_assert( TIsDerivedFrom< FilterClass, UDataprepFilter >::IsDerived, "This implementation wasn't tested for a filter that isn't a child of DataprepFilter" );
		static_assert( TIsDerivedFrom< FetcherClass, UDataprepFetcher >::IsDerived, "This implementation wasn't tested for a fetcher that isn't a child of DataprepFetcher" );

		TArray<bool> FilterResultPerObject;
		FilterResultPerObject.AddZeroed( Objects.Num() );
		TAtomic<int32> ObjectThatPassedCount( 0 );

		auto UpdateFilterResult = [&Filter, &ObjectThatPassedCount, &FilterResultPerObject] (int32 Index, bool bFetchSucceded, const FetchedDataType& FetchedData )
		{
			if (bFetchSucceded)
			{
				bool bPassedFilter = Filter.Filter( FetchedData );
				FilterResultPerObject[Index] = bPassedFilter;
				if ( bPassedFilter )
				{
					ObjectThatPassedCount++;
				}
			}
			else
			{
				FilterResultPerObject[Index] = false;
			}
		};

		DoFiltering<FilterClass, FetcherClass, FetchedDataType>( Filter, Fetcher, Objects, UpdateFilterResult );
	
		const bool bIsExcludingObjectThatPassedFilter = Filter.IsExcludingResult();

		TArray< UObject* > SelectedObjects;
		int32 SelectionSize = bIsExcludingObjectThatPassedFilter ?  Objects.Num() - ObjectThatPassedCount.Load() : ObjectThatPassedCount.Load();
		SelectedObjects.Reserve( SelectionSize );
		for (int32 Index = 0; Index < Objects.Num(); Index++)
		{
			// If the filter is excluding the objects that passed the condition filter we inverse our selection criteria
			if ( FilterResultPerObject[Index] != bIsExcludingObjectThatPassedFilter )
			{
				SelectedObjects.Add( Objects[Index] );
			}
		}

		return SelectedObjects;
	}

	template<class FetchedDataType>
	static void UpdateReportedDataIfSupported(const FetchedDataType& FetchedData, FDataprepSelectionInfo& OutInfo)
	{
		if constexpr (TIsFilterDataDisplayable<FetchedDataType>::Value)
		{
			OutInfo.FetchedData.Set<FetchedDataType>( FetchedData );
			OutInfo.bWasDataFetchedAndCached = true;
		}
		else
		{
			// Do nothing since the data type is not supported
			OutInfo.bWasDataFetchedAndCached = false;
		}
	}

	/**
	 * A implementation for the filtering of the objects using a filter and fetcher
	 * This one also gather additional info on how the filtering was done
	 * This implementation is already multi-threaded when it's possible.
	 */
	template< class FilterClass, class FetcherClass, class FetchedDataType>
	void FilterAndGatherInfo(const FilterClass& Filter, const FetcherClass& Fetcher, const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults)
	{
		static_assert( TIsDerivedFrom< FilterClass, UDataprepFilter >::IsDerived, "This implementation wasn't tested for a filter that isn't a child of DataprepFilter" );
		static_assert( TIsDerivedFrom< FetcherClass, UDataprepFetcher >::IsDerived, "This implementation wasn't tested for a fetcher that isn't a child of DataprepFetcher" );

		check( InObjects.Num() == OutFilterResults.Num() );

		auto UpdateFilterResult = [&Filter, &OutFilterResults] (int32 Index, bool bFetchSucceded, const FetchedDataType& FetchedData )
		{
			FDataprepSelectionInfo& SelectionInfo = OutFilterResults[ Index ];
			SelectionInfo.bHasPassFilter = false;

			if ( bFetchSucceded )
			{
				SelectionInfo.bHasPassFilter = Filter.Filter( FetchedData );
				UpdateReportedDataIfSupported<FetchedDataType>( FetchedData, SelectionInfo );
			}
			else
			{
				SelectionInfo.bWasDataFetchedAndCached = false;
			}

			if ( Filter.IsExcludingResult() )
			{
				SelectionInfo.bHasPassFilter = !SelectionInfo.bHasPassFilter;
			}
		};

		DoFiltering<FilterClass, FetcherClass, FetchedDataType>( Filter, Fetcher, InObjects, UpdateFilterResult );
	}

	/**
	 * A implementation for the filtering of the objects using a filter and fetcher
	 * This implementation is already multi-threaded when it's possible.
	 */
	template< class FilterClass, class FetcherClass, class FetchedDataType>
	void FilterAndStoreInArrayView(const FilterClass& Filter, const FetcherClass& Fetcher, const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults)
	{
		static_assert( TIsDerivedFrom< FilterClass, UDataprepFilter >::IsDerived, "This implementation wasn't tested for a filter that isn't a child of DataprepFilter" );
		static_assert( TIsDerivedFrom< FetcherClass, UDataprepFetcher >::IsDerived, "This implementation wasn't tested for a fetcher that isn't a child of DataprepFetcher" );

		check( InObjects.Num() == OutFilterResults.Num() );

		auto UpdateFilterResult = [&Filter, &OutFilterResults] (int32 Index, bool bFetchSucceded, const FetchedDataType& FetchedData )
		{
			bool& bHasPass = OutFilterResults[ Index ];
			bHasPass = false;
			if ( bFetchSucceded )
			{
				bHasPass = Filter.Filter( FetchedData );
			}

			if ( Filter.IsExcludingResult() )
			{
				bHasPass = !bHasPass;
			}
		};

		DoFiltering<FilterClass, FetcherClass, FetchedDataType>( Filter, Fetcher, InObjects, UpdateFilterResult );
	}
}
