// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepSelectionTransform.h"

#include "DataprepCoreLogCategory.h"

#define LOCTEXT_NAMESPACE "DataprepSelectionTransform"

void UDataprepSelectionTransform::Execute(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	OnExecution(InObjects, OutObjects);
}

void UDataprepSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	UE_LOGF(LogDataprepCore, Error, "Please override UDataprepSelectionTransform::OnExecution_Implementation.");
}

FText UDataprepSelectionTransform::GetDisplayTransformName_Implementation() const
{
	return this->GetClass()->GetDisplayNameText();
}

FText UDataprepSelectionTransform::GetTooltip_Implementation() const
{
	return this->GetClass()->GetToolTipText();
}

FText UDataprepSelectionTransform::GetCategory_Implementation() const
{
	return LOCTEXT("DefaultSelectionTransformCategory", "Selection Transform");
}

FText UDataprepSelectionTransform::GetAdditionalKeyword_Implementation() const
{
	return FText();
}

void UDataprepRecursiveSelectionTransform::ApplySelectionTransform(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	UE_LOGF(LogDataprepCore, Error, "Please override UDataprepRecursiveSelectionTransform::ApplySelectionTransform.");
}

void UDataprepRecursiveSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	ApplySelectionTransform(InObjects, OutObjects);

	if (AllowRecursionLevels > 0 && OutObjects.Num() > 0)
	{
		TArray<UObject*> NewInputObjects(OutObjects);
		TArray<UObject*> NewOutputObjects;
		TSet<UObject*> ResultSet(OutObjects);

		// Sets used to check for breaking conditions
		TSet<UObject*> PrevInputSet;
		TSet<UObject*> NewOutputSet;
		
		for (int32 Level = AllowRecursionLevels; Level > 0; --Level)
		{
			ApplySelectionTransform(NewInputObjects, NewOutputObjects);

			if (NewOutputObjects.Num() == 0)
			{
				break; // No more output
			}

			NewOutputSet.Empty(NewOutputObjects.Num());
			NewOutputSet.Append(NewOutputObjects);

			if (PrevInputSet.Difference(NewOutputSet).Num() == 0)
			{
				break; // Infinite recursion
			}

			const int32 PrevResultSetNum = ResultSet.Num();

			ResultSet.Append(NewOutputObjects);

			if (PrevResultSetNum == ResultSet.Num())
			{
				break; // Nothing new
			}

			// Cache input to be able to detect infinite recursion
			PrevInputSet.Empty(NewInputObjects.Num());
			PrevInputSet.Append(NewInputObjects);

			NewInputObjects.Empty(NewOutputObjects.Num());
			NewInputObjects.Append(NewOutputObjects);
			NewOutputObjects.Empty();
		}

		OutObjects = ResultSet.Array();
	}
}

#undef LOCTEXT_NAMESPACE
