// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepStringFilter.h"

#include "DataprepCoreLogCategory.h"
#include "Misc/TVariantMeta.h"
#include "SelectionSystem/DataprepSelectionSystemUtils.h"
#include "SelectionSystem/DataprepStringFetcher.h"

bool UDataprepStringFilter::Filter(const FString& String) const
{
	TFunction<bool( const FString& )> MatchString = [this, &String]( const FString& InStringToMatch ) -> bool
	{
		switch ( StringMatchingCriteria )
		{
			case EDataprepStringMatchType::Contains:
				return String.Contains( InStringToMatch );
			case EDataprepStringMatchType::ExactMatch:
				return String.Equals( InStringToMatch );
			case EDataprepStringMatchType::MatchesWildcard:
				return String.MatchesWildcard( InStringToMatch );
			default:
				check( false );
				break;
		}

		return false;
	};

	if ( bMatchInArray )
	{
		check( UserStringArray );
		for ( const FString& Str : UserStringArray->Strings )
		{
			if ( MatchString( Str ) )
			{
				return true;
			}
		}
	}
	else
	{
		return MatchString( UserString );
	}

	return false;
}

TArray<UObject*> UDataprepStringFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	if ( StringFetcher )
	{
		return DataprepSelectionSystemUtils::FilterObjects< UDataprepStringFilter, UDataprepStringFetcher, FString >( *this, *StringFetcher, Objects );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepStringFilter::FilterObjects: There was no Fetcher" );
	return {};
}

void UDataprepStringFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	if ( StringFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndGatherInfo< UDataprepStringFilter, UDataprepStringFetcher, FString >( *this, *StringFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepStringFilter::FilterAndGatherInfo: There was no Fetcher" );
}

void UDataprepStringFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	if ( StringFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndStoreInArrayView< UDataprepStringFilter, UDataprepStringFetcher, FString >( *this, *StringFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepStringFilter::FilterAndStoreInArrayView: There was no Fetcher" );
}

TSubclassOf<UDataprepFetcher> UDataprepStringFilter::GetAcceptedFetcherClass() const
{
	return UDataprepStringFetcher::StaticClass();
}

void UDataprepStringFilter::SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass)
{
	UClass* NewFetcherClass = FetcherClass;

	if ( NewFetcherClass && NewFetcherClass->IsChildOf( GetAcceptedFetcherClass() ) )
	{
		UClass* OldFetcherClass = StringFetcher ? StringFetcher->GetClass() : nullptr;
		if ( NewFetcherClass != OldFetcherClass )
		{
			Modify();
			StringFetcher = NewObject< UDataprepStringFetcher >( this, NewFetcherClass, NAME_None, RF_Transactional );
		}
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepStringFilter::SetFetcher: The Fetcher Class is not compatible" );
	}
}

const UDataprepFetcher* UDataprepStringFilter::GetFetcherImplementation() const
{
	return StringFetcher;
}

FText UDataprepStringFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepSringFilter", "StringFilterCategory", "String");
}

EDataprepStringMatchType UDataprepStringFilter::GetStringMatchingCriteria() const
{
	return StringMatchingCriteria;
}

FString UDataprepStringFilter::GetUserString() const
{
	return UserString;
}

bool UDataprepStringFilter::GetMatchInArray() const
{
	return bMatchInArray;
}

UDataprepStringFilterMatchingArray* UDataprepStringFilter::GetStringArray()
{
	if ( !UserStringArray )
	{
		UserStringArray = NewObject< UDataprepStringFilterMatchingArray >( this, NAME_None, RF_Public );
	}
	return UserStringArray;
}

void UDataprepStringFilter::SetStringMatchingCriteria(EDataprepStringMatchType InStringMatchingCriteria)
{
	if ( StringMatchingCriteria != InStringMatchingCriteria )
	{
		Modify();
		StringMatchingCriteria = InStringMatchingCriteria;
	}
}

void UDataprepStringFilter::SetUserString(FString InUserString)
{
	if ( UserString != InUserString )
	{
		Modify();
		UserString = InUserString;
	}
}

void UDataprepStringFilter::SetMatchInArray(bool bInSet)
{
	bMatchInArray = bInSet;
}
