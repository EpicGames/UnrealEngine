// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSystem/DataprepStringsArrayFilter.h"

#include "DataprepCoreLogCategory.h"
#include "SelectionSystem/DataprepSelectionSystemUtils.h"
#include "SelectionSystem/DataprepStringFilter.h"
#include "SelectionSystem/DataprepStringsArrayFetcher.h"

bool UDataprepStringsArrayFilter::Filter(const TArray<FString>& StringArray) const
{
	bool bResult = false;

	for (int Index = 0; Index < StringArray.Num(); ++Index)
	{
		switch (StringMatchingCriteria)
		{
			case EDataprepStringMatchType::Contains:
				bResult = StringArray[Index].Contains(UserString, ESearchCase::IgnoreCase);
				break;
			case EDataprepStringMatchType::ExactMatch:
				bResult = StringArray[Index].Equals(UserString, ESearchCase::IgnoreCase);				
				break;
			case EDataprepStringMatchType::MatchesWildcard:
				bResult = StringArray[Index].MatchesWildcard(UserString, ESearchCase::IgnoreCase);
				break;
		}

		if (bResult)
		{
			break;
		}
	}

	return bResult;
}

TArray<UObject*> UDataprepStringsArrayFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	if ( StringsArrayFetcher )
	{
		return DataprepSelectionSystemUtils::FilterObjects< UDataprepStringsArrayFilter, UDataprepStringsArrayFetcher, TArray<FString> >( *this, *StringsArrayFetcher, Objects );
	}

	UE_LOGF(LogDataprepCore, Error, "UDataprepStringsArrayFilter::FilterObjects: There was no Fetcher");
	return {};
}

void UDataprepStringsArrayFilter::FilterAndGatherInfo(const TArrayView<UObject *>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	if ( StringsArrayFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndGatherInfo< UDataprepStringsArrayFilter, UDataprepStringsArrayFetcher, TArray<FString> >( *this, *StringsArrayFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepStringsArrayFilter::FilterAndGatherInfo: There was no Fetcher" );
}

void UDataprepStringsArrayFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	if ( StringsArrayFetcher )
	{
		return DataprepSelectionSystemUtils::FilterAndStoreInArrayView< UDataprepStringsArrayFilter, UDataprepStringsArrayFetcher, TArray<FString> >( *this, *StringsArrayFetcher, InObjects, OutFilterResults );
	}

	UE_LOGF( LogDataprepCore, Error, "UDataprepStringsArrayFilter::FilterAndStoreInArrayView: There was no Fetcher" );
}

TSubclassOf<UDataprepFetcher> UDataprepStringsArrayFilter::GetAcceptedFetcherClass() const
{
	return UDataprepStringsArrayFetcher::StaticClass();
}

void UDataprepStringsArrayFilter::SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass)
{
	UClass* NewFetcherClass = FetcherClass;

	if ( NewFetcherClass && NewFetcherClass->IsChildOf( GetAcceptedFetcherClass() ) )
	{
		UClass* OldFetcherClass = StringsArrayFetcher ? StringsArrayFetcher->GetClass() : nullptr;
		if ( NewFetcherClass != OldFetcherClass )
		{
			Modify();
			StringsArrayFetcher = NewObject< UDataprepStringsArrayFetcher >( this, NewFetcherClass, NAME_None, RF_Transactional );
		}
	}
	else
	{
		ensure( false );
		UE_LOGF( LogDataprepCore, Error, "UDataprepStringFilter::SetFetcher: The Fetcher Class is not compatible" );
	}
}

const UDataprepFetcher* UDataprepStringsArrayFilter::GetFetcherImplementation() const
{
	return StringsArrayFetcher;
}

FText UDataprepStringsArrayFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepSringsArrayFilter", "StringArrayFilterCategory", "String");
}

EDataprepStringMatchType UDataprepStringsArrayFilter::GetStringMatchingCriteria() const
{
	return StringMatchingCriteria;
}

FString UDataprepStringsArrayFilter::GetUserString() const
{
	return UserString;
}

bool UDataprepStringsArrayFilter::GetMatchInArray() const
{
	return bMatchInArray;
}

UDataprepStringFilterMatchingArray* UDataprepStringsArrayFilter::GetStringArray()
{
	if ( !UserStringArray )
	{
		UserStringArray = NewObject< UDataprepStringFilterMatchingArray >( this, NAME_None, RF_Transient );
	}
	return UserStringArray;
}

void UDataprepStringsArrayFilter::SetStringMatchingCriteria(EDataprepStringMatchType InStringMatchingCriteria)
{
	if (StringMatchingCriteria != InStringMatchingCriteria)
	{
		Modify();
		StringMatchingCriteria = InStringMatchingCriteria;
	}
}

void UDataprepStringsArrayFilter::SetUserString(FString InUserString)
{
	if (UserString != InUserString)
	{
		Modify();
		UserString = InUserString;
	}
}

void UDataprepStringsArrayFilter::SetMatchInArray(bool bInSet)
{
	bMatchInArray = bInSet;
}
