// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Internationalization/Text.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMaterialInterface;
namespace EMessageSeverity { enum Type : int; }

class AActor;
class IDataprepProgressReporter;
class UDataprepAssetInterface;
class UStaticMesh;
class UWorld;

namespace DataprepCorePrivateUtils
{
	/**
	 * Move an array element to another spot
	 * This operation take O(n) time. Where n is the absolute value of SourceIndex - DestinationIndex
	 * @param SourceIndex The Index of the element to move
	 * @param DestinationIndex The index of where the element will be move to
	 * @return True if the element was move
	 */
	template<class TArrayClass>
	bool MoveArrayElement(TArrayClass& Array, int32 SourceIndex, int32 DestinationIndex)
	{
		if ( Array.IsValidIndex( SourceIndex ) && Array.IsValidIndex( DestinationIndex ) && SourceIndex != DestinationIndex )
		{
			// Swap the operation until all operation are at right position. O(n) where n is Upper - Lower 

			while ( SourceIndex < DestinationIndex )
			{
				Array.Swap( SourceIndex, DestinationIndex );
				DestinationIndex--;
			}
			
			while ( DestinationIndex < SourceIndex )
			{
				Array.Swap( DestinationIndex, SourceIndex );
				DestinationIndex++;
			}

			return true;
		}

		return false;
	}

	/**
	 * Mark the input object for kill and unregister it
	 * @param Asset		The object to be deleted
	 */
	void DeleteRegisteredAsset(UObject* Asset);

	/** Returns directory where to store temporary files when running Dataprep asset */
	DATAPREPCORE_API const FString& GetRootTemporaryDir();

	/** Returns content folder where to create temporary assets when running Dataprep asset */
	DATAPREPCORE_API const FString& GetRootPackagePath();

	/**
	 * Logs messages in output log and message panel using "Dataprep Core" label
	 * @param Severity				Severity of the message to log
	 * @param Message				Message to log
	 * @param NotificationText		Text of the notification to display to alert the user 
	 * @remark Notification is only done ifNotificationText is not empty
	 */
	void LogMessage(EMessageSeverity::Type Severity, const FText& Message, const FText& NotificationText = FText());

	/**
	 * Clear 
	 */
	void ClearAssets(const TArray< TWeakObjectPtr< UObject > >& Assets);

	/** Build the render data based on the current geometry available in the static mesh */
	void BuildStaticMeshes(TSet<UStaticMesh*>& StaticMeshes, TFunction<bool(UStaticMesh*)> ProgressFunction, bool bForceBuild = false);

	/**
	 * Ensures the material is ready for rendering
	 * @remark This is only to be shared with the Dataprep editor
	 */
	DATAPREPCORE_API void CompileMaterial(UMaterialInterface* MaterialInterface);

	/**
	 * Dataprep analytics utils
	 */
	namespace Analytics
	{
		DATAPREPCORE_API void RecipeExecuted( UDataprepAssetInterface* InDataprepAsset );
		DATAPREPCORE_API void DataprepAssetCreated( UDataprepAssetInterface* InDataprepAsset );
		DATAPREPCORE_API void DataprepEditorOpened( UDataprepAssetInterface* InDataprepAsset );
		DATAPREPCORE_API void ExecuteTriggered( UDataprepAssetInterface* InDataprepAsset );
		DATAPREPCORE_API void ImportTriggered( UDataprepAssetInterface* InDataprepAsset );
		DATAPREPCORE_API void CommitTriggered( UDataprepAssetInterface* InDataprepAsset );
	}
}
