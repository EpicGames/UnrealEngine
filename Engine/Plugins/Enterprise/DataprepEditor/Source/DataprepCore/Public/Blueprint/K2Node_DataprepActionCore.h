// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "K2Node.h"

#include "K2Node_DataprepActionCore.generated.h"


class UDataprepActionAsset;
class UEdGraphPin;

UCLASS(Abstract)
class DATAPREPCORE_API UK2Node_DataprepActionCore : public UK2Node
{
	GENERATED_BODY()

public:
	UDataprepActionAsset* GetDataprepAction() const { return DataprepActionAsset; }
	void CreateDataprepActionAsset();
	void DuplicateExistingDataprepActionAsset(const UDataprepActionAsset& DataprepAction);


protected:
	UPROPERTY()
	TObjectPtr<UDataprepActionAsset> DataprepActionAsset;
};


