// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepParameterizableObject.h"

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/World.h"
#include "Types/SlateVector2.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepActionAsset.generated.h"

// Forward Declarations
class AActor;
class IDataprepLogger;
class IDataprepProgressReporter;
class UDataprepActionAsset;
class UDataprepFetcher;
class UDataprepFilter;
class UDataprepOperation;

struct FDataprepOperationContext;

template <class T>
class TSubclassOf;

namespace DataprepActionAsset
{
	/**
	 * Callback function used to confirm continuation after executing an operation or a filter
	 * @param ActionAsset			The action asset checking for continuation 
	 * @param OperationExecuted		Executed operation if not null 
	 * @param FilterExecuted		Executed filter if not null
	 */
	typedef TFunction<bool(UDataprepActionAsset* /* ActionAsset */)> FCanExecuteNextStepFunc;

	/**
	 * Callback used to report a global change to the content it is working on
	 * @param ActionAsset		The action asset reporting the change 
	 * @param bWorldChanged		Indicates changes happened in the world
	 * @param bAssetChanged		Indicates the set of assets has changed
	 * @param NewAssets			New set of assets. Only valid if bAssetChanged is true
	 */
	typedef TFunction<void(const UDataprepActionAsset* /* ActionAsset */, bool /* bWorldChanged */, bool /* bAssetChanged */, const TArray< TWeakObjectPtr<UObject> >& /* NewAssets */)> FActionsContextChangedFunc;
}

UCLASS()
class DATAPREPCORE_API UDataprepActionStep : public UObject
{
	GENERATED_BODY()

public:

	UDataprepActionStep()
		: bIsEnabled(true)
		, StepObject(nullptr)
	{}

	// Begin UObject Interface
	virtual void PostLoad() override;
	// End UObject Interface

	UPROPERTY()
	bool bIsEnabled;

	UDataprepParameterizableObject* GetStepObject()
	{
		return StepObject;
	}

	const UDataprepParameterizableObject* GetStepObject() const
	{
		return StepObject;
	}

	TSoftClassPtr<UDataprepParameterizableObject> GetPathOfStepObjectClass()
	{
		return PathOfStepObjectClass;
	}

private:
	// Only the dataprep action has the right to change the step object.
	friend class UDataprepActionAsset;

	// The actual object of the step
	UPROPERTY()
	TObjectPtr<UDataprepParameterizableObject> StepObject;

	// Will be used for future error message if the step object can't be loaded
	UPROPERTY()
	TSoftClassPtr<UDataprepParameterizableObject> PathOfStepObjectClass;

	// The operation will only be not null if the step is a operation
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the step and step type instead."))
	TObjectPtr<UDataprepOperation> Operation_DEPRECATED;

	
	// The Filter will only be not null if the step is a Filter/Selector
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the step and step type instead."))
	TObjectPtr<UDataprepFilter> Filter_DEPRECATED;
};

/** Structure to pass execution context to action */
struct FDataprepActionContext
{
	FDataprepActionContext() {}

	FDataprepActionContext& SetWorld( UWorld* InWorld )
	{ 
		WorldPtr = TWeakObjectPtr<UWorld>(InWorld);
		return *this;
	}

	FDataprepActionContext& SetAssets( TArray< TWeakObjectPtr< UObject > >& InAssets )
	{
		Assets.Empty( InAssets.Num() );
		Assets.Append( InAssets );
		return *this;
	}

	FDataprepActionContext& SetProgressReporter( const TSharedPtr< IDataprepProgressReporter >& InProgressReporter )
	{
		ProgressReporterPtr = InProgressReporter;
		return *this;
	}

	FDataprepActionContext& SetLogger( const TSharedPtr< IDataprepLogger >& InLogger )
	{
		LoggerPtr = InLogger;
		return *this;
	}

	FDataprepActionContext& SetTransientContentFolder( const FString& InTransientContentFolder )
	{
		TransientContentFolder = InTransientContentFolder;
		return *this;
	}

	FDataprepActionContext& SetCanExecuteNextStep( DataprepActionAsset::FCanExecuteNextStepFunc InCanExecuteNextStepFunc )
	{
		ContinueCallback = InCanExecuteNextStepFunc;
		return *this;
	}

	FDataprepActionContext& SetActionsContextChanged( DataprepActionAsset::FActionsContextChangedFunc InActionsContextChangedFunc )
	{
		ContextChangedCallback = InActionsContextChangedFunc;
		return *this;
	}

	/** Hold onto the world the consumer will process */
	TWeakObjectPtr< UWorld > WorldPtr;

	/** Set of assets the consumer will process */
	TSet< TWeakObjectPtr< UObject > > Assets;

	/** Path to transient content folder where were created */
	FString TransientContentFolder;

	/** Hold onto the reporter that the consumer should use to report progress */
	TSharedPtr< IDataprepProgressReporter > ProgressReporterPtr;

	/** Hold onto the logger that the consumer should use to log messages */
	TSharedPtr<  IDataprepLogger > LoggerPtr;

	/** Delegate called by an action after the execution of each step */
	DataprepActionAsset::FCanExecuteNextStepFunc ContinueCallback;

	/** Delegate called by an action if the working content has changed after the execution of an operation */
	DataprepActionAsset::FActionsContextChangedFunc ContextChangedCallback;
};

// Persists graphical state of the node associated with this action asset
UCLASS()
class UDataprepActionAppearance : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bIsExpanded;

	UPROPERTY()
	FDeprecateSlateVector2D NodeSize;

	UPROPERTY()
	int32 GroupId;

	UPROPERTY()
	bool bGroupIsEnabled;
};

class UDataprepActionAsset;
DECLARE_EVENT(UDataprepActionAsset, FOnStepsOrderChanged);
DECLARE_EVENT_OneParam(UDataprepActionAsset, FOnStepAboutToBeRemoved, UDataprepParameterizableObject* /** The step object */ );
DECLARE_EVENT_TwoParams(UDataprepParameterizableObject, FOnStepWasEdited, UDataprepParameterizableObject* /** The step object of the step (The object edited might be a subobject of it) */, struct FPropertyChangedChainEvent&)

UCLASS()
class DATAPREPCORE_API UDataprepActionAsset : public UObject
{
	GENERATED_BODY()

public:

	UDataprepActionAsset();

	virtual ~UDataprepActionAsset();

	// Begin UObject Interface
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	// End UObject Interface

	/**
	 * Execute the action on a specific set of objects
	 * @param Objects The objects on which the action will operate
	 */
	UFUNCTION(BlueprintCallable, Category = "Execution")
	void Execute(const TArray<UObject*>& InObjects);

	/**
	 * Execute the action
	 * @param InActionsContext	Shared context which the action's steps will be executed on 
	 * @param SpecificStep		Specific step to execute within the action
	 * @param bOnly				If true (default), only the specific step is executed,
	 *							otherwise the action is executed up to the specific step
	 */
	void ExecuteAction(const TSharedPtr<FDataprepActionContext>& InActionsContext, UDataprepActionStep* SpecificStep = nullptr, bool bSpecificStepOnly = true);

	/**
	 * Add a new step base on the type of step
	 * @param StepType This type of operation or fetcher(filter) we want to use
	 * @return The index of the added step or index none if the StepType is invalid
	 */
	int32 AddStep(TSubclassOf<UDataprepParameterizableObject> StepType);

	/**
	 * Add a copy of the step to the action
	 * @param ActionStep The step we want to duplicate in the action
	 * @return The index of the added step or index none if the step is invalid
	 */
	int32 AddStep(const UDataprepActionStep* ActionStep);

	/**
	 * Add a copy of each step to the action
	 * @param ActionSteps The array of steps we want to duplicate in the action
	 * @return The index of the last added step or index none if no step is invalid
	 */
	int32 AddSteps(const TArray<const UDataprepActionStep*>& ActionSteps);

	/**
	 * Add a copy of the step object to the action
	 * @param StepObject The step object we want to duplicate in the action
	 * @return The index of the added step or index none if the step is invalid
	 */
	int32 AddStep(const UDataprepParameterizableObject* StepObject);

	/**
	 * Insert a copy of the step into the action at the requested index
	 * @param Index Index to insert the action step at
	 * @param ActionStep The step to duplicate in the action
	 * @return The index of the added step or index none if the step is invalid
	 */
	bool InsertStep(const UDataprepActionStep* ActionStep, int32 Index);

	/**
	 * Insert a copy of each step into the action at the requested index
	 * @param Index Index to insert the action step at
	 * @param ActionSteps The array of steps to duplicate in the action
	 * @return The index of the added step or index none if the step is invalid
	 */
	bool InsertSteps(const TArray<const UDataprepActionStep*>& ActionSteps, int32 Index);

	/**
	 * Access to a step of the action
	 * @param Index the index of the desired step
	 * @return A pointer to the step if it exist, otherwise nullptr
	 */
	TWeakObjectPtr<UDataprepActionStep> GetStep(int32 Index);

	/**
	 * Access to a step of the action
	 * @param Index the index of the desired step
	 * @return A const pointer to the operation if it exist, otherwise nullptr
	 */
	const TWeakObjectPtr<UDataprepActionStep> GetStep(int32 Index) const;

	/**
	 * Get the number of steps of this action 
	 * @return The number of steps
	 */
	int32 GetStepsCount() const;

	/**
	 * Get enabled status of an operation
	 * @param Index The index of the operation
	 * @return True if the operation is enabled. Always return false if the operation index is invalid
	 */
	bool IsStepEnabled(int32 Index) const;

	/**
	 * Allow to set the enabled state of a step
	 * @param Index The index of the step
	 * @param bEnable The new enabled state of the step
	 */
	void EnableStep(int32 Index, bool bEnable);

	/**
	 * Move a step to another spot in the order of steps
	 * This operation take O(n) time. Where n is the absolute value of StepIndex - DestinationIndex
	 * @param StepIndex The Index of the step to move
	 * @param DestinationIndex The index of where the step will be move to
	 * @return True if the step was move
	 */
	bool MoveStep(int32 StepIndex, int32 DestinationIndex);

	/**
	 * Swap the steps
	 * @param FirstIndex The index of the first step
	 * @param SecondIndex The index of the seconds step
	 */
	bool SwapSteps(int32 FirstIndex, int32 SecondIndex);

	/**
	 * Remove a step from the action
	 * @param Index The index of the step to remove
	 * @param bDiscardParametrization If true, remove parameterization associated with action steps
	 * @return True if a step was removed
	 */
	bool RemoveStep(int32 Index, bool bDiscardParametrization = true);

	/**
	 * Remove an array of steps from the action
	 * @param Indices Array of step indices to remove
	 * @param bDiscardParametrization If true, remove parameterization associated with action steps
	 * @return True if at least one step has been removed
	 */
	bool RemoveSteps(const TArray<int32>& Indices, bool bDiscardParametrization = true);

	/**
	 * Allow an observer to be notified when the steps order changed that also include adding and removing steps
	 * @return The delegate that will be broadcasted when the steps order changed
	 */
	FOnStepsOrderChanged& GetOnStepsOrderChanged();

	/**
	 * Allow an observer to be notified when a step is about to be removed from the action
	 * @return The event that will be broadcasted
	 */
	FOnStepAboutToBeRemoved& GetOnStepAboutToBeRemoved();

	/**
	 * Allow an observer to be notified when a step was edited (the step itself or a sub object of it)
	 * @return The event that will be broadcasted
	 */
	FOnStepWasEdited& GetOnStepWasEdited();

	/**
	 * @return The appearance information associated with this action asset
	 */
	UDataprepActionAppearance* GetAppearance();

	UPROPERTY(Transient)
	bool bExecutionInterrupted;

	UPROPERTY()
	bool bIsEnabled;

	/** Getter and Setter on the UI text of the action */
	const TCHAR* GetLabel() const { return *Label; }
	void SetLabel( const TCHAR* InLabel ) { Modify(); Label = InLabel ? InLabel : TEXT(""); }

	/**
	 * Do the necessary notification so that the Dataprep system can react properly to removal of this action
	 */
	void NotifyDataprepSystemsOfRemoval();

	/**
	 * Add an operation to the action
	 * @param OperationClass The class of the operation
	 * @return The index of the added operation or index none if the class is 
	 */
	UE_DEPRECATED( 4.26, "Please use the function add step and simply pass the OperationClass" )
	int32 AddOperation(const TSubclassOf<UDataprepOperation>& OperationClass);

	/**
	 * Add a filter and setup it's fetcher
	 * @param FilterClass The type of filter we want
	 * @param FetcherClass The type of fetcher that we want. 
	 * @return The index of the added filter or index none if the classes are incompatible or invalid
	 * Note that fetcher most be compatible with the filter
	 */
	UE_DEPRECATED( 4.26, "Please use the function add step and simply pass the FetcherClass" )
	int32 AddFilterWithAFetcher(const TSubclassOf<UDataprepFilter>& FilterClass, const TSubclassOf<UDataprepFetcher>& FetcherClass);

private:
	/**
	 * Duplicate and add an asset to the Dataprep's and action's working set
	 * @param Asset			If not null, the asset will be duplicated
	 * @param AssetName		Name of the asset to create. Name collision will be performed before naming the asset
	 * @returns				The asset newly created
	 */
	UObject* OnAddAsset(const UObject* Asset, const TCHAR* AssetName);

	/**
	 * Create and add an asset to the Dataprep's and action's working set
	 * @param AssetClass	If Asset is null, an asset of the given class will be returned
	 * @param AssetName		Name of the asset to create. Name collision will be performed before naming the asset
	 * @returns				The asset newly created
	 */
	UObject* OnCreateAsset(UClass* AssetClass, const TCHAR* AssetName);

	/**
	 * Add an actor to the Dataprep's transient world and action's working set
	 * @param ActorClass	Class of the actor to create
	 * @param ActorName		Name of the actor to create. Name collision will be performed before naming the asset
	 * @returns				The actor newly created
	 */
	AActor* OnCreateActor(UClass* ActorClass, const TCHAR* ActorName);

	/**
	 * Remove an object from the Dataprep's and/or action's working set
	 * @param Object			Object to be removed from the working set 
	 * @param bLocalContext		If set to true, the object is removed from the current working set.
	 *							The object will not be accessible to any subsequent operation using the current context.
	 *							If set to false, the object is removed from the Dataprep's working set.
	 *							The object will not be accessible to any subsequent operation in the Dataprep's pipeline.
	 */
	void OnRemoveObject(UObject* Object, bool bLocalContext);

	/**
	 * Add an array of assets to the list of modified assets
	 * @param Assets		An array of assets which have been modifed
	 */
	void OnAssetsModified(TArray<UObject*> Assets);

	/**
	 * Delete an array of objects from the Dataprep's and action's working set
	 * @param Objects		The array of objects to delete
	 * @remark	The deletion of the object is deferred. However, if the object is not an asset, it is removed from
	 *			the Dataprep's transient world. If the object is an asset, it is moved to the transient package, no
	 *			action is taken to clean up any object referencing this asset.
	 * @remark	After execution, the object is not accessible by any subsequent operation in the Dataprep's pipeline.
	 */
	void OnDeleteObjects( TArray<UObject*> Objects);

	/**
	 * Executes the deletion and removal requested by an operation after its execution
	 * Notifies any observer about what has changed
	 */
	void ProcessWorkingSetChanged();

	/** Returns the outer to be used according to an asset's class */
	UObject* GetAssetOuterByClass( UClass* AssetClass );

	/** Add an asset to the execution context */
	void AddAssetToContext( UObject* NewAsset, const TCHAR* DesiredName );

	/** Creates a copy af action step, including its parameterization */
	UDataprepActionStep* DuplicateStep(const UDataprepActionStep* InActionStep);

	/** Array of operations and/or filters constituting this action */
	UPROPERTY()
	TArray<TObjectPtr<UDataprepActionStep>> Steps;

	UPROPERTY()
	TObjectPtr<UDataprepActionAppearance> Appearance;

	/** Broadcasts any change to the stack of steps */
	FOnStepsOrderChanged OnStepsOrderChanged;

	/** Broadcast when a step is about to be removed */
	FOnStepAboutToBeRemoved OnStepsAboutToBeRemoved;

	/** Broacast when a subobject of a step or a step object was edited */
	FOnStepWasEdited OnStepWasEdited;

	FDelegateHandle OnAssetDeletedHandle;

	/** Pointer to the context passed to the action for its execution */
	TSharedPtr<FDataprepActionContext> ContextPtr;

	/** Context passed to the operation for its execution */
	TSharedPtr<FDataprepOperationContext> OperationContext;

	/** Array of objects requested to be deleted by an operation */
	TArray<UObject*> ObjectsToDelete;

	/** Set of objects which have been modified during the execution of an operation */
	TSet< TWeakObjectPtr< UObject > > ModifiedAssets;

	/** Array of objects which have been added during the execution of an operation */
	TArray<UObject*> AddedObjects;

	/** Array of objects requested to be removed by an operation */
	TArray<TPair<UObject*,bool>> ObjectsToRemove;

	/** Marker to check if an operation has made any changes to the action's working set */
	bool bWorkingSetHasChanged;

	/** UI label of the action */
	UPROPERTY(EditAnywhere, Category = "Label")
	FString Label;

	/** Package which static meshes will be added to */
	TWeakObjectPtr< UPackage > PackageForStaticMesh;

	/** Package which textures will be added to */
	TWeakObjectPtr< UPackage > PackageForTexture;

	/** Package which materials will be added to */
	TWeakObjectPtr< UPackage > PackageForMaterial;

	/** Package which level sequences will be added to */
	TWeakObjectPtr< UPackage > PackageForAnimation;
};
