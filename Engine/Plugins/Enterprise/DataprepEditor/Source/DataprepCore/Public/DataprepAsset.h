// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepActionAsset.h"
#include "DataprepAssetInterface.h"
#include "DataprepAssetProducers.h"

#include "DataprepAsset.generated.h"

class UDataprepActionAsset;
class UDataprepActionStep;
class UDataprepParameterizableObject;
class UDataprepParameterization;
class UDataprepProducers;
class UEdGraphNode;

struct FDataprepActionContext;
struct FDataprepConsumerContext;
struct FDataprepProducerContext;

/**
 * A DataprepAsset is an implementation of the DataprepAssetInterface using
 * a Blueprint as the recipe pipeline. The Blueprint is composed of DataprepAction
 * nodes linearly connected.
 */
UCLASS(BlueprintType)
class DATAPREPCORE_API UDataprepAsset : public UDataprepAssetInterface
{
	GENERATED_BODY()

public:
	virtual ~UDataprepAsset() = default;

	// UObject interface
	virtual void PostLoad() override;
	virtual bool Rename(const TCHAR* NewName/* =nullptr */, UObject* NewOuter/* =nullptr */, ERenameFlags Flags/* =REN_None */) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	// End of UObject interface

	// UDataprepAssetInterface interface
	virtual void ExecuteRecipe( const TSharedPtr<FDataprepActionContext>& InActionsContext ) override;
	virtual bool HasActions() const override { return ActionAssets.Num() > 0; }
	virtual const TArray<UDataprepActionAsset*>& GetActions() const override { return ActionAssets; }
private:
	virtual TArray<UDataprepActionAsset*> GetCopyOfActions(TMap<UObject*,UObject*>& OutOriginalToCopy) const override;
	// End of UDataprepAssetInterface interface

public:
	int32 GetActionCount() const { return ActionAssets.Num(); }

	UDataprepActionAsset* GetAction(int32 Index)
	{
		return const_cast<UDataprepActionAsset*>( static_cast<const UDataprepAsset*>(this)->GetAction(Index) );
	}

	const UDataprepActionAsset* GetAction(int32 Index) const;

	int32 GetActionIndex(UDataprepActionAsset* ActionAsset) const
	{
		return ActionAssets.Find(ActionAsset);
	}

	/**
	 * Add a copy of the action to the Dataprep asset
	 * @param Action The action we want to duplicate in the Dataprep asset. Parameter can be null
	 * @return The index of the added action or index none if the action is invalid
	 * @remark If action is nullptr, a new DataprepActionAsset is simply created
	 */
	int32 AddAction(const UDataprepActionAsset* Action = nullptr);

	/**
	 * Creates an action from the array of action steps or one action per action steps
	 * then add the action(s) to the Dataprep asset
	 * @param ActionSteps The array of action steps to process
	 * @param bCreateOne Indicates if one or more action assets should be created. By default one is created
	 * @return The index of the last added action or index none if the action is invalid
	 */
	int32 AddAction(const TArray<const UDataprepActionStep*>& ActionSteps);

	/**
	 * Add the actions to the Dataprep asset
	 * @param Actions The array of action to add
	 * @param bCreateOne Indicates if one or more action assets should be created. By default one is created
	 * @return The index of the last added action or index none if the action is invalid
	 */
	int32 AddActions(const TArray<const UDataprepActionAsset*>& Actions);

	/**
	 * Creates a new action 
	 * then insert the action to the Dataprep asset at the requested index
	 * @param Index The index at which the insertion must happen
	 * @return True if the insertion is successful, false if the index is invalid
	 */
	bool InsertAction(int32 Index);
	
	/**
	 * Creates an action from the array of action steps or one action per action steps
	 * then insert the action(s) to the Dataprep asset at the requested index
	 * @param ActionSteps The array of action steps to process
	 * @param Index The index at which the insertion must happen
	 * @return True if the insertion is successful, false if the action steps or the index are invalid
	 */
	bool InsertAction(const TArray<const UDataprepActionStep*>& InActionSteps, int32 Index);

	/**
	 * Insert a copy of the action to the Dataprep asset at the requested index
	 * @param Action The action we want to duplicate in the Dataprep asset
	 * @param Index The index at which the insertion must happen
	 * @return True if the insertion is successful, false if the action or the index are invalid
	 */
	bool InsertAction(const UDataprepActionAsset* InAction, int32 Index);

	/**
	 * Insert a copy of each action into the Dataprep asset at the requested index
	 * @param Actions The array of actions to insert
	 * @param Index The index at which the insertion must happen
	 * @return True if the insertion is successful, false if the actions or the index are invalid
	 */
	bool InsertActions(const TArray<const UDataprepActionAsset*>& InActions, int32 Index);

	/**
	 * Move an action to another spot in the order of actions
	 * This operation take O(n) time. Where n is the absolute value of SourceIndex - DestinationIndex
	 * @param SourceIndex The Index of the action to move
	 * @param DestinationIndex The index of where the action will be move to
	 * @return True if the action was moved
	 */
	bool MoveAction(int32 SourceIndex, int32 DestinationIndex);

	/**
	 * Move group of actions to another spot in the order of actions
	 * @param FirstIndex The Index of the first action to move
	 * @param Count Number of actions to move
	 * @param MovePositions How many positions/offset (can be negative too) to move the actions
	 * @return True if the action group was moved
	 */
	bool MoveActions(int32 FirstIndex, int32 Count, int32 MovePositions);

	/**
	 * Swap the actions of a Dataprep asset
	 * @param FirstActionIndex The index of the first action
	 * @param SecondActionIndex The index of the second action
	 */
	bool SwapActions(int32 FirstActionIndex, int32 SecondActionIndex);

	/**
	 * Remove an action from the Dataprep asset
	 * @param Index The index of the action to remove
	 * @param bDiscardParametrization If true, remove parameterization associated with action steps
	 * @return True if the action was removed
	 */
	bool RemoveAction(int32 Index, bool bDiscardParametrization = true);

	/**
	 * Remove a set of actions from the Dataprep asset
	 * @param Index The index of the action to remove
	 * @param bDiscardParametrization If true, remove parameterization associated with action steps
	 * @return True if the action was removed
	 */
	bool RemoveActions(const TArray<int32>& Indices, bool bDiscardParametrization = true);

	/**
	 * Allow an observer to be notified of an change in the pipeline
	 * return The event that will be broadcasted when a object has receive a modification that might change the result of the pipeline
	 */
	DECLARE_EVENT_TwoParams(UDataprepAsset, FOnDataprepActionAssetChange, UObject* /*The object that was modified*/, FDataprepAssetChangeType)
	FOnDataprepActionAssetChange& GetOnActionChanged() { return OnActionChanged; }

	/**
	 * Allow an observer to be notified of the removal of some step from the asset
	 */
	DECLARE_EVENT_OneParam(UDataprepAsset, FOnStepObjectsAboutToBeRemoved, const TArrayView<UDataprepParameterizableObject*>&)
	FOnStepObjectsAboutToBeRemoved& GetOnStepObjectsAboutToBeRemoved() { return OnStepObjectsAboutToBeRemoved; }

	struct FRestrictedToActionAsset
	{
	private:
		friend UDataprepActionAsset;
		static void NotifyAssetOfTheRemovalOfSteps(UDataprepAsset& DataprepAsset, const TArrayView<UDataprepParameterizableObject*>& StepObjects);
	};

	bool CreateParameterization();

public:
	// Functions specific to the parametrization of the Dataprep asset

	/**
	 * Event to notify the ui that a Dataprep parametrization was modified
	 * This necessary as the ui for the parameterization is only updated by manual event (the ui don't pull new values each frame)
	 * Note on the objects param: The parameterized objects that should refresh their ui. If nullptr all widgets that can display some info on the parameterization should be refreshed
	 */
	DECLARE_EVENT_OneParam(UDataprepParameterization, FDataprepParameterizationStatusForObjectsChanged, const TSet<UObject*>* /** Objects */)
	FDataprepParameterizationStatusForObjectsChanged OnParameterizedObjectsStatusChanged;

	// Internal Use only (the current implementation might be subject to change)
	virtual UObject* GetParameterizationObject() override;

	void BindObjectPropertyToParameterization(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain,const FName& Name);

	bool IsObjectPropertyBinded(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain) const;

	FName GetNameOfParameterForObjectProperty(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain) const;

	void RemoveObjectPropertyFromParameterization(UDataprepParameterizableObject* Object, const TArray<struct FDataprepPropertyLink>& InPropertyChain);

	void GetExistingParameterNamesForType(FProperty* Property, bool bIsDescribingFullProperty, TSet<FString>& OutValidExistingNames, TSet<FString>& OutInvalidNames) const;

	// Internal only for now
	UDataprepParameterization* GetDataprepParameterization() { return Parameterization; }

protected:
#if WITH_EDITORONLY_DATA
	/** DEPRECATED: Pointer to data preparation pipeline blueprint previously used to process input data */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Using directly ActionAssets property instead of Blueprint to manage actions."))
	TObjectPtr<UBlueprint> DataprepRecipeBP_DEPRECATED;
	// end of temp code for nodes development

	/** DEPRECATED: List of producers referenced by the asset */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property moved to UDataprepAssetInterface as Inputs."))
	TArray< FDataprepAssetProducer > Producers_DEPRECATED;

	/** DEPRECATED: COnsumer referenced by the asset */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property moved to UDataprepAssetInterface as Output."))
	TObjectPtr<UDataprepContentConsumer> Consumer_DEPRECATED;
#endif

private:
	/** Handler for when an Dataprep asset from version prior to 4.25 is loaded */
	void OnOldAssetLoaded(UObject* Asset);

	/** Create string hash from all of the current actions's appearances */
	FString HashActionsAppearance() const;

private:
	/** DEPRECATED: Pointer to the entry node of the pipeline blueprint previously used to process input data */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Using directly ActionAssets property instead of Blueprint to manage actions."))
	TObjectPtr<UEdGraphNode> StartNode_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UDataprepParameterization> Parameterization;

	UPROPERTY()
	TArray<TObjectPtr<UDataprepActionAsset>> ActionAssets;

	FOnDataprepActionAssetChange OnActionChanged;

	FOnStepObjectsAboutToBeRemoved OnStepObjectsAboutToBeRemoved;

	/** Used on undo/redo to detect addition/removal compared to change in execution order of actions */
	FString SignatureBeforeUndoRedo;
	FString AppearanceSignatureBeforeUndoRedo;
};