// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepAssetInterface.h"

#include "DataprepAssetInstance.generated.h"

class UDataprepActionAsset;
class UDataprepAsset;
class UDataprepParameterizationInstance;

/**
 * A DataprepAssetInstance is an implementation of the DataprepAssetInterface sharing
 * its recipe pipeline with an existing Dataprep asset or another DataprepAssetInstance.
 * It has its own inputs and output. It has overrides of the externalized parameters
 * of the pipeline.
 */
UCLASS(BlueprintType)
class DATAPREPCORE_API UDataprepAssetInstance : public UDataprepAssetInterface
{
	GENERATED_BODY()

public:
	UDataprepAssetInstance()
		: Parent(nullptr)
	{
	}

	virtual ~UDataprepAssetInstance() = default;

	// UObject interface
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UDataprepAssetInterface interface
	virtual void ExecuteRecipe( const TSharedPtr<FDataprepActionContext>& InActionsContext ) override;
	virtual bool HasActions() const override { return Parent ? Parent->HasActions() : false; }
	virtual UObject* GetParameterizationObject() override;
	virtual const TArray<UDataprepActionAsset*>& GetActions() const override;
private:
	virtual TArray<UDataprepActionAsset*> GetCopyOfActions(TMap<UObject*,UObject*>& OutOriginalToCopy) const override;
	// End of UDataprepAssetInterface interface

public:
	const UDataprepAssetInterface* GetParent() const { return Parent; }
	bool SetParent(UDataprepAssetInterface* InParent, bool bNotifyChanges = true);

	/**
	 * Allow an observer to be notified when the parent asset has changed
	 * @return The delegate that will be broadcasted when the parent asset has changed
	 */
	DECLARE_EVENT(UDataprepAssetInstance, FOnDataprepAssetInstanceChanged )
	FOnDataprepAssetInstanceChanged& GetOnParentChanged() { return OnParentChanged; }

protected:
	/** Parent Dataprep asset's interface */
	UPROPERTY()
	TObjectPtr<UDataprepAssetInterface> Parent;

	UPROPERTY()
	TObjectPtr<UDataprepParameterizationInstance> Parameterization;

	/** Delegate broadcasted when the consumer or one of the producers has changed */
	FOnDataprepAssetInstanceChanged OnParentChanged;

private:
	/** Returns a pointer to the UDataprepAsset */
	UDataprepAsset* GetRootParent(UDataprepAssetInterface* InParent);

private:
	UPROPERTY()
	TArray<TObjectPtr<UDataprepActionAsset>> ActionsFromDataprepAsset;
};