// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"

#include "DataprepAssetInterface.generated.h"

class UDataprepActionAsset;
class UDataprepContentConsumer;
class UDataprepAssetProducers;

struct FDataprepActionContext;
struct FDataprepConsumerContext;
struct FDataprepProducerContext;

enum class FDataprepAssetChangeType : uint8
{
	ProducerAdded,
	ProducerRemoved,
	ProducerModified,
	ConsumerModified,
	RecipeModified,
	ActionAdded,
	ActionModified,
	ActionMoved,
	ActionRemoved,
	ActionAppearanceModified,
};

/**
 * A DataprepAssetInterface is composed of a set of producers, inputs, a consumer, output,
 * and a recipe, set of actions. The producers generate assets and populate a given world.
 * The pipeline modifies the assets and the actors of the given world. Finally, the consumer
 * converts the assets and the given world. An FBX exporter is a possible consumer.
 * This class is an abstract modeling the data preparation pipeline.
 */
UCLASS(Abstract, BlueprintType)
class DATAPREPCORE_API UDataprepRecipeInterface : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Allow an observer to be notified when when the recipe has changed
	 * @return The delegate that will be broadcasted when the recipe has changed
	 */
	DECLARE_EVENT(UDataprepRecipeInterface, FOnDataprepRecipeChanged )
	FOnDataprepRecipeChanged& GetOnChanged() { return OnChanged; }

protected:
	/** Delegate broadcasted when the recipe has changed */
	FOnDataprepRecipeChanged OnChanged;
};

UCLASS(Abstract, BlueprintType)
class DATAPREPCORE_API UDataprepAssetInterface : public UObject
{
	GENERATED_BODY()

public:
	UDataprepAssetInterface();

	// UObject interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	// End of UObject interface

	virtual ~UDataprepAssetInterface() = default;

	/** @return pointer on the recipe */
	const UDataprepRecipeInterface* GetRecipe() const { return Recipe; }

	UDataprepRecipeInterface* GetRecipe() { return Recipe; }

	/** @return pointer on the consumer */
	const UDataprepContentConsumer* GetConsumer() const { return Output; }

	UDataprepContentConsumer* GetConsumer() { return Output; }

	/**
	 * Create a new consumer of the given class
	 * @param NewConsumerClass	Class of the consumer create
	 * @param bNotifyChanges	Indicates if observers should be notified of changes 
	 * @return a pointer on the newly created consumer. Null pointer if the creation failed.
	 */
	UDataprepContentConsumer* SetConsumer( UClass* NewConsumerClass, bool bNotifyChanges = true );

	/** @return pointer on the producers */
	const UDataprepAssetProducers* GetProducers() const { return Inputs; }

	UDataprepAssetProducers* GetProducers() { return Inputs; }

	/**
	 * Run all producers of the Dataprep asset
	 * @param InContext		Context containing all the data required to properly run producers
	 * @param OutAssets		Array of assets filled up by the producers when run
	 */
	virtual TArray< TWeakObjectPtr< UObject > > RunProducers(const FDataprepProducerContext& InContext);

	/**
	 * Run the consumer associated with the Dataprep asset
	 * @param InContext		Context containing all the data required to properly run a consumer
	 * @return false if the consumer failed to execute
	 */
	virtual bool RunConsumer(const FDataprepConsumerContext& InContext);

	/**
	 * Sequentially execute all the actions held by the Dataprep asset
	 * @param InOutContext	Context containing all the data required to properly run actions
	 * @remark The context can be modified by one or more actions. The caller must check on that
	 */
	virtual void ExecuteRecipe(const TSharedPtr<FDataprepActionContext>& InActionsContext) { unimplemented(); }


	/** @return True if the Dataprep recipe is actionable */
	virtual bool HasActions() const { return false; }

	/** @return the array of actions of the recipe */
	virtual const TArray<UDataprepActionAsset*>& GetActions() const;

	/** Temporary function to allow the dataprep editor to show the parametrization */
	virtual UObject* GetParameterizationObject()
	{
		unimplemented();
		return {};
	}

	/**
	 * Allow an observer to be notified when the consumer or one of the producer has changed
	 * @return The delegate that will be broadcasted when the consumer or one of the producer has changed
	 */
	DECLARE_EVENT_OneParam(UDataprepAssetInterface, FOnDataprepAssetChanged, FDataprepAssetChangeType )
	FOnDataprepAssetChanged& GetOnChanged() { return OnChanged; }

protected:
#if WITH_EDITORONLY_DATA

	/** Recipe associated to the Dataprep asset */
	UPROPERTY()
	TObjectPtr<UDataprepRecipeInterface> Recipe;

	/** Producers associated to the Dataprep asset */
	UPROPERTY()
	TObjectPtr<UDataprepAssetProducers> Inputs;

	/** Consumer associated to the Dataprep asset */
	UPROPERTY()
	TObjectPtr<UDataprepContentConsumer> Output;
#endif

protected:

	/**
	 * Answer notification that the producer has changed
	 * @param ChangeType	Type of change made on the producers, added, removed or modified
	 * @param Index			Index of the producer which has changed. Unused.
	 */
	void OnProducersChanged(FDataprepAssetChangeType /* ChangeType */, int32 /* Index */);

	/** Answer notification that the consumer has changed */
	void OnConsumerChanged();

	/** Answer notification that the recipe has changed */
	void OnRecipeChanged();

	/** Does the actual execution of the pipeline with the specified action list */
	void ExecuteRecipe_Internal(const TSharedPtr<FDataprepActionContext>& InActionsContext, const TArray<UDataprepActionAsset*>& ActionAssets);

private:
	/** Get a copy of the Actions from the recipe with map of the original object to the copy objects */
	virtual TArray<UDataprepActionAsset*> GetCopyOfActions(TMap<UObject*,UObject*>& OutOriginalToCopy) const 
	{
		unimplemented();
		return {};
	}

	/** The DataprepAssetInstance needs to be able to access to some restricted function */
	friend class UDataprepAssetInstance;

protected:
	/** Delegate broadcasted when the consumer or one of the producers has changed */
	FOnDataprepAssetChanged OnChanged;
};