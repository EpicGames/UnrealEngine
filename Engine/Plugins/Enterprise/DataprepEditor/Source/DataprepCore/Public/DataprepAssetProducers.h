// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepAssetInterface.h"

#include "DataprepAssetProducers.generated.h"

class UDataprepContentProducer;

struct FDataprepProducerContext;

/** Structure to hold on a producer and its configuration */
USTRUCT()
struct FDataprepAssetProducer
{
	GENERATED_BODY()

	FDataprepAssetProducer() {}

	FDataprepAssetProducer(UDataprepContentProducer* InProducer, bool bInEnabled )
		: Producer(InProducer)
		, bIsEnabled( bInEnabled )
	{}

	FDataprepAssetProducer(UDataprepContentProducer* InProducer, bool bInEnabled, int32 SuperseedingIndex )
		: Producer(InProducer)
		, bIsEnabled( bInEnabled )
		, SupersededBy( SuperseedingIndex )
	{}

	UPROPERTY()
	TObjectPtr<UDataprepContentProducer> Producer = nullptr;

	UPROPERTY()
	bool bIsEnabled = true;

	UPROPERTY()
	int32 SupersededBy = INDEX_NONE;
};

/**
 * A UDataprepAssetProducers is a utility class to manage the set of producers constituting
 * the inputs of a DataprepAssetInterface. It provides a set of methods to edit the set of
 * producers and their respective configuration.
 */
UCLASS()
class DATAPREPCORE_API UDataprepAssetProducers : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UDataprepAssetProducers() = default;

	// UObject interface
	virtual void Serialize( FArchive& Ar ) override;
	virtual void PostEditUndo() override;
	// End of UObject interface

	/**
	 * Add a producer of a given class
	 * @param ProducerClass Class of the Producer to add
	 * @return true if addition was successful, false otherwise
	 */
	UDataprepContentProducer* AddProducer(UClass* ProducerClass);

	/**
	 * Add a producer of a given class but don't pop any ui
	 * @param ProducerClass Class of the Producer to add
	 * @return true if addition was successful, false otherwise
	 */
	UDataprepContentProducer* AddProducerAutomated(UClass* ProducerClass);

	/**
	 * Add a copy of a producer
	 * @param InProducer	Producer to duplicate
	 * @return true if addition was successful, false otherwise
	 */
	UDataprepContentProducer* CopyProducer( const UDataprepContentProducer* InProducer );

	/**
	 * Remove the producer at a given index
	 * @param IndexToRemove Index of the producer to remove
	 * @return true if remove was successful, false otherwise
	*/
	bool RemoveProducer( int32 IndexToRemove );

	/**
	 * Remove all producers
	 */
	void RemoveAllProducers();

	/**
	 * Enable/Disable the producer at a given index
	 * @param Index Index of the producer to update
	 */
	void EnableProducer(int32 Index, bool bValue);

	/**
	 * Enable/Disable the producer at a given index
	 * @param Index Index of the producer to update
	 */
	bool EnableAllProducers(bool bValue);

	/**
	 * Toggle the producer at a given index
	 * @param Index Index of the producer to update
	 */
	void ToggleProducer( int32 Index )
	{
		EnableProducer( Index, !IsProducerEnabled( Index ) );
	}

	int32 GetProducersCount() const { return AssetProducers.Num(); }

	/** @return pointer on producer at Index-th position in AssetProducers array. nullptr if Index is invalid */
	const UDataprepContentProducer* GetProducer(int32 Index) const;


	UDataprepContentProducer* GetProducer(int32 Index)
	{
		return const_cast<UDataprepContentProducer*>( static_cast<const UDataprepAssetProducers*>(this)->GetProducer( Index ) );
	}

	/** @return True if producer at Index-th position is enabled. Returns false if disabled or Index is invalid */
	bool IsProducerEnabled(int32 Index) const
	{ 
		return AssetProducers.IsValidIndex(Index) ? AssetProducers[Index].bIsEnabled : false;
	}

	/** @return True if producer at Index-th position is superseded by an enabled producer. Returns false if not superseded or superseder is disabled or Index is invalid */
	bool IsProducerSuperseded(int32 Index) const
	{ 
		return AssetProducers.IsValidIndex(Index) ? AssetProducers[Index].SupersededBy != INDEX_NONE && AssetProducers.IsValidIndex(AssetProducers[Index].SupersededBy) && AssetProducers[AssetProducers[Index].SupersededBy].bIsEnabled : false;
	}

	/**
	 * Run all producers of the Dataprep asset
	 * @param InContext		Context containing all the data required to properly run producers
	 * @param OutAssets		Array of assets filled up by the producers when run
	 */
	TArray< TWeakObjectPtr< UObject > > Produce( const FDataprepProducerContext& InContext );

	/**
	 * Allow an observer to be notified when when one of the producer has changed
	 * @return The delegate that will be broadcasted when one of the producer has changed
	 */
	DECLARE_EVENT_TwoParams(UDataprepAssetProducers, FOnDataprepProducersChanged, FDataprepAssetChangeType, int32 )
	FOnDataprepProducersChanged& GetOnChanged() { return OnChanged; }

protected:
#if WITH_EDITORONLY_DATA
	/** List of producers referenced by the asset */
	UPROPERTY()
	TArray< FDataprepAssetProducer > AssetProducers;

	/** List of producers referenced by the asset */
	UPROPERTY()
	int32 Padding;
#endif

protected:
	/**
	 * Answer notification that the consumer has changed
	 * @param InProducer Producer which has changed
	 */
	void OnProducerChanged( const UDataprepContentProducer* InProducer );

	/**
	 * Append an FDataprepAssetProducer
	 * @param AssetProducer		FDataprepAssetProducer to append
	 * @return Index of the added FDataprepAssetProducer. INDEX_NONE if the addition failed
	 */
	int32 AddAssetProducer( const FDataprepAssetProducer& AssetProducer );

private:

	/**
	 * Helper to check superseding on all producers after changes made on one producer
	 * @param Index Index of the producer to validate against
	 * @param bChangeAll Set to true if changes on input producer implied changes on other producers
	 */
	void ValidateProducerChanges( int32 Index, bool &bChangeAll );

	UDataprepContentProducer* AddProducer_Internal(UClass* ProducerClass, bool bIsAutomated);

	/** Delegate broadcasted when one of the producers has changed */
	FOnDataprepProducersChanged OnChanged;

	/**
	 * @remark Legacy purpose:	AddAssetProducer is used to migrate old DataprepAsset package to the new DataprepAsset.
	 *							This should be removed in a couple of releases when old DataprepAsset packages are not supported anymore.
	 */
	friend class UDataprepAsset;
};