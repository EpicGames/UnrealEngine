// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "Templates/SubclassOf.h"

#include "DataprepAssetUserData.generated.h"

class UDataprepAssetInterface;
class UDataprepEditingOperation;

/** A DataprepAssetUserData is used to mark assets or actors created through a Dataprep pipeline  */
UCLASS(BlueprintType, meta = (ScriptName = "DataprepUserData", DisplayName = "Dataprep User Data"))
class DATAPREPCORE_API UDataprepAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/** Dataprep operation which was used to generate the hosting object, if applicable */
	UPROPERTY()
	TSoftObjectPtr<UDataprepEditingOperation> DataprepOperationPtr;

	/** Dataprep asset which was used to generate the hosting object */
	UPROPERTY()
	TSoftObjectPtr<UDataprepAssetInterface> DataprepAssetPtr;
};
