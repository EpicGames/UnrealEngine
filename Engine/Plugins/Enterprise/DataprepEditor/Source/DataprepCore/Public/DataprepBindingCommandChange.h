// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Change.h"

template<class ObservedObjectType, typename DelatgateToBindType>
class TDataprepBindingCommandChange final : public FCommandChange
{
public:
	DECLARE_DELEGATE_RetVal(DelatgateToBindType&, FDelegateAccessor);
	DECLARE_DELEGATE_TwoParams(OnUndoRedoDelegateType, ObservedObjectType&, const FDelegateHandle&);

	using FDelegate = typename DelatgateToBindType::FDelegate;

	TDataprepBindingCommandChange(bool bInWasBindingAdded
		, ObservedObjectType& ObservedObject
		, FDelegate&& InBinding
		, FDelegateAccessor InDelegateAccessor
		, OnUndoRedoDelegateType&& InOnApply
		, OnUndoRedoDelegateType&& InOnRevert
		, const FDelegateHandle& InHandle
		);

	virtual void Apply(UObject* Object) override;

	virtual void Revert(UObject* Object) override;

	virtual FString ToString() const override
	{
		return TEXT("Binding Tracking");
	}

private:
	bool bWasBindingAdded;
	TWeakObjectPtr<ObservedObjectType> ObservedObjectWeakPtr;
	FDelegate Binding;
	FDelegateAccessor DelegateAccessor;
	OnUndoRedoDelegateType OnApply;
	OnUndoRedoDelegateType OnRevert;
	FDelegateHandle Handle;
};

template<class ObservedObjectType, typename DelatgateToBindType>
TDataprepBindingCommandChange<ObservedObjectType, DelatgateToBindType>::TDataprepBindingCommandChange(
	bool bInWasBindingAdded
	, ObservedObjectType& ObservedObject
	, FDelegate&& InBinding
	, FDelegateAccessor InDelegateAccessor
	, OnUndoRedoDelegateType&& InOnApply
	, OnUndoRedoDelegateType&& InOnRevert
	, const FDelegateHandle& InHandle
	)
	: bWasBindingAdded( bInWasBindingAdded )
	, ObservedObjectWeakPtr( &ObservedObject )
	, Binding( MoveTemp( InBinding ) )
	, DelegateAccessor( MoveTemp( InDelegateAccessor ) )
	, OnApply( MoveTemp( InOnApply ) )
	, OnRevert( MoveTemp( InOnRevert ) )
	, Handle( InHandle )
{
}

template<class ObservedObjectType, typename DelatgateToBindType>
void TDataprepBindingCommandChange<ObservedObjectType, DelatgateToBindType>::Apply(UObject* Object)
{
	ObservedObjectType* ObservedObject = ObservedObjectWeakPtr.Get();
	if ( ObservedObject && DelegateAccessor.IsBound() )
	{
		DelatgateToBindType& Delegate = DelegateAccessor.Execute();
		if ( bWasBindingAdded )
		{
			Handle = Delegate.Add( Binding );
		}
		else
		{
			Delegate.Remove( Handle );
		}
		OnApply.ExecuteIfBound( *ObservedObject, Handle );
	}
}

template<class ObservedObjectType, typename DelatgateToBindType>
void TDataprepBindingCommandChange<ObservedObjectType, DelatgateToBindType>::Revert(UObject* Object)
{
	ObservedObjectType* ObservedObject = ObservedObjectWeakPtr.Get();
	if ( ObservedObject && DelegateAccessor.IsBound() )
	{
		DelatgateToBindType& Delegate = DelegateAccessor.Execute();
		if ( bWasBindingAdded )
		{
			Delegate.Remove( Handle );
		}
		else
		{
			Handle = Delegate.Add( Binding );
		}
		OnRevert.ExecuteIfBound( *ObservedObject, Handle );
	}
}
