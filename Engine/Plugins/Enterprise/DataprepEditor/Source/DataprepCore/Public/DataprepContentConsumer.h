// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"

#include "Engine/AssetUserData.h"
#include "Engine/World.h"

#include "DataprepContentConsumer.generated.h"

/**
 * A UDataprepConsumerUserData is used to mark an asset or an actor to be respectively moved to a sub-folder or sub-level
 * A DataprepContentConsumer is not obligated to honor this marker 
 */
UCLASS(BlueprintType, meta = (ScriptName = "DataprepConsumerUserData", DisplayName = "Dataprep Consumer User Data"))
class DATAPREPCORE_API UDataprepConsumerUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	void AddMarker(const FString& MarkerName, const FString& Value)
	{
		Markers.FindOrAdd(MarkerName) = Value;
	}

	const FString& GetMarker(const FString& MarkerName) const;

protected:
	/** Map for Dataprep entities to store data consumable or not by a consumer */
	UPROPERTY()
	TMap<FString,FString> Markers;
};

struct FDataprepConsumerContext
{
	FDataprepConsumerContext() : bSilentMode(false) {}

	FDataprepConsumerContext& SetWorld( UWorld* InWorld )
	{ 
		WorldPtr = TWeakObjectPtr<UWorld>(InWorld);
		return *this;
	}

	FDataprepConsumerContext& SetAssets( TArray< TWeakObjectPtr< UObject > >& InAssets )
	{
		Assets.Empty(InAssets.Num());
		Assets.Append(InAssets);
		return *this;
	}

	FDataprepConsumerContext& SetProgressReporter( const TSharedPtr< IDataprepProgressReporter >& InProgressReporter )
	{
		ProgressReporterPtr = InProgressReporter;
		return *this;
	}

	FDataprepConsumerContext& SetLogger( const TSharedPtr< IDataprepLogger >& InLogger )
	{
		LoggerPtr = InLogger;
		return *this;
	}

	FDataprepConsumerContext& SetTransientContentFolder( const FString& InTransientContentFolder )
	{
		TransientContentFolder = InTransientContentFolder;
		return *this;
	}

	FDataprepConsumerContext& SetSilentMode( bool InSilentMode )
	{
		bSilentMode = InSilentMode;
		return *this;
	}

	/** Hold onto the world the consumer will process */
	TWeakObjectPtr< UWorld > WorldPtr;

	/** Array of assets the consumer will process */
	TArray< TWeakObjectPtr< UObject > > Assets;

	/** Path to transient content folder where were created */
	FString TransientContentFolder;

	/** Hold onto the reporter that the consumer should use to report progress */
	TSharedPtr< IDataprepProgressReporter > ProgressReporterPtr;

	/** Hold onto the logger that the consumer should use to log messages */
	TSharedPtr<  IDataprepLogger > LoggerPtr;

	/**
	 * Indicates the execution of the consumer must be silent
	 * This member is true when executed in headless mode
	 */
	bool bSilentMode;
};

/**
 * Abstract class providing the minimal services required for a DataprepConsumer
 * 
 * Use the SDataprepConsumerWidget class to detail the properties of this class
 */
UCLASS(Abstract, config = EditorSettings, BlueprintType)
class DATAPREPCORE_API UDataprepContentConsumer : public UObject
{
	GENERATED_BODY()

public:

	UDataprepContentConsumer();

	// UObject interface
	virtual void PostEditUndo() override;
	// End of UObject interface

	/**
	 * Successively calls Initialize, Run and Reset.
	 * @param InContext : The world the consumer must process on. This world must be assumed to be transient.
	 * @return true if the calls were successful, false otherwise
	 * @remark A copy of the incoming context is made by the consumer. The internal context is cleared by a call to Reset
	 * @remark If TargetContentFolder member is empty, it is set to the package path of the consumer
	 * @remark The consumer is expected to remove objects it has consumed from the world and/or assets' array
	 */
	bool Consume(const FDataprepConsumerContext& InContext);

	/** Name used by the UI to be displayed. */
	virtual const FText& GetLabel() const { return FText::GetEmpty(); }

	/**
	 * Text briefly describing what the consumer is doing with the world and assets it consumes.
	 * Note: This text will be used as a tooltip in the UI.
	 */
	virtual const FText& GetDescription() const { return FText::GetEmpty(); }

	/**
	 * Sets the name of the level the consumer should move objects to if applicable.
	 * @param InLevelName : New name for the consumer's level.
	 * @param OutFailureReason : String explaining reason of failure to set the level name
	 * @return true if the name has been successfully set
	 * @remark if InLevelName is empty or equal to 'current' (case insensitive), no change is made
	 */
	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	bool SetLevelName(const FString& InLevelName, FText& OutFailureReason);

	/**
	 * Sets the name of the level the consumer should move objects to if applicable.
	 * This version won't pop any ui
	 * @param InLevelName : New name for the consumer's level.
	 * @param OutFailureReason : String explaining reason of failure to set the level name
	 * @return true if the name has been successfully set
	 * @remark if InLevelName is empty or equal to 'current' (case insensitive), no change is made
	 */
	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	bool SetLevelNameAutomated(const FString& InLevelName, FText& OutFailureReason);

	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	const FString& GetLevelName() { return LevelName; }

	/**
	 * Sets the path of the package the consumer should move assets to if applicable.
	 * Generally, this package path is substituted to the temporary path the assets are in
	 * @param InTargetContentFolder : Path of the package to save any assets in
	 * @param OutFailureReason : String explaining reason of failure to set the target content folder
	 * @return true if the assignment has been successful, false otherwise
	 * @remark if InPackagePath is empty the package path of the consumer is used
	 */
	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	bool SetTargetContentFolder(const FString& InTargetContentFolder, FText& OutFailureReason);

	/**
	 * Sets the path of the package the consumer should move assets to if applicable.
	 * This version won't pop any ui
	 * Generally, this package path is substituted to the temporary path the assets are in
	 * @param InTargetContentFolder : Path of the package to save any assets in
	 * @param OutFailureReason : String explaining reason of failure to set the target content folder
	 * @return true if the assignment has been successful, false otherwise
	 * @remark if InPackagePath is empty the package path of the consumer is used
	 */
	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	bool SetTargetContentFolderAutomated(const FString& InTargetContentFolder, FText& OutFailureReason);

	UFUNCTION(BlueprintCallable, Category = DataprepConsumer)
	const FString& GetTargetContentFolder() { return TargetContentFolder; }

	/**
	 * Returns a well-formed path to use when calling CreatePackage to create the target package.
	 */
	FString GetTargetPackagePath() const;

	/**
	 * Allow an observer to be notified when one of the properties of the consumer changes
	 * @return The delegate that will be broadcasted when the consumer changed
	 */
	DECLARE_EVENT(UDataprepContentConsumer, FDataprepConsumerChanged)
	FDataprepConsumerChanged& GetOnChanged()
	{
		return OnChanged;
	}

	/** Name of marker to be used by consumers to indicate specific output for assets or actors */
	static const FString RelativeOutput;

protected:

	/**
	 * Sets the name of the level the consumer should move objects to if applicable.
	 * @param InLevelName : New name for the consumer's level.
	 * @param OutFailureReason : String explaining reason of failure to set the level name
	 * @param bIsAutomated If it is true the function shouldn't require any feedback from the user
	 * @return true if the name has been successfully set
	 * @remark if InLevelName is empty or equal to 'current' (case insensitive), no change should be made
	 */
	virtual bool SetLevelNameImplementation(const FString& InLevelName, FText& OutFailureReason, const bool bIsAutomated);

	/**
	 * Sets the path of the package the consumer should move assets to if applicable.
	 * Generally, this package path is substituted to the temporary path the assets are in
	 * @param InTargetContentFolder : Path of the package to save any assets in
	 * @param OutFailureReason : String explaining reason of failure to set the target content folder
	 * @param bIsAutomated If it is true the function shouldn't require any feedback from the user
	 * @return true if the assignment has been successful, false otherwise
	 * @remark if InPackagePath is empty the package path of the consumer is used
	 */
	virtual bool SetTargetContentFolderImplementation(const FString& InTargetContentFolder, FText& OutFailureReason, const bool bIsAutomated);

	/**
	 * Initialize the consumer to be ready for the next call to the Run method.
	 * @return true if the initialization was successful, false otherwise
	 */
	virtual bool Initialize() { return false; }

	/** Requests the consumer to perform its operation. */
	virtual bool Run() { return false; }

	/**
	 * Clean up the objects used by the consumer. This call follows a call to Run.
	 * @remark The consumer must assume that the world and assets it has not consumed are about to be deleted.
	 */
	virtual void Reset() {}

	// Start of helper functions to log messages and report progress
	void LogInfo(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogInfo( Message, *this );
		}
	}

	void LogWarning(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogWarning( Message, *this );
		}
	}

	void LogError(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogError( Message, *this );
		}
	}
	// End of helper functions to log messages

protected:
	UPROPERTY()
	FString TargetContentFolder;

	UPROPERTY()
	FString LevelName;

	/** Context which the consumer will run with */
	FDataprepConsumerContext Context;

	/** Delegate to broadcast changes to the consumer */
	FDataprepConsumerChanged OnChanged;

private:
	/** Add a UDataprepAssetUserData object to each asset's and root component's AssetUserData */
	void AddDataprepAssetUserData();
};