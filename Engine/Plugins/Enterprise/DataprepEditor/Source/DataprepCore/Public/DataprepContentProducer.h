// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"

#include "Engine/World.h"
#include "UObject/Package.h"

#include "DataprepContentProducer.generated.h"

class FFeedbackContext;

/** Structure to pass execution context to producer */
struct FDataprepProducerContext
{
	FDataprepProducerContext() {}

	/** World where the producer must add its actors */
	TWeakObjectPtr<UWorld> WorldPtr;

	/**
	* Package of the content folder under which the producer must create assets to
	* @remark: It is important to follow this rule since the Dataprep consumer assumes this is where all created assets are located
	*/
	TWeakObjectPtr<UPackage> RootPackagePtr;

	/** Reporter the producer should use to report progress */
	TSharedPtr< IDataprepProgressReporter > ProgressReporterPtr;

	/** Logger the producer should use to log messages */
	TSharedPtr<  IDataprepLogger > LoggerPtr;

	/** Helpers to set different members of the context */
	FDataprepProducerContext& SetWorld( UWorld* InWorld )
	{ 
		WorldPtr = TWeakObjectPtr< UWorld >(InWorld);
		return *this;
	}

	FDataprepProducerContext& SetRootPackage( UPackage* InRootPackage )
	{
		RootPackagePtr = TWeakObjectPtr< UPackage >(InRootPackage);
		return *this;
	}

	FDataprepProducerContext& SetProgressReporter( const TSharedPtr< IDataprepProgressReporter >& InProgressReporter )
	{
		ProgressReporterPtr = InProgressReporter;
		return *this;
	}

	FDataprepProducerContext& SetLogger( const TSharedPtr< IDataprepLogger >& InLogger )
	{
		LoggerPtr = InLogger;
		return *this;
	}
};

/**
 * Abstract class to derived from to be a producer in the Dataprep asset
 */
UCLASS(Abstract, BlueprintType)
class DATAPREPCORE_API UDataprepContentProducer : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Successively calls the Initialize, Execute and Reset methods
	 * Additionally, to avoid naming collision, rename the newly created actors to include the producer's namespace.
	 * @param InContext		Context containing all the data required to perform a run.
	 * @param OutAssets		Array of assets created by the producer
	 * @param OutReason		Text containing description of failure if the method returns false
	 *
	 * @remark The world in the context should be assumed to be transient
	 */
	bool Produce(const FDataprepProducerContext& InContext, TArray< TWeakObjectPtr< UObject > >& OutAssets);

	/** Name used by the UI to be displayed. */
	virtual const FText& GetLabel() const { return FText::GetEmpty(); }

	/**
	 * Text briefly describing what the producer is doing to populate the world and assets.
	 * Note: This text will be used as a tooltip in the UI.
	 */
	virtual const FText& GetDescription() const { return FText::GetEmpty(); }

	/**
	 * Text briefly describing what the producer is doing to populate the world and assets.
	 * Note: This text will be used as a tooltip in the UI.
	 */
	virtual FString GetNamespace() const;

	/**
	 * Allow an observer to be notified when one of the properties of the producer changes
	 * @return The delegate that will be broadcasted when the consumer changed
	 * @remark: Subclass of UDataprepContentProducer must use this mechanism to communicate changes to the owning Dataprep asset
	 */
	DECLARE_EVENT_OneParam( UDataprepContentProducer, FDataprepProducerChanged, const UDataprepContentProducer* )
	FDataprepProducerChanged& GetOnChanged()
	{
		return OnChanged;
	}

	/**
	 * A producer supersede another if its produces the same content or a super-set of the other's content
	 * @param OtherProducer : Other producer to compare to.
	 * @return true if the other producer produces the same content or a sub-set of it
	 * @remark Each sub-class must implement this method
	 */
	virtual bool Supersede(const UDataprepContentProducer* OtherProducer) const { unimplemented(); return true; }

	/**
	 * Allow a producer to pop a ui after being created.
	 * @return false to cancel the insertion of the new producer
	 */
	virtual bool CanAddToProducersArray(bool bIsAutomated)
	{
		return true;
	}

	/**
	 * Allow a producer to pop a ui after being created.
	 * @return false to cancel the insertion of the new producer
	 */
	virtual bool IsActive()
	{
		return true;
	}

	/** Returns true if the producer was cancelled during execution */
	bool IsCancelled() { return Context.ProgressReporterPtr.IsValid() ? Context.ProgressReporterPtr->IsWorkCancelled() : false; }

protected:

	/**
	 * Initialize the producer to be ready for a call to the Execute method.
	 * @param OutReason		Text containing description of failure if the method returns false
	 * @return true if the initialization was successful, false otherwise
	 */
	virtual bool Initialize() { return false; };

	/**
	 * Populates the world and fill up the array of assets.
	 * Reminder: All assets must be stored in the sub-package of the package provided in the context
	 */
	virtual bool Execute(TArray< TWeakObjectPtr< UObject > >& OutAssets) { return false; };

	/**
	 * Clean up the objects used by the producer. This call follows a call to Execute.
	 * Note: The producer must assume that the world and assets it has produced are about to be deleted.
	 */
	virtual void Reset() {};

	// Start of helper functions to log messages and report progress
	void LogInfo(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogInfo( Message, *this );
		}
	}

	void LogWarning(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogWarning( Message, *this );
		}
	}

	void LogError(const FText& Message)
	{
		if ( Context.LoggerPtr.IsValid() )
		{
			Context.LoggerPtr->LogError( Message, *this );
		}
	}
	// End of helper functions to log messages

	bool IsValid() { return Context.WorldPtr.IsValid() && Context.RootPackagePtr.IsValid(); }

protected:
	/** Context which the producer will run with */
	FDataprepProducerContext Context;

	/** Delegate to broadcast changes to the producer */
	FDataprepProducerChanged OnChanged;

	/** The feedback context in use to check for user cancellation */
	TSharedPtr<FFeedbackContext> FeedbackContext;

private:
	void Terminate()
	{
		Reset();

		// Release hold onto all context's objects
		Context.WorldPtr.Reset();
		Context.RootPackagePtr.Reset();
		Context.ProgressReporterPtr.Reset();
		Context.LoggerPtr.Reset();
	}
};