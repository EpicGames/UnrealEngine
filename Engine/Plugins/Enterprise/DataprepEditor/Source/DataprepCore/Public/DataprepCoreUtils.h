// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IDataprepLogger.h"
#include "IDataprepProgressReporter.h"

#include "FeedbackContextEditor.h"
#include "HAL/FeedbackContextAnsi.h"

#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

class UDataprepAsset;
class UDataprepActionAsset;
class UDataprepAssetInterface;
class UDataprepParameterizableObject;
struct FScopedSlowTask;
template <class t> class TSubclassOf;

class DATAPREPCORE_API FDataprepCoreUtils
{
public:

	/**
	 * Return the Dataprep asset that own the object, if the object is part of a Dataprep asset
	 * @return nullptr if the object is not a part of a Dataprep asset
	 */
	static UDataprepAsset* GetDataprepAssetOfObject(UObject* Object);

	/**
	 * Return the Dataprep action asset that own the object, if the object is part of a Dataprep action asset
	 * @return nullptr if the object is not a part of a Dataprep action asset
	 */
	static UDataprepActionAsset* GetDataprepActionAssetOf(UObject* Object);

	/** Delete the objects and do the manipulation required to safely delete the assets */
	static void PurgeObjects(TArray<UObject*> Objects);

	/**
	 * Checks if the input object is an asset even if transient or in a transient package
	 * @param Object	The object to check on
	 * @return	true if Object has the right flags or Object is of predefined classes
	 * @remark	This method is *solely* intended for objects manipulated by a Dataprep asset
	 */
	static bool IsAsset(UObject* Object);

	/** 
	 * Rename this object to a unique name, or change its outer.
	 * 
	 * @param	Object		The object to rename
	 * @param	NewName		The new name of the object, if null then NewOuter should be set
	 * @param	NewOuter	New Outer this object will be placed within, if null it will use the current outer
	 * @remark	This method is *solely* intended for objects manipulated by a Dataprep asset
	 */
	static void RenameObject(UObject* Object, const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr)
	{
		if(Object != nullptr)
		{
			Object->Rename( NewName, NewOuter, REN_NonTransactional | REN_DontCreateRedirectors );
		}
	}

	/** 
	 * Move an object to the /Game/Transient package.
	 * 
	 * @param	Object		The object to move
	 * @remark	This method is *solely* intended for objects manipulated by a Dataprep asset
	 */
	static void MoveToTransientPackage(UObject* Object)
	{
		if(Object != nullptr)
		{
			Object->Rename( nullptr, GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors );
		}
	}

	/**
	 * Execute a Dataprep asset (import, execute and commit)
	 * @param DataprepAssetInterface The Dataprep asset to execute
	 * @param Logger A optional log manager for the Dataprep producers, operations and consumer
	 * @param Reporter A optional progress reporter for the Dataprep producers, operations and consumer
	 */
	static bool ExecuteDataprep(UDataprepAssetInterface* DataprepAssetInterface, const TSharedPtr<IDataprepLogger>& Logger, const TSharedPtr<IDataprepProgressReporter>& Reporter);

	/**
	 * Check if class can be used to create a step for a Dataprep action
	 * @param StepType The class for the creation of the step
	 * @param OutValidRootClass The valid root class if found.
	 * @param OutMessageIfInvalid A message to report if the class is invalid.
	 */
	static bool IsClassValidForStepCreation(const TSubclassOf<UDataprepParameterizableObject>& StepType, UClass*& OutValidRootClass, FText& OutMessageIfInvalid);

	/**
	 * Return the type of step that action step might be (Filter or Operation)
	 */
	static UClass* GetTypeOfActionStep(const UDataprepParameterizableObject* Object);

	/**
	 * Helper function to build assets for use in the Dataprep pipeline

	 * @param	Assets					Array of weak pointer on the assets to be build
	 * @param	ProgressReporterPtr		Pointer to a IDataprepProgressReporter interface. This pointer can be invalid.
	 */
	static void BuildAssets(const TArray< TWeakObjectPtr<UObject> >& Assets, const TSharedPtr<IDataprepProgressReporter>& ProgressReporterPtr );

	/**
	 * Helper function to remove a step from an action.
	 * This function will remove the action from the Dataprep asset owning it if this is the last step.
	 * 
	 * @param	ActionAsset		Action asset to perform the operation on
	 * @param	Indices			Array of step's indices to remove.
	 * @param	ActionIndex		Set to INDEX_NONE if the action was not removed from its owning Dataprep asset. Valid index otherwise.
	 * @param	bDiscardParametrization If true, discrad parameterization associated with the action steps
	 * @return  Returns true if the removal was successful, false otherwise
	 */
	static bool RemoveSteps(UDataprepActionAsset* ActionAsset, const TArray<int32>& Indices, int32& ActionIndex, bool bDiscardParametrization = true );

	/**
	 * Helper function to remove a step from an action.
	 * This function will remove the action from the Dataprep asset owning it if this is the last step.
	 * 
	 * @param	ActionAsset		Action asset to perform the operation on
	 * @param	Index			Index of the step to remove.
	 * @param	bDiscardParametrization If true, discrad parameterization associated with the action step
	 * @param	ActionIndex		Set to INDEX_NONE if the action was not removed from its owning Dataprep asset. Valid index otherwise.
	 */
	static bool RemoveStep(UDataprepActionAsset* ActionAsset, int32 Index, int32& ActionIndex, bool bDiscardParametrization = true)
	{
		return RemoveSteps(ActionAsset, { Index }, ActionIndex, bDiscardParametrization);
	}

	class DATAPREPCORE_API FDataprepLogger : public IDataprepLogger
	{
	public:
		virtual ~FDataprepLogger() {}

		// Begin IDataprepLogger interface
		virtual void LogInfo(const FText& InLogText, const UObject& InObject) override;
		virtual void LogWarning(const FText& InLogText, const UObject& InObject) override;
		virtual void LogError(const FText& InLogText,  const UObject& InObject) override;
		// End IDataprepLogger interface

	};

	class DATAPREPCORE_API FDataprepFeedbackContext : public FFeedbackContextEditor
	{
	public:
		/** 
		 * We want to override this method in order to cache the cancel result and not clear it,
		 * so it can be checked multiple times with the correct result!
		 * (FFeedbackContextEditor::ReceivedUserCancel clears it)
		 */
		virtual bool ReceivedUserCancel() override 
		{ 
			if ( !bTaskWasCancelledCache )
			{
				bTaskWasCancelledCache = FFeedbackContextEditor::ReceivedUserCancel();
			}
			return bTaskWasCancelledCache;
		}

	private:
		bool bTaskWasCancelledCache = false;
	};

	class DATAPREPCORE_API FDataprepProgressUIReporter : public IDataprepProgressReporter
	{
	public:
		FDataprepProgressUIReporter()
			: bIsCancelled(false)
		{
		}

		FDataprepProgressUIReporter( TSharedRef<FFeedbackContext> InFeedbackContext )
			: FeedbackContext(InFeedbackContext)
			, bIsCancelled(false)
		{
		}

		virtual ~FDataprepProgressUIReporter()
		{
		}

		// Begin IDataprepProgressReporter interface
		virtual void BeginWork( const FText& InTitle, float InAmountOfWork, bool bInterruptible = true ) override;
		virtual void EndWork() override;
		virtual void ReportProgress( float Progress, const FText& InMessage ) override;
		virtual bool IsWorkCancelled() override;
		virtual FFeedbackContext* GetFeedbackContext() const override;
		// End IDataprepProgressReporter interface

	private:
		TArray< TSharedPtr< FScopedSlowTask > > ProgressTasks;
		TSharedPtr< FFeedbackContext > FeedbackContext;
		bool bIsCancelled;
	};

	class DATAPREPCORE_API FDataprepProgressTextReporter : public IDataprepProgressReporter
	{
	public:
		FDataprepProgressTextReporter()
			: TaskDepth(0)
			, FeedbackContext( new FFeedbackContextAnsi )
		{
		}

		virtual ~FDataprepProgressTextReporter()
		{
		}

		// Begin IDataprepProgressReporter interface
		virtual void BeginWork( const FText& InTitle, float InAmountOfWork, bool bInterruptible = true ) override;
		virtual void EndWork() override;
		virtual void ReportProgress( float Progress, const FText& InMessage ) override;
		virtual bool IsWorkCancelled() override;
		virtual FFeedbackContext* GetFeedbackContext() const override;

	private:
		int32 TaskDepth;
		TUniquePtr<FFeedbackContextAnsi> FeedbackContext;
	};

	/**
	 * Collect on the valid actors in the input World
	 * @param World			World to parse
	 * @param OutActors		Actors present in the world
	 */
	static void GetActorsFromWorld(const UWorld* World, TArray<UObject*>& OutActors);

	/**
	 * Collect on the valid actors in the input World
	 * @param World			World to parse
	 * @param OutActors		Actors present in the world
	 */
	static void GetActorsFromWorld(const UWorld* World, TArray<AActor*>& OutActors);

	/**
	 * Delete the specified folder including all assets and sub-folders inside it.
	 * @param BaseTemporaryPath			Root folder to start from
	 * @remark This is called at the end of the execution of a Dataprep asset to remove all
	 *		   the temporary assets and folders created during its execution.
	 */
	static void DeleteTemporaryFolders( const FString& BaseTemporaryPath );
};

