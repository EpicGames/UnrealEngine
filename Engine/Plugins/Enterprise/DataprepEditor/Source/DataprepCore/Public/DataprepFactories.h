// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LevelFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "DataprepFactories.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UDataprepAssetFactory : public UFactory
{
	GENERATED_BODY()

public:
	UDataprepAssetFactory();

	//~ Begin UFactory Interface
	virtual bool ShouldShowInNewMenu() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

UCLASS(MinimalAPI, BlueprintType)
class UDataprepAssetInstanceFactory : public UFactory
{
	GENERATED_BODY()

public:
	UDataprepAssetInstanceFactory();

	/**
	 * The parent of the of the instance to create
	 */
	UPROPERTY(EditAnywhere, Category="Settings")
	TObjectPtr<class UDataprepAsset> Parent;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};



