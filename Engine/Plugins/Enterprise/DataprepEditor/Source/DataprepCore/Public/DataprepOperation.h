// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepParameterizableObject.h"

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"

#include "DataprepOperation.generated.h"

/**
 * Delegate used by a Dataprep operation to add a copy of an asset to the Dataprep's working set
 * @return	A new asset which is either a duplicate of Asset if not null or of class AssetClass and name AssetName
 * @remark	Returns nullptr if Asset and AssetClass is null
 * @remark	If Asset is not null, it will not be added to the list of assets tracked by Dataprep. It can safely be deleted
 */
DECLARE_DELEGATE_RetVal_TwoParams(UObject* /* NewAsset */, FDataprepAddAsset, const UObject* /* Asset */, const TCHAR* /* Assetname */ )

/**
 * Delegate used by a Dataprep operation to create and add an asset to the Dataprep's working set
 * @return	A new asset which is either a duplicate of Asset if not null or of class AssetClass and name AssetName
 * @remark	Returns nullptr if Asset and AssetClass is null
 * @remark	If Asset is not null, it will not be added to the list of assets tracked by Dataprep. It can safely be deleted
 */
DECLARE_DELEGATE_RetVal_TwoParams(UObject* /* NewAsset */, FDataprepCreateAsset, UClass* /* AssetClass */, const TCHAR* /* Assetname */ )

/**
 * Delegate used by a Dataprep operation to add an actor to the Dataprep's working set
 * @return	A new Actor of class ActorClass and name ActorName added to the Dataprep's transient world
 */
DECLARE_DELEGATE_RetVal_TwoParams(class AActor* /* Actor */, FDataprepCreateActor, UClass* /* ActorClass */, const TCHAR* /* ActorName */ )

/**
 * Delegate used by a Dataprep operation to remove an object
 * If bLocalContext is true, the object is removed from the working set of the action owning the operation
 * Otherwise, the object is removed from the Dataprep's working set
 * @remark If the object is removed from the Dataprep's working set, the operation is therefore owning the object.
 */
DECLARE_DELEGATE_TwoParams(FDataprepRemoveObject, UObject* /* Object */, bool /* bLocalContext */ )

/**
 * Delegate used by a Dataprep operation to indicates an asset has been modified
 */
DECLARE_DELEGATE_OneParam(FDataprepAssetsModified, TArray<UObject*> /* Objects */ )

/**
 * Delegate used by a Dataprep operation to remove and delete an array of objects from the Dataprep's working set.
 * After execution, Object will not be accessible by subsequent operations.
 * The asset is NOT immediately deleted. This is a deferred deletion.
 */
DECLARE_DELEGATE_OneParam(FDataprepDeleteObjects, TArray<UObject*> /* Objects */ )

class DATAPREPCORE_API FDataprepOperationCategories
{
public:
	static FText ActorOperation;
	static FText AssetOperation;
	static FText MeshOperation;
	static FText ObjectOperation;
};

class FDataprepWorkReporter;
class UDataprepEditingOperation;

/** Experimental struct. Todo add struct wide comment */
USTRUCT(BlueprintType)
struct FDataprepContext
{
	GENERATED_BODY()

	/**
	 * This is the objects on which an operation can operate
	 */
	UPROPERTY( BlueprintReadOnly, Category = "Dataprep")
	TArray<TObjectPtr<UObject>> Objects;
};

/**
 * Contains all data regarding the context in which an operation will be executed
 */
struct FDataprepOperationContext
{
	// The context contains on which the operation should operate on.
	TSharedPtr<struct FDataprepContext> Context;

	/**
	 * Delegate to indicate an asset has been modified
	 */
	FDataprepAssetsModified AssetsModifiedDelegate;

	/**
	 * Delegate to duplicate and add an asset to the Dataprep content
	 * Only effective if the operation is of class UDataprepEditingOperation
	 */
	FDataprepAddAsset AddAssetDelegate;

	/**
	 * Delegate to create and add an asset to the Dataprep content
	 * Only effective if the operation is of class UDataprepEditingOperation
	 */
	FDataprepCreateAsset CreateAssetDelegate;

	/**
	 * Delegate to create and add an actor to the Dataprep content
	 * Only effective if the operation is of class UDataprepEditingOperation
	 */
	FDataprepCreateActor CreateActorDelegate;

	/**
	 * Delegate to remove an object from the Dataprep's or the action's working set
	 * Only effective if the operation is of class UDataprepEditingOperation
	 */
	FDataprepRemoveObject RemoveObjectDelegate;

	/**
	 * Delegate to remove an asset from the Dataprep content
	 * Only effective if the operation is of class UDataprepEditingOperation
	 */
	FDataprepDeleteObjects DeleteObjectsDelegate;

	// Optional Logger to capture the log produced by an operation (via the functions LogInfo, LogWarning and LogError).
	TSharedPtr<class IDataprepLogger> DataprepLogger;

	// Optional Progress Reporter to capture any progress reported by an operation.
	TSharedPtr<class IDataprepProgressReporter> DataprepProgressReporter;
};

/**
 * Base class for all Dataprep operations
 * Dataprep operations act on a set of input obejcts and can modify their properties: f.e. change materials, add metadata etc.
 */
UCLASS(Abstract, Blueprintable)
class DATAPREPCORE_API UDataprepOperation : public UDataprepParameterizableObject
{
	GENERATED_BODY()

	// User friendly interface start here ======================================================================
public:

	/**
	 * Execute the operation
	 * @param InObjects The objects that the operation will operate on
	 */
	UFUNCTION(BlueprintCallable, Category = "Execution")
	void Execute(const TArray<UObject*>& InObjects);

protected:
	
	/**
	 * This function is called when the operation is executed.
	 * If your defining your operation in Blueprint or Python this is the function to override.
	 * @param InContext The context contains the data that the operation should operate on.
	 */
	UFUNCTION(BlueprintNativeEvent)
	void OnExecution(const FDataprepContext& InContext);

	/**
	 * This function is the same has OnExcution, but it's the extension point for an operation defined in c++.
	 * It will be called on the operation execution.
	 * @param InContext The context contains the data that the operation should operate on
	 */
	virtual void OnExecution_Implementation(const FDataprepContext& InContext);

	/**
	 * Add an info to the log
	 * @param InLogText The text to add to the log
	 */
	UFUNCTION(BlueprintCallable, Category = "Log", meta = (HideSelfPin = "true"))
	void LogInfo(const FText& InLogText);

	/**
	 * Add a warning to the log
	 * @param InLogText The text to add to the log
	 */
	UFUNCTION(BlueprintCallable,  Category = "Log", meta = (HideSelfPin = "true"))
	void LogWarning(const FText& InLogText);

	/**
	 * Add Error to the log
	 * @param InLogText The text to add to the log
	 */
	UFUNCTION(BlueprintCallable,  Category = "Log", meta = (HideSelfPin = "true"))
	void LogError(const FText& InLogError);

	/**
	 * Indicates the beginning of a new work to report on
	 * @param InDescription		Text describing the work about to begin
	 * @param InAmountOfWork	Expected total amount of work
	 */
	UFUNCTION(BlueprintCallable,  Category = "Report", meta = (HideSelfPin = "true"))
	void BeginWork( const FText& InDescription, float InAmountOfWork );

	/** Indicates the end of the work */
	UFUNCTION(BlueprintCallable,  Category = "Report", meta = (HideSelfPin = "true"))
	void EndWork();

	/**
	 * Report foreseen progress on the current work
	 * @param IncrementOfWork	Amount of progress foreseen until the next call
	 * @param InMessage			Message to be displayed along side the reported progress
	 */
	UFUNCTION(BlueprintCallable,  Category = "Report", meta = (HideSelfPin = "true"))
	void ReportProgress( float IncrementOfWork, const FText& InMessage );

	/**
	 * Indicates an array of assets has changed during the operation. It is important to use this function
	 * if the modifications on the assets impact their appearance
	 * @param Assets			Array of assets which have been modified
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Operation")
	void AssetsModified( TArray<UObject*> Assets );

	/**
	 * Create a task to report progress during the execution of an operation
	 * @param InDescription		The description of the task about to be performed
	 * @param InAmountOfWork	Expected amount of work for the task
	 * @param InIncrementOfWork	Expected increment during the execution of the task
	 */
	TSharedPtr<FDataprepWorkReporter> CreateTask( const FText& InDescription, float InAmountOfWork, float InIncrementOfWork = 1.0f );

	/** Returns true if the operation was canceled during execution */
	bool IsCancelled();

	// User friendly interface end here ========================================================================

public:
	/**
	 * Prepare the operation for the execution and execute it.
	 * This allow the operation to report information such as log to the operation context
	 * @param InOperationContext This contains the data necessary for the setup of the operation and also the DataprepContext
	 */
	void ExecuteOperation(TSharedRef<FDataprepOperationContext>& InOperationContext);

	/** 
	 * Allows to change the name of the fetcher for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetDisplayOperationName() const;

	/**
	 * Allows to change the tooltip of the fetcher for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetTooltip() const;

	/**
	 * Allows to change the tooltip of the fetcher for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetCategory() const;

	/**
	 * Allows to add more keywords for when a user is searching for the fetcher in the ui.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display|Search")
	FText GetAdditionalKeyword() const;

	virtual FText GetDisplayOperationName_Implementation() const;
	virtual FText GetTooltip_Implementation() const;
	virtual FText GetCategory_Implementation() const;
	virtual FText GetAdditionalKeyword_Implementation() const;

	// Everything below is only for the Dataprep systems internal use =========================================
private:
	TSharedPtr<const FDataprepOperationContext> OperationContext;

	friend class UDataprepEditingOperation;
};


/**
 * Base class for all Dataprep editing operations
 * Dataprep editing operations act on a set of input obejcts and can modify their properties, 
 * but also can create new objects or delete existing ones (like assets and actors), based on the 
 * information they receive as an input
 */
UCLASS(Abstract, Blueprintable)
class DATAPREPCORE_API UDataprepEditingOperation : public UDataprepOperation
{
	GENERATED_BODY()

protected:
	/**
	 * DUplicate and add an asset to the Dataprep's and action's working set
	 * @param Asset			If not null, the asset will be duplicated
	 * @param AssetName		Name of the asset to create. Name collision will be checked and fixed before naming the asset
	 * @returns				The asset newly created
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	UObject* AddAsset(const UObject* Asset, const FString& AssetName );

	/**
	 * Create and add an asset to the Dataprep's and action's working set
	 * @param AssetClass	If Asset is null, an asset of the given class will be returned
	 * @param AssetName		Name of the asset to create. Name collision will be checked and fixed before naming the asset
	 * @returns				The asset newly created
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	UObject* CreateAsset(UClass* AssetClass, const FString& AssetName );

	/**
	 * Add an actor to the Dataprep's transient world and action's working set
	 * @param ActorClass	Class of the actor to create
	 * @param ActorName		Name of the actor to create. Name collision will be performed before naming the asset
	 * @returns				The actor newly created
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	AActor* CreateActor( UClass* ActorClass, const FString& ActorName );

	/**
	 * Remove an object from the Dataprep's and/or action's working set
	 * @param Object			Object to be removed from the working set 
	 * @param bLocalContext		If set to true, the object is removed from the current working set.
	 *							The object will not be accessible to any subsequent operation using the current context.
	 *							If set to false, the object is removed from the Dataprep's working set.
	 *							The object will not be accessible to any subsequent operation in the Dataprep's pipeline.
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	void RemoveObject(UObject* Object, bool bLocalContext = false);

	/**
	 * Remove an array of objects from the Dataprep's and/or action's working set
	 * @param Objects			An array of objects to be removed from the working set 
	 * @param bLocalContext		If set to true, the object is removed from the current working set.
	 *							The object will not be accessible to any subsequent operation using the current context.
	 *							If set to false, the object is removed from the Dataprep's working set.
	 *							The object will not be accessible to any subsequent operation in the Dataprep's pipeline.
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	void RemoveObjects(TArray<UObject*> Objects, bool bLocalContext = false);

	/**
	 * Delete an object from the Dataprep's working set
	 * @param Object		The object to be deleted
	 * @remark	The deletion of the object is deferred. However, if the object is not an asset, it is removed from
	 *			the Dataprep's transient world. If the object is an asset, it is moved to the transient package, no
	 *			action is taken to clean up any object referencing this asset.
	 * @remark	After execution, the object is not accessible by any subsequent operation in the Dataprep's pipeline.
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	void DeleteObject(UObject* Objects);

	/**
	 * Delete an array of objects from the Dataprep's and action's working set
	 * @param Objects		The array of objects to delete
	 * @remark	The deletion of the object is deferred. However, if the object is not an asset, it is removed from
	 *			the Dataprep's transient world. If the object is an asset, it is moved to the transient package, no
	 *			action is taken to clean up any object referencing this asset.
	 * @remark	After execution, the object is not accessible by any subsequent operation in the Dataprep's pipeline.
	 */
	UFUNCTION(BlueprintCallable,  Category = "Dataprep | Editing Operation")
	void DeleteObjects(TArray<UObject*> Objects);
};