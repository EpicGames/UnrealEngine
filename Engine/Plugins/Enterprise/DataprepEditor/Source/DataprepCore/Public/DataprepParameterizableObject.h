// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Templates/Function.h"

#include "DataprepParameterizableObject.generated.h"

struct FPropertyChangedChainEvent;

/**
 * The base class of all the object that can interact with the dataprep parameterization
 * This include all the parameterizable object and the parameterization object itself
 * Also all the object that can be place in a dataprep action derive from it
 */
UCLASS()
class DATAPREPCORE_API UDataprepParameterizableObject : public UObject
{
public:
	GENERATED_BODY()

	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/*
	 * Broadcast when a object has been post edited
	 * Note that it doesn't broadcast on the interactive events
	 */
	DECLARE_EVENT_TwoParams(UDataprepParameterizableObject, FOnPostEdit, UDataprepParameterizableObject&, FPropertyChangedChainEvent&);
	FOnPostEdit& GetOnPostEdit()
	{ 
		return OnPostEdit;
	}

private:

	FOnPostEdit OnPostEdit;
};

