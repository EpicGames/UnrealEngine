// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Blueprint.h"

#include "DataprepRecipe.generated.h"

class AActor;
class UWorld;

UCLASS(Deprecated, Blueprintable, BlueprintType, meta = (DisplayName = "DEPRECATED Data Preparation Recipe", DeprecationMessage = "No use of Blueprint with Dataprep."))
class DATAPREPCORE_API UDEPRECATED_DataprepRecipe : public UObject
{
	GENERATED_BODY()

public:
	UDEPRECATED_DataprepRecipe() {}

	/** Begin UObject override */
	virtual bool IsEditorOnly() const override { return true; }
	/** End UObject override */

public:
	/**
	 * DEPRECATED
	 * Returns all actors contained in its attached world
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Query", meta = (HideSelfPin = "true"), meta=(DeprecatedFunction, DeprecationMessage="No use of Blueprint with Dataprep."))
	virtual TArray<AActor*> GetActors() const
	{
		return TArray<AActor*>();
	}

	/**
	 * DEPRECATED
	 * Returns all valid assets contained in attached world
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Query", meta = (DeprecatedFunction, DeprecationMessage="No use of Blueprint with Dataprep.", HideSelfPin = "true"))
	virtual TArray<UObject*> GetAssets() const
	{
		return TArray<UObject*>();
	}

	/**
	 * DEPRECATED
	 * Function used to trigger the execution of the pipeline
	 * An event node associated to this function must be in the pipeline graph to run it.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(DeprecatedFunction, DeprecationMessage="No use of Blueprint with Dataprep."))
	void TriggerPipelineTraversal();
};
