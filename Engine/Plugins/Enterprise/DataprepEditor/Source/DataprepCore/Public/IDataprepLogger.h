// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FText;
class UObject;

/**
 * This is the interface that a logger must implement to work with the Dataprep core functionalities
 */
class IDataprepLogger
{
public:
	/**
	 * Records an info produced by an operation
	 * @param InLogText The text logged
	 * @param InObject The object that produced the log
	 */
	virtual void LogInfo(const FText& InLogText, const UObject& InObject) = 0;

	/**
	 * Records a warning produced by an operation
	 * @param InLogText The text logged
	 * @param InObject The object that produced the log
	 */
	virtual void LogWarning(const FText& InLogText, const UObject& InObject) = 0;

	/**
	 * Records a error produced by an operation
	 * @param InLogText The text logged
	 * @param InObject The object that produced the log
	 */
	virtual void LogError(const FText& InLogText,  const UObject& InObject) = 0;

	virtual ~IDataprepLogger() = default;
};
