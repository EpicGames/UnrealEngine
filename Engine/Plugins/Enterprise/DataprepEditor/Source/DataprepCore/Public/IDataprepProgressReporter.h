// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FText;
class UObject;
class FFeedbackContext;

/**
 * This is the interface that a progress reporter must implement to work with FDataprepWorkReporter
 * 
 */
class IDataprepProgressReporter
{
public:
	virtual ~IDataprepProgressReporter() = default;

	/**
	 * Indicates the beginning of a new work to report on
	 * @param InDescription		Text describing the work about to begin
	 * @param InAmountOfWork	Expected total amount of work
	 */
	virtual void BeginWork( const FText& InDescription, float InAmountOfWork, bool bInterruptible = true ) = 0;

	/** Indicates the end of the work */
	virtual void EndWork() = 0;

	/**
	 * Report foreseen progress on the current work
	 * @param IncrementOfWork	Amount of progress foreseen until the next call
	 * @param InMessage			Message to be displayed along side the reported progress
	 */
	virtual void ReportProgress( float IncrementOfWork, const FText& InMessage ) = 0;

	/** Returns true if the work has been cancelled */
	virtual bool IsWorkCancelled() = 0;

	/** Returns the feedback context used by this progress reporter */
	virtual FFeedbackContext* GetFeedbackContext() const = 0;

	friend class FDataprepWorkReporter;
};

/**
 * Helper class to encapsulate the IDataprepProgressReporter interface
 * 
 */
class DATAPREPCORE_API FDataprepWorkReporter
{
public:
	/**
	 * Report foreseen progress on the current task
	 * @param InDescription			Description of the task
	 * @param InAmountOfWork		Total amount of work for the task
	 * @param InIncrementOfWork		Amount of incremental work at each step within the task
	 */
	FDataprepWorkReporter( const TSharedPtr<IDataprepProgressReporter>& InReporter, const FText& InDescription, float InAmountOfWork, float InIncrementOfWork = 1.0f, bool bInterruptible = true );

	~FDataprepWorkReporter();

	/**
	 * Report foreseen incremental amount of work until next call
	 * @param InMessage				Message to be displayed along side the reported progress
	 * @param InIncrementOfWork		Amount of incremental work foreseen during the next step
	 */
	void ReportNextStep( const FText& InMessage, float InIncrementOfWork );

	/**
	 * Report foreseen default incremental amount of work until next call
	 * @param InMessage		Message to be displayed along side the reported progress
	 */
	void ReportNextStep( const FText& InMessage )
	{
		ReportNextStep( InMessage, DefaultIncrementOfWork );
	}

	bool IsWorkCancelled() const;

private:
	/** Dataprep progress reporter associated with the task */
	TSharedPtr<IDataprepProgressReporter> Reporter;

	/** Default incremental amount of work for each step constituting the task */
	float DefaultIncrementOfWork;
};