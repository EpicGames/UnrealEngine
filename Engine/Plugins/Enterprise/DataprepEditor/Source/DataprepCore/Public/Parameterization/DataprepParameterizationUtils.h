// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "DataprepParameterizationUtils.generated.h"

class IPropertyHandle;
class UDataprepAsset;
class FProperty;

enum class EParametrizationState : uint8
{
	CanBeParameterized,
	IsParameterized,
	ParentIsParameterized,
	InvalidForParameterization
};

/**
 * A small context that help when constructing the widgets for the parameterization
 */
struct FDataprepParameterizationContext
{
	TArray<FDataprepPropertyLink> PropertyChain;
	EParametrizationState State = EParametrizationState::CanBeParameterized;
};


USTRUCT()
struct FDataprepPropertyLink
{
	GENERATED_BODY()

	FDataprepPropertyLink( FProperty* InCachedProperty, const FName& InPropertyName, int32 InContenerIndex )
		: CachedProperty(InCachedProperty)
		, PropertyName( InPropertyName )
		, ContainerIndex( InContenerIndex )
	{}

	FDataprepPropertyLink() = default;
	FDataprepPropertyLink(const FDataprepPropertyLink& Other) = default;
	FDataprepPropertyLink(FDataprepPropertyLink&& Other) = default;
	FDataprepPropertyLink& operator=(const FDataprepPropertyLink& Other) = default;
	FDataprepPropertyLink& operator=(FDataprepPropertyLink&& Other) = default;

	friend bool operator==(const FDataprepPropertyLink& A,const FDataprepPropertyLink& B);

	UPROPERTY()
	TFieldPath<FProperty> CachedProperty;

	UPROPERTY()
	FName PropertyName;

	UPROPERTY()
	int32 ContainerIndex = INDEX_NONE;
};

uint32 GetTypeHash(const FDataprepPropertyLink& PropertyLink);

class DATAPREPCORE_API FDataprepParameterizationUtils
{
public:
	/**
	 * Take a property handle from the details view and generate the property for the dataprep parameterization
	 * @param PropertyHandle This is the handle from the details panel
	 * @return A non empty array if we were able make a compatible property chain
	 */
	static TArray<FDataprepPropertyLink> MakePropertyChain(TSharedPtr<IPropertyHandle> PropertyHandle);

	/**
	 * Take a Property Changed Event and extract a dataprep property link chain from it
	 * @param PropertyChangedEvent The event
	 * @return A non empty array if we were able make a compatible property chain
	 */
	static TArray<FDataprepPropertyLink>  MakePropertyChain(FPropertyChangedChainEvent& PropertyChangedEvent);

	/**
	 * Take a already existing parameterization context and create a new version including the handle.
	 */
	static FDataprepParameterizationContext CreateContext(TSharedPtr<IPropertyHandle> PropertyHandle, const FDataprepParameterizationContext& ParameterisationContext);

	/**
	 * Grab the dataprep used for the parameterization of the object.
	 * @param Object The object from which the bindings would be created
	 * @return A valid pointer if the object was valid for the parameterization
	 */
	static UDataprepAsset* GetDataprepAssetForParameterization(UObject* Object);

	/**
	 * Check if a property is chain supported by the dataprep parameterization systems
	 * Note that the cached properties of the chain must not be null
	 */
	static bool IsPropertyChainValid(const TArray<FDataprepPropertyLink>& PropertyChain);
};
