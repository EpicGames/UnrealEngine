// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFetcher.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepBoolFetcher.generated.h"

/**
 * The bool fetcher is a specialized type of fetcher for the booleans
 */
UCLASS(Abstract, Blueprintable, Meta = (DisplayName = "Condition"))
class DATAPREPCORE_API UDataprepBoolFetcher : public UDataprepFetcher
{
	GENERATED_BODY()

public:

	/**
	 * This function is called when the fetcher is executed.
	 * If your defining your fetcher in Blueprint this is the function to override.
	 * @param Object The object from which the fetcher should try to retrieve the boolean
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the boolean from the object this bool must be set to true
	 * @return The fetched bool
	 */
	UFUNCTION(BlueprintNativeEvent)
	bool Fetch(const UObject* Object, bool& bOutFetchSucceded) const;

	/**
	 * This function is the same has Fetch, but it's the extension point for an operation defined in c++.
	 * It will be called on the fetcher execution.
	 * @param Object The object from which the fetcher should try to retrieve the boolean
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the boolean from the object this bool must be set to true
	 * @return The fetched bool
	 */
	virtual bool Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
	{
		bOutFetchSucceded = false;
		return {};
	} 
};

