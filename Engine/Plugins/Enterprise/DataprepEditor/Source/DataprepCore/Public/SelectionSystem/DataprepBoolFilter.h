// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFilter.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepBoolFilter.generated.h"

class UDataprepBoolFetcher;

UCLASS()
class DATAPREPCORE_API UDataprepBoolFilter : public UDataprepFilter
{
	GENERATED_BODY()

public:
	bool Filter(const bool bResult) const;

	//~ Begin UDataprepFilter Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject *>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	virtual FText GetFilterCategoryText() const override;
	virtual TSubclassOf<UDataprepFetcher> GetAcceptedFetcherClass() const override;
	virtual void SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass) override;

private:
	virtual const UDataprepFetcher* GetFetcherImplementation() const override;
	//~ Begin UDataprepFilter Interface

	UPROPERTY()
	TObjectPtr<UDataprepBoolFetcher> BoolFetcher;
};
