// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepParameterizableObject.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepFetcher.generated.h"

/**
 * The Dataprep fetcher is a base class for Dataprep Selection system.
 * The responsibility of fetcher is return a certain type of data for a object. (Look at DataprepStringFetcher for a example)
 * This abstract base class exist currently for the discovery process, some compile time validation and to propagate UDataprepParameterizableObject
 */
UCLASS(Abstract)
class DATAPREPCORE_API UDataprepFetcher : public UDataprepParameterizableObject
{
	GENERATED_BODY()

public:
	/**
	 * Is this fetcher safe to use in a multi thread execution?
	 */
	virtual bool IsThreadSafe() const { return false; }

	/** 
	 * Allows to change the name of the fetcher for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetDisplayFetcherName() const;

	/**
	 * The name displayed on node title.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Display")
	FText GetNodeDisplayFetcherName() const;

	/**
	 * Allows to change the tooltip of the fetcher for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetTooltipText() const;

	/**
	 * Allows to add more keywords for when a user is searching for the fetcher in the ui.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display|Search")
	FText GetAdditionalKeyword() const;


	// The Native way to override the blueprint native events above
	virtual FText GetDisplayFetcherName_Implementation() const;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	virtual FText GetTooltipText_Implementation() const;
	virtual FText GetAdditionalKeyword_Implementation() const;
};


