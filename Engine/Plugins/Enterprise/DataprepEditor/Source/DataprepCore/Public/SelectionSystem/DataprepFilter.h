// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepParameterizableObject.h"

#include "SelectionSystem/DataprepFetcher.h"

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepFilter.generated.h"

struct FDataprepSelectionInfo;

/**
 * The Dataprep Filter a base class for the Dataprep selection system
 * It's main responsibility is to filter a array of object and to return the selected objects
 */
UCLASS(Abstract, BlueprintType)
class DATAPREPCORE_API UDataprepFilter : public UDataprepParameterizableObject
{
	GENERATED_BODY()

public:

	// Begin UObject Interface
	virtual void PostCDOContruct() override;
	// End UObject Interface

	/**
	 * Take an array of objects and return the objects that pass the filter
	 * @param Objects The object to filter
	 * @return The object that passed the filtering
	 */
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const { return {}; }

	/**
	 * Take an array of object and output the result into the arrays
	 * @param InObjects The object to filter
	 * @param OutFilterResult Will put true at the same index of the object if it passed the filter
	 * @param OutFetchedSucces Will put true at the same index of the object if the fetcher fetched a value
	 * @param OutFetchedLabel Will put a display of information contextual to the filter/fetcher
	 */
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const {};

	/**
	 * Take an array of object and output the result into the result array
	 * @param InObjects The object to filter
	 * @param OutFilterResult Will put true at the same index of the object if it has passed the filter
	 */
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const {};

	/**
	 * Is this filter safe to use in a multi thread execution?
	 */
	virtual bool IsThreadSafe() const { return false; } 

	/**
	 * Return the selector category for this filter
	 * Imagine the category as the following: Select by|Your filter category| data fetched by the fetcher
	 * Here a full example: Select by|String with|Object Name
	 */
	virtual FText GetFilterCategoryText() const { return {}; }

	/** 
	 * Return the type of fetcher associated with this filter
	 */
	virtual TSubclassOf<UDataprepFetcher> GetAcceptedFetcherClass() const 
	{
		// must be override
		unimplemented();
		return {};
	}

	/**
	 * Set a new fetcher for this filter
	 * Note: This should only set a new fetcher if the fetcher is a subclass of the result of GetAcceptedFetcherClass and if it's not the same class as the current one
	 */
	virtual void SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass) { unimplemented(); /** must be override */ }

	UFUNCTION(BlueprintCallable, Category="Filter")
	UDataprepFetcher* GetFetcher()
	{
		return const_cast<UDataprepFetcher*>( static_cast<const UDataprepFilter*>( this )->GetFetcherImplementation() );
	}

	const UDataprepFetcher* GetFetcher() const
	{
		return GetFetcherImplementation();
	}

	/**
	 * Allow the filter to exclude only the element that would normally pass the filter
	 * @param bIsExcludingResult Should the filter be a excluding filter
	 */
	void SetIsExcludingResult(bool bInIsExcludingResult)
	{
		Modify();
		bIsExcludingResult = bInIsExcludingResult;
		FProperty* Property = StaticClass()->FindPropertyByName( GET_MEMBER_NAME_CHECKED( UDataprepFilter, bIsExcludingResult ) );
		check(Property);
		FEditPropertyChain EditChain;
		EditChain.AddHead( Property );
		EditChain.SetActivePropertyNode( Property );
		FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
		FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
		PostEditChangeChainProperty( EditChangeChainEvent );
	}

	/**
	 * Is this filter a excluding filter.
	 */
	bool IsExcludingResult() const
	{
		return bIsExcludingResult;
	}

	/**
	 * Return the type of filter to use for a fetcher
	 */
	static UClass* GetFilterTypeForFetcherType(UClass* FetcherClass);

private:

	virtual const UDataprepFetcher* GetFetcherImplementation() const
	{
		// must be override
		unimplemented();
		return {};
	}

	// Is this filter an excluding filter (a filter that produces the inverse of its normal output)
	UPROPERTY(EditAnywhere, Category="Filter")
	bool bIsExcludingResult = false;

	static TMap<UClass*, UClass*> FetcherClassToFilterClass;
};

/**
 * The Dataprep Filter a base class for the Dataprep selection system
 * It's main responsibility is to filter a array of object and to return the selected objects
 * 
 * This version of the filter does not support fetchers, but instead works (filters) directly on the input objects
 */
UCLASS(Abstract, BlueprintType)
class DATAPREPCORE_API UDataprepFilterNoFetcher : public UDataprepParameterizableObject
{
	GENERATED_BODY()

public:

	/**
	 * Take an array of objects and return the objects that pass the filter
	 * @param Objects The object to filter
	 * @return The object that passed the filtering
	 */
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const { return {}; }

	/**
	 * Take an array of object and output the result into the arrays
	 * @param InObjects The object to filter
	 * @param OutFilterResult Will put true at the same index of the object if it passed the filter
	 * @param OutFetchedSucces Will put true at the same index of the object if the fetcher fetched a value
	 * @param OutFetchedLabel Will put a display of information contextual to the filter/fetcher
	 */
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const {};

	/**
	 * Take an array of object and output the result into the result array
	 * @param InObjects The object to filter
	 * @param OutFilterResult Will put true at the same index of the object if it has passed the filter
	 */
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const {};

	/**
	 * Is this filter safe to use in a multi thread execution?
	 */
	virtual bool IsThreadSafe() const { return false; } 

	/**
	 * Return the selector category for this filter
	 * Imagine the category as the following: Select by|Your filter category| data fetched by the fetcher
	 * Here a full example: Select by|String with|Object Name
	 */
	virtual FText GetFilterCategoryText() const { return {}; }

	/** 
	 * Allows to change the name of the filter for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetDisplayFilterName() const;

	/**
	 * The name displayed on node title.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Display")
	FText GetNodeDisplayFilterName() const;

	/**
	 * Allows to change the tooltip of the filter for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetTooltipText() const;

	/**
	 * Allows to add more keywords for when a user is searching for the filter in the ui.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display|Search")
	FText GetAdditionalKeyword() const;

	// The Native way to override the blueprint native events above
	virtual FText GetDisplayFilterName_Implementation() const;
	virtual FText GetTooltipText_Implementation() const;
	virtual FText GetAdditionalKeyword_Implementation() const;

	/**
	 * Allow the filter to exclude only the element that would normally pass the filter
	 * @param bIsExcludingResult Should the filter be a excluding filter
	 */
	void SetIsExcludingResult(bool bInIsExcludingResult)
	{
		Modify();
		bIsExcludingResult = bInIsExcludingResult;
		FProperty* Property = StaticClass()->FindPropertyByName( GET_MEMBER_NAME_CHECKED( UDataprepFilterNoFetcher, bIsExcludingResult ) );
		check(Property);
		FEditPropertyChain EditChain;
		EditChain.AddHead( Property );
		EditChain.SetActivePropertyNode( Property );
		FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
		FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
		PostEditChangeChainProperty( EditChangeChainEvent );
	}

	/**
	 * Is this filter a excluding filter.
	 */
	bool IsExcludingResult() const
	{
		return bIsExcludingResult;
	}

private:
	// Is this filter an excluding filter (a filter that produces the inverse of its normal output)
	UPROPERTY(EditAnywhere, Category="Filter")
	bool bIsExcludingResult = false;
};
