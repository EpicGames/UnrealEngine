// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFilter.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepFloatFilter.generated.h"

class UDataprepFloatFetcher;

UENUM()
enum class EDataprepFloatMatchType : uint8
{
	LessThan,
	GreatherThan,
	IsNearlyEqual
};

UCLASS()
class DATAPREPCORE_API UDataprepFloatFilter : public UDataprepFilter
{
	GENERATED_BODY()

public:
	bool Filter(float Float) const;

	// Begin UDataprepFilter Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	virtual FText GetFilterCategoryText() const override;
	virtual TSubclassOf<UDataprepFetcher> GetAcceptedFetcherClass() const override;
	virtual void SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass) override;
private:
	virtual const UDataprepFetcher* GetFetcherImplementation() const override;
	// End UDataprepFilter Interface

public:

	EDataprepFloatMatchType GetFloatMatchingCriteria() const;
	float GetEqualValue() const;
	float GetTolerance() const;

	void SetFloatMatchingCriteria(EDataprepFloatMatchType FloatMatchingCriteria);
	void SetEqualValue(float EqualValue);
	void SetTolerance(float Tolerance);

private:
	// The source of float selected by the user
	UPROPERTY()
	TObjectPtr<UDataprepFloatFetcher> FloatFetcher;

	// The matching criteria used when checking if a fetched value can pass the filter
	UPROPERTY(EditAnywhere, Category = "Filter")
	EDataprepFloatMatchType FloatMatchingCriteria;

	// The value to use for the comparison against the fetched value
	UPROPERTY(EditAnywhere, Category = "Filter")
	float EqualValue;

	// The value used for the tolerance when doing a nearly equal
	UPROPERTY(EditAnywhere, Category = "Filter")
	float Tolerance = KINDA_SMALL_NUMBER;
};

