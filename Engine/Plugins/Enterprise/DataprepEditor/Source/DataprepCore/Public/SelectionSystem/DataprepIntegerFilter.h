// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFilter.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepIntegerFilter.generated.h"

class UDataprepIntegerFetcher;

UENUM()
enum class EDataprepIntegerMatchType : uint8
{
	LessThan,
	GreatherThan,
	IsEqual,
	InBetween
};

UCLASS()
class DATAPREPCORE_API UDataprepIntegerFilter : public UDataprepFilter
{
	GENERATED_BODY()

public:
	bool Filter(int Integer) const;

	//~ Begin UDataprepFilter Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	virtual FText GetFilterCategoryText() const override;
	virtual TSubclassOf<UDataprepFetcher> GetAcceptedFetcherClass() const override;
	virtual void SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass) override;

private:
	virtual const UDataprepFetcher* GetFetcherImplementation() const override;
	//~ Begin UDataprepFilter Interface

public:
	EDataprepIntegerMatchType GetIntegerMatchingCriteria() const;
	void SetIntegerMatchingCriteria(EDataprepIntegerMatchType InIntegerMatchingCriteria);

	int GetEqualValue() const;
	void SetEqualValue(int InValue);

	int GetFromValue() const;
	void SetFromValue(int InValue);

	int GetToValue() const;
	void SetToValue(int InValue);

private:
	// The source of int selected by the user
	UPROPERTY()
	TObjectPtr<UDataprepIntegerFetcher> IntFetcher;

	// The matching criteria used when checking if a fetched value can pass the filter
	UPROPERTY(EditAnywhere, Category = Filter)
	EDataprepIntegerMatchType IntegerMatchingCriteria;

	// The value to use when doing the comparison against the fetched value
	UPROPERTY(EditAnywhere, Category = Filter)
	int EqualValue;

	// A value used for the in-between check
	UPROPERTY(EditAnywhere, Category = Filter)
	int FromValue;

	// A value used for the in-between check
	UPROPERTY(EditAnywhere, Category = Filter)
	int ToValue;
};
