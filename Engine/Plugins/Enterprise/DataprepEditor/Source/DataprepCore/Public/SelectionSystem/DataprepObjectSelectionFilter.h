// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFilter.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepObjectSelectionFilter.generated.h"

UCLASS(BlueprintType, NotBlueprintable, HideCategories = (Filter), Meta = (Hidden, DisplayName="Selection", ToolTip = "Filter selected objects"))
class DATAPREPCORE_API UDataprepObjectSelectionFilter : public UDataprepFilterNoFetcher
{
	GENERATED_BODY()

public:
	UDataprepObjectSelectionFilter() : NumAssets( 0 ), NumActors( 0 ) {}

	//~ Begin UDataprepFilterNoFetcher Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	//~ End UDataprepFilterNoFetcher Interface

	void SetSelection( const FString& InTransientContentPath, const TArray<UObject*>& InSelectedObjects );

	const TArray< FString >& GetCachedNames() const { return CachedNames; }

	int32 GetNumAssets() const { return NumAssets; }
	int32 GetNumActors() const { return NumActors; }

private:
	void RunFilter(const TArrayView<UObject* const>& InputObjects, TArray<UObject*>& FilteredObjects, const TArrayView<bool>* OutFilterResults) const;

private:
	// Partial paths of objects.
	// For assets, the transient part is cutoff (as it changes between runs), 
	// for actors, the path up to level outer is cutoff (in case level name changes between runs)
	UPROPERTY()
	TArray< FString > SelectedObjectPaths;

	UPROPERTY()
	int32 NumAssets;

	UPROPERTY()
	int32 NumActors;

	// Cache some of the names of selected objects, up to some limit, for preview purpose
	UPROPERTY()
	TArray< FString > CachedNames;
};
