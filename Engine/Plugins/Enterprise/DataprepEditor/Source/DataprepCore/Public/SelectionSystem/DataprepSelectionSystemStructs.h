// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TVariant.h"

// Those template must be keep in sync with the variant. Those are the data types that the filter preview can display the data in the columns (scene preview and asset preview)
template <class T>
struct TIsFilterDataDisplayable
{
	enum { Value = false };
};

template <>
struct TIsFilterDataDisplayable<int32>
{
	enum { Value = true };
};

template <>
struct TIsFilterDataDisplayable<float>
{
	enum { Value = true };
};

template <>
struct TIsFilterDataDisplayable<FString>
{
	enum { Value = true };
};

/**
 * This type is the variant used by the dataprep filter preview system to store a fetched data and then displaying it latter.
 * if you update this, add the new type to the list above. 
 * Try to compile it. It should fail on a couple of functors. 
 * Update those and keep the functions explicit so that the compilation will fail when adding a type to the variant.
 * This way the programmer will be force to acknowledge how those functors should behave on the new type
 */
using FFilterVariantData = TVariant< FEmptyVariantState, int32, float, FString >;

/**
 * A struct to cache the result of a filter with some additional info
 */
struct DATAPREPCORE_API FDataprepSelectionInfo
{
	FFilterVariantData FetchedData;
	uint8 bHasPassFilter:1;
	uint8 bWasDataFetchedAndCached:1;
};
