// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepParameterizableObject.h"

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"

#include "DataprepSelectionTransform.generated.h"

UCLASS(Abstract, Blueprintable)
class DATAPREPCORE_API UDataprepSelectionTransform : public UDataprepParameterizableObject
{
	GENERATED_BODY()

public:

	/**
	 * Execute the transform
	 * @param InObjects Input objects the transform will operate on
	 * @param OutObjects Resulting objects after the transform was executed
	 */
	UFUNCTION(BlueprintCallable, Category = "Execution")
	void Execute(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects);

protected:
	
	/**
	 * This function is called when the transform is executed.
	 * If your defining your transform in Blueprint or Python this is the function to override.
	 * @param InObjects Input objects the transform will operate on
	 * @param OutObjects Resulting objects after the transform was executed
	 */
	UFUNCTION(BlueprintNativeEvent)
	void OnExecution(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects);

	/**
	 * This function is the same has OnExcution, but it's the extension point for an transform defined in c++.
	 * It will be called on the transform execution.
	 * @param InObjects Input objects the transform will operate on
	 * @param OutObjects Resulting objects after the transform was executed
	 */
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects);

public:
	/** 
	 * Allows to change the name of the transform for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetDisplayTransformName() const;

	/**
	 * Allows to change the tooltip of the transform for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetTooltip() const;

	/**
	 * Allows to change the category of the transform for the ui if needed.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display")
	FText GetCategory() const;

	/**
	 * Allows to add more keywords for when a user is searching for the fetcher in the ui.
	 */
	UFUNCTION(BlueprintNativeEvent,  Category = "Display|Search")
	FText GetAdditionalKeyword() const;

	virtual FText GetDisplayTransformName_Implementation() const;
	virtual FText GetTooltip_Implementation() const;
	virtual FText GetCategory_Implementation() const;
	virtual FText GetAdditionalKeyword_Implementation() const;

	// Specifies if input objects that have matching type can be added to the result
	UPROPERTY(EditAnywhere, Category = SelectionTransform)
	bool bOutputCanIncludeInput = true;
};

UCLASS(Abstract, Blueprintable)
class DATAPREPCORE_API UDataprepRecursiveSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

public:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;

	virtual void ApplySelectionTransform(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects);

private:
	// How many times is it allowed to apply another transform on the result
	UPROPERTY(EditAnywhere, Category = SelectionTransform)
	int32 AllowRecursionLevels = -1;
};
