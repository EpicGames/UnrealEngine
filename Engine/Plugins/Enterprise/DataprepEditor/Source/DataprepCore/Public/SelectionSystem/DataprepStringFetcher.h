// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFetcher.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepStringFetcher.generated.h"

/**
 * The string fetcher is a specialized type of fetcher for the string
 */
UCLASS(Abstract, Blueprintable, Meta = (DisplayName = "String"))
class DATAPREPCORE_API UDataprepStringFetcher : public UDataprepFetcher
{
	GENERATED_BODY()

public:

	/**
	 * This function is called when the fetcher is executed.
	 * If your defining your fetcher in Blueprint this is the function to override.
	 * @param Object The object from which the fetcher should try to retrieve the string
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the string from the object this bool must be set to true
	 * @return The fetched string
	 */
	UFUNCTION(BlueprintNativeEvent)
	FString Fetch(const UObject* Object, bool& bOutFetchSucceded) const;

	/**
	 * This function is the same has Fetch, but it's the extension point for an operation defined in c++.
	 * It will be called on the fetcher execution.
	 * @param Object The object from which the fetcher should try to retrieve the string
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the string from the object this bool must be set to true
	 * @return The fetched string
	 */
	virtual FString Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
	{
		bOutFetchSucceded = false;
		return {};
	} 
};
