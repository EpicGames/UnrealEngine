// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFilter.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepStringFilter.generated.h"

class UDataprepStringFetcher;

UENUM()
enum class EDataprepStringMatchType : uint8
{
	Contains,
	MatchesWildcard,
	ExactMatch
};

UCLASS()
class DATAPREPCORE_API UDataprepStringFilterMatchingArray : public UDataprepParameterizableObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Filter)
	TArray<FString> Strings;

	// UI state that needs to be serialized along with string values
	UPROPERTY()
	bool bExpanded = true;
};

UCLASS()
class DATAPREPCORE_API UDataprepStringFilter : public UDataprepFilter
{
	GENERATED_BODY()

public:

	bool Filter(const FString& String) const;

	//~ Begin UDataprepFilter Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	virtual FText GetFilterCategoryText() const override;
	virtual TSubclassOf<UDataprepFetcher> GetAcceptedFetcherClass() const override;
	virtual void SetFetcher(const TSubclassOf<UDataprepFetcher>& FetcherClass) override;

private:
	virtual const UDataprepFetcher* GetFetcherImplementation() const override;
	//~ Begin UDataprepFilter Interface

public:
	EDataprepStringMatchType GetStringMatchingCriteria() const;
	FString GetUserString() const;
	bool GetMatchInArray() const;
	UDataprepStringFilterMatchingArray* GetStringArray();

	void SetStringMatchingCriteria(EDataprepStringMatchType StringMatchingCriteria);
	void SetUserString(FString UserString);
	void SetMatchInArray(bool bInSet);

private:
	// The matching criteria used when checking if a fetched value can pass the filter
	UPROPERTY(EditAnywhere, Category = Filter)
	EDataprepStringMatchType StringMatchingCriteria;

	// The string used when doing the comparison
	UPROPERTY(EditAnywhere, Category = Filter)
	FString UserString;

	UPROPERTY(EditAnywhere, Category = Filter)
	TObjectPtr<UDataprepStringFilterMatchingArray> UserStringArray;

	UPROPERTY(EditAnywhere, Category = Filter)
	bool bMatchInArray;

	// The source of string selected by the user
	UPROPERTY()
	TObjectPtr<UDataprepStringFetcher> StringFetcher;
};
