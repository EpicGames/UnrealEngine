// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFetcher.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepStringsArrayFetcher.generated.h"

UCLASS(Abstract, Blueprintable, Meta = (DisplayName = "Strings Array"))
class DATAPREPCORE_API UDataprepStringsArrayFetcher : public UDataprepFetcher
{
	GENERATED_BODY()

public:

	/**
	 * This function is called when the fetcher is executed.
	 * If you are defining your fetcher in Blueprint this is the function to override.
	 * @param Object The object from which the fetcher should try to retrieve the string
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the string from the object this bool must be set to true
	 * @return The fetched string
	 */
	UFUNCTION(BlueprintNativeEvent)
	TArray<FString> Fetch(const UObject* Object, bool& bOutFetchSucceded) const;

	/**
	 * This function is the same has Fetch, but it's the extension point for an operation defined in c++.
	 * It will be called on the fetcher execution.
	 * @param Object The object from which the fetcher should try to retrieve the string
	 * @param bOutFetchSucceded If the fetcher managed to retrieve the string from the object this bool must be set to true
	 * @return The fetched string
	 */
	virtual TArray<FString> Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
	{
		bOutFetchSucceded = false;
		return {};
	} 
};
