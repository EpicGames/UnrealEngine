// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataprepEditor : ModuleRules
	{
		public DataprepEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AdvancedPreviewScene",
					"ApplicationCore",
					"AssetRegistry",
					"AssetDefinition",
					"AssetTools",
					"BlueprintGraph",
					"Blutility",
					"Core",
					"CoreUObject",
					"ContentBrowser",
					"DataprepCore",
					"DesktopPlatform",
					"EditorFramework",
					"EditorStyle",
					"EditorWidgets",
					"EditorWidgets",
					"Engine",
					"GraphEditor",
					"InputCore",
					"Kismet",
					"KismetCompiler",
					"KismetWidgets",
					"Landscape",
					"MainFrame",
					"MeshDescription",
					"MeshUtilities",
					"MeshUtilitiesCommon",
					"MessageLog",
					"Projects",
					"PropertyEditor",
					"RHI",
					"SceneOutliner",
					"Slate",
					"SlateCore",
					"StatsViewer",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd",
					"SubobjectEditor",
					"SubobjectDataInterface",
					"RenderCore",
				}
			);

			PrivateIncludePaths.AddRange(
				new string[] {
					"DataprepCore/Private/Shared",
				});
		}
	}
}
