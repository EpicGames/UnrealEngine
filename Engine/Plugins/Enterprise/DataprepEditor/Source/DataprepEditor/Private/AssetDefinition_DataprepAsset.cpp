// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DataprepAsset.h"

#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditor.h"
#include "DataprepEditorModule.h"
#include "DataprepFactories.h"

#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#include "Algo/AnyOf.h"
#include "Algo/AllOf.h"

FText UAssetDefinition_DataprepAssetInterface::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions_DataprepAssetInterface",  "Name", "Dataprep Interface" );
}

TSoftClassPtr<> UAssetDefinition_DataprepAssetInterface::GetAssetClass() const
{
	return UDataprepAssetInterface::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DataprepAssetInterface::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(NSLOCTEXT("AssetTypeActions_DataprepAsset", "Name", "Dataprep Asset")) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_DataprepAssetInterface::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UDataprepAssetInterface* DataprepAssetInterface : OpenArgs.LoadObjects<UDataprepAssetInterface>())
	{
		TSharedRef<FDataprepEditor> NewDataprepEditor(new FDataprepEditor());
		NewDataprepEditor->InitDataprepEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, DataprepAssetInterface);
	}

	return EAssetCommandResult::Handled;
}

FText UAssetDefinition_DataprepAsset::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions_DataprepAsset", "Name", "Dataprep Asset");
}

TSoftClassPtr<UObject> UAssetDefinition_DataprepAsset::GetAssetClass() const
{
	return UDataprepAsset::StaticClass();
}

FText UAssetDefinition_DataprepAssetInstance::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions_DataprepAssetInstance", "Name", "Dataprep Asset Instance");
}

TSoftClassPtr<UObject> UAssetDefinition_DataprepAssetInstance::GetAssetClass() const
{
	return UDataprepAssetInstance::StaticClass();
}

namespace MenuExtension_DataprepAsset
{
	static void CreateInstance(const FToolMenuContext& MenuContext)
	{
		// Code is inspired from MenuExtension_MaterialInterface::ExecuteNewMIC
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UDataprepAssetInterface>(
			CBContext->LoadSelectedObjects<UDataprepAssetInterface>(), UDataprepAssetInstance::StaticClass(), TEXT("_Inst"), [](UDataprepAssetInterface* SourceObject)
			{
				UDataprepAssetInstanceFactory* Factory = nullptr;
				if(UDataprepAsset* DataprepAsset = Cast<UDataprepAsset>(SourceObject))
				{
					Factory = NewObject<UDataprepAssetInstanceFactory>();
					Factory->Parent = DataprepAsset;
				}
				return Factory;
			}
		);
	}

	static void ExecuteDataprepAssets(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		for (UDataprepAssetInterface* DataprepAssetInterface : CBContext->LoadSelectedObjects<UDataprepAssetInterface>())
		{
			// Nothing to do if the Dataprep asset does not have any inputs
			if(DataprepAssetInterface->GetProducers()->GetProducersCount() > 0)
			{
				FDataprepCoreUtils::ExecuteDataprep( DataprepAssetInterface
					, MakeShared<FDataprepCoreUtils::FDataprepLogger>()
					, MakeShared<FDataprepCoreUtils::FDataprepProgressUIReporter>() );
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDataprepAssetInterface::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					// #ueent_remark: An instance of an instance is not supported for 4.24.
					// Do not expose 'Create Instance' menu entry if at least one Dataprep asset is an instance
					bool bContainsAnInstance  = Algo::AnyOf(CBContext->SelectedAssets, [](const FAssetData& InAssetData)
					{
						return InAssetData.GetClass() == UDataprepAssetInstance::StaticClass();
					});

					// Disable execute if any of the input assets has no consumer
					bool bCanExecute  = Algo::AllOf(CBContext->SelectedAssets, [](const FAssetData& InAssetData)
					{
						if (UDataprepAssetInterface* DataprepAsset = Cast<UDataprepAssetInterface>(InAssetData.GetAsset()))
						{
							return DataprepAsset->GetConsumer() != nullptr;
						}
						return true;
					});
					
					if (!bContainsAnInstance)
					{
						const TAttribute<FText> Label = NSLOCTEXT("AssetTypeActions_DataprepAssetInterface", "CreateInstance", "Create Instance");
						const TAttribute<FText> ToolTip = NSLOCTEXT("AssetTypeActions_DataprepAssetInterface", "CreateInstanceTooltip", "Creates a parameterized Dataprep asset using this Dataprep asset as a base.");
						const FSlateIcon Icon = FSlateIcon();
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&CreateInstance);

						InSection.AddMenuEntry("DataprepAssetInterface_CreateInstance", Label, ToolTip, Icon, UIAction);
					}

					const TAttribute<FText> Label = NSLOCTEXT("AssetTypeActions_DataprepAssetInterface", "RunAsset", "Execute");
					const TAttribute<FText> ToolTip = NSLOCTEXT("AssetTypeActions_DataprepAssetInterface", "RunAssetTooltip", "Runs the Dataprep asset's producers, execute its recipe, finally runs the consumer");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteDataprepAssets);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([bCanExecute](const FToolMenuContext& Context){ return bCanExecute; });

					InSection.AddMenuEntry("DataprepAssetInterface_Execute", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}
