// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetTypeActions_Base.h"

#include "AssetDefinition_DataprepAsset.generated.h"


UCLASS()
class UAssetDefinition_DataprepAssetInterface : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 0, 0); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_DataprepAsset : public UAssetDefinition_DataprepAssetInterface
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 255, 0); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_DataprepAssetInstance : public UAssetDefinition_DataprepAssetInterface
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 0, 255); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	// UAssetDefinition End
};
