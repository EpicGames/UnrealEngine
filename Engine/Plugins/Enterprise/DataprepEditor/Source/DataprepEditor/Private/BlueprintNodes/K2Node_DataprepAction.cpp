// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodes/K2Node_DataprepAction.h"

// Dataprep includes
#include "DataprepAsset.h"

// Engine includes
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "KismetCompiler.h"

#define LOCTEXT_NAMESPACE "DataprepActionK2Node"

const FText DataprepActionCategory( LOCTEXT( "DataprepActionK2NodeCategory", "Dataprep Action" ) );

const FName UDEPRECATED_K2Node_DataprepAction::ThenPinName( TEXT("") );
const FName UDEPRECATED_K2Node_DataprepAction::InObjectsPinName( TEXT("Objects") );

UDEPRECATED_K2Node_DataprepAction::UDEPRECATED_K2Node_DataprepAction()
{
	bCanRenameNode = true;
	ActionTitle = LOCTEXT("DefaultNodeTitle", "New Action").ToString();
}

void UDEPRECATED_K2Node_DataprepAction::PostReconstructNode()
{
	Super::PostReconstructNode();

	// Move content of deprecated DataprepAction_DEPRECATED to the new DataprepActionAsset.
	if(DataprepAction_DEPRECATED)
	{
		DataprepActionAsset = DataprepAction_DEPRECATED;
		DataprepAction_DEPRECATED = nullptr;
	}

	if(DataprepActionAsset)
	{
		DataprepActionAsset->SetLabel( *ActionTitle );
	}
}


void UDEPRECATED_K2Node_DataprepAction::AllocateDefaultPins()
{
	// Inputs
	CreatePin( EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute );
	
	// Hack don't put the objects pin when then node is in a Dataprep asset
	bool bIsInADataprepAsset = false;
	UObject* Outer = GetOuter();
	while ( Outer )
	{
		if ( Outer->GetClass() == UDataprepAsset::StaticClass() )
		{
			bIsInADataprepAsset = true;
			break;
		}
 
		Outer = Outer->GetOuter();
	}
 
	if ( !bIsInADataprepAsset )
	{
		UEdGraphNode::FCreatePinParams ArrayPinParams;
		ArrayPinParams.ContainerType = EPinContainerType::Array;
		ArrayPinParams.bIsReference = true;
		UEdGraphPin* ObjectsPin = CreatePin( EGPD_Input, UEdGraphSchema_K2::PC_Object, InObjectsPinName, ArrayPinParams );
		ObjectsPin->PinType.PinSubCategoryObject = UObject::StaticClass();
	}
 
	// Outputs
	CreatePin( EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then );
 
	PreloadObject( DataprepActionAsset );
 
	Super::AllocateDefaultPins();
}

FText UDEPRECATED_K2Node_DataprepAction::GetMenuCategory() const
{
	return DataprepActionCategory;
}

FLinearColor UDEPRECATED_K2Node_DataprepAction::GetNodeTitleColor() const
{
	return FLinearColor(0.0036765f, 0.3864294f, 0.2501584f);
}

FText UDEPRECATED_K2Node_DataprepAction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString( ActionTitle );
}

void UDEPRECATED_K2Node_DataprepAction::OnRenameNode(const FString& NewName)
{
	ActionTitle = NewName;
	if ( DataprepActionAsset )
	{
		DataprepActionAsset->SetLabel( *NewName );
	}
}

void UDEPRECATED_K2Node_DataprepAction::DestroyNode()
{
	if ( DataprepActionAsset )
	{
		Modify();

		DataprepActionAsset->NotifyDataprepSystemsOfRemoval();

		// Force the transaction system to restore the action
		DataprepActionAsset = nullptr;
	}
 
	Super::DestroyNode();
}

TSharedPtr<class INameValidatorInterface> UDEPRECATED_K2Node_DataprepAction::MakeNameValidator() const
{
	// The name doesn't matter
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

void UDEPRECATED_K2Node_DataprepAction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar)const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);
 
		auto CustomizeNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
		{
			UDEPRECATED_K2Node_DataprepAction* DataprepActionNode  = CastChecked<UDEPRECATED_K2Node_DataprepAction>(NewNode);
 
			UBlueprint* Blueprint = DataprepActionNode->GetBlueprint();
			if (Blueprint && Blueprint->GeneratedClass && !bIsTemplateNode )
			{
				Blueprint->Modify();
				DataprepActionNode->CreateDataprepActionAsset();
			}
		};
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic( CustomizeNodeLambda );
 
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UDEPRECATED_K2Node_DataprepAction::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UK2Node_CallFunction* CallOperation = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallOperation->FunctionReference.SetExternalMember( GET_FUNCTION_NAME_CHECKED(UDataprepActionAsset, Execute), UDataprepActionAsset::StaticClass() );
	CallOperation->AllocateDefaultPins();
 
	// Manipulate the self pin
	UEdGraphPin& CallSelfPin = *CallOperation->FindPinChecked( UEdGraphSchema_K2::PSC_Self, EGPD_Input );
	CallSelfPin.DefaultObject = DuplicateObject( DataprepActionAsset, GetBlueprint()->GeneratedClass );
 
	// Connects the objects pins
	UEdGraphPin& CallFunctionInObjectsPin = *CallOperation->FindPinChecked( TEXT("InObjects"), EGPD_Input );
	CompilerContext.MovePinLinksToIntermediate( GetInObjectsPin(), CallFunctionInObjectsPin );
	
	// Connects the executions Pins
	CompilerContext.MovePinLinksToIntermediate( *GetExecPin(), *CallOperation->GetExecPin() );
	CompilerContext.MovePinLinksToIntermediate( *CallOperation->GetThenPin(), GetOutExecutionPin() );
}

UEdGraphPin& UDEPRECATED_K2Node_DataprepAction::GetOutExecutionPin() const
{
	return *FindPinChecked(UEdGraphSchema_K2::PN_Then, EGPD_Output);
}

UEdGraphPin& UDEPRECATED_K2Node_DataprepAction::GetInObjectsPin() const
{
	return *FindPinChecked( InObjectsPinName, EGPD_Input);
}

#undef LOCTEXT_NAMESPACE
