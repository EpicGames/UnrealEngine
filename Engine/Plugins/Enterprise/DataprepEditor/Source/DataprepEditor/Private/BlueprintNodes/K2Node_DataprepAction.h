// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Blueprint/K2Node_DataprepActionCore.h"

#include "K2Node_DataprepAction.generated.h"

namespace ENodeTitleType { enum Type : int; }


class FBlueprintActionDatabaseRegistrar;
class UDataprepActionAsset;
class UEdGraphPin;

UCLASS(MinimalAPI, Deprecated, meta = (DeprecationMessage = "No use of Blueprint with Dataprep."))
class UDEPRECATED_K2Node_DataprepAction : public UK2Node_DataprepActionCore
{
	GENERATED_BODY()

public:
	UDEPRECATED_K2Node_DataprepAction();

	// Begin EdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void DestroyNode() override;
	TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	// End EdGraphNode interface

	// Begin UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void PostReconstructNode() override;
	// End UK2Node interface.

private:
	UEdGraphPin& GetOutExecutionPin() const;
	UEdGraphPin& GetInObjectsPin() const;

protected:
	UPROPERTY()
	FString ActionTitle;

	UPROPERTY()
	TObjectPtr<UDataprepActionAsset> DataprepAction_DEPRECATED;

	static const FName ThenPinName;
	static const FName InObjectsPinName;
};


