// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditor.h"

#include "Widgets/DataprepAssetView.h"


#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DataprepEditor"

TSharedRef<SDockTab> FDataprepEditor::SpawnTabDetails(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == DetailsTabId);

	return SAssignNew(DetailsTabPtr, SDockTab)
		.Label(LOCTEXT("DataprepEditor_DetailsTab_Title", "Details"))
		[
			SNullWidget::NullWidget
		];
}

void FDataprepEditor::CreateDetailsViews()
{
	DetailsView = SNew( SGraphNodeDetailsWidget );
}

void FDataprepEditor::OnPipelineEditorSelectionChanged(const TSet<UObject*>& SelectedNodes)
{
	if (DetailsView.IsValid())
	{
		SetDetailsObjects(SelectedNodes, true);
	}
}

void FDataprepEditor::SetDetailsObjects(const TSet<UObject*>& Objects, bool bCanEditProperties)
{
	if (DetailsView.IsValid())
	{
		DetailsView->ShowDetailsObjects(Objects.Array());
		DetailsView->SetCanEditProperties(bCanEditProperties);

		if ( DetailsTabPtr.IsValid() && DetailsTabPtr.Pin()->GetContent() != DetailsView )
		{
			DetailsTabPtr.Pin()->SetContent( DetailsView.ToSharedRef() );
		}
	}
}

void FDataprepEditor::TryInvokingDetailsTab(bool bFlash)
{
	if (TabManager->HasTabSpawner(FDataprepEditor::DetailsTabId))
	{
		TSharedPtr<SDockTab> DataprepTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(TabManager.ToSharedRef());

		// We don't want to force this tab into existence when the data prep editor isn't in the foreground and actively
		// being interacted with.  So we make sure the window it's in is focused and the tab is in the foreground.
		if (DataprepTab.IsValid() && DataprepTab->IsForeground())
		{
			TSharedPtr<SWindow> ParentWindow = DataprepTab->GetParentWindow();
			if (ParentWindow.IsValid() && ParentWindow->HasFocusedDescendants())
			{
				if (!DetailsTabPtr.IsValid())
				{
					// Show the details panel if it doesn't exist.
					TabManager->TryInvokeTab(FDataprepEditor::DetailsTabId);
				}
			}
		}
	}

	if (bFlash)
	{
		if (DetailsTabPtr.IsValid())
		{
			TSharedPtr<SDockTab> DetailsTab = DetailsTabPtr.Pin();
			DetailsTab->FlashTab();
		}
	}
}

#undef LOCTEXT_NAMESPACE
