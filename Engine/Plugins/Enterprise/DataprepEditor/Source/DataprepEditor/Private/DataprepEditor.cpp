// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditor.h"


#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepContentConsumer.h"
#include "DataprepContentProducer.h"
#include "DataprepCorePrivateUtils.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorActions.h"
#include "DataprepEditorLogCategory.h"
#include "DataprepEditorModule.h"
#include "DataprepEditorMenu.h"
#include "DataprepGraph/DataprepGraph.h"
#include "PreviewSystem/DataprepPreviewAssetColumn.h"
#include "DataprepStats.h"
#include "PreviewSystem/DataprepPreviewSceneOutlinerColumn.h"
#include "Engine/Level.h"
#include "PreviewSystem/DataprepPreviewSystem.h"
#include "EngineLogs.h"
#include "Widgets/DataprepAssetView.h"
#include "ISceneOutliner.h"
#include "Widgets/DataprepGraph/SDataprepGraphEditor.h"
#include "Misc/PackageName.h"
#include "Widgets/SDataprepEditorViewport.h"
#include "ToolMenu.h"
#include "Widgets/SDataprepPalette.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/Paths.h"
#include "SceneOutlinerModule.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SDataprepProducersWidget.h"
#include "Widgets/SDataprepStats.h"

#include "Materials/MaterialInstanceConstant.h"

#define LOCTEXT_NAMESPACE "DataprepEditor"

extern const FName DataprepEditorAppIdentifier;

const FName FDataprepEditor::ScenePreviewTabId(TEXT("DataprepEditor_ScenePreview"));
const FName FDataprepEditor::AssetPreviewTabId(TEXT("DataprepEditor_AssetPreview"));
const FName FDataprepEditor::PaletteTabId(TEXT("DataprepEditor_Palette"));
const FName FDataprepEditor::DetailsTabId(TEXT("DataprepEditor_Details"));
const FName FDataprepEditor::DataprepAssetTabId(TEXT("DataprepEditor_Dataprep"));
const FName FDataprepEditor::SceneViewportTabId(TEXT("DataprepEditor_SceneViewport"));
const FName FDataprepEditor::DataprepStatisticsTabId(TEXT("DataprepEditor_Statistics"));
const FName FDataprepEditor::DataprepGraphEditorTabId(TEXT("DataprepEditor_GraphEditor"));

const FName FDataprepEditor::StatNameDrawCalls(TEXT("DrawCalls"));

static bool bLogTiming = true;

namespace UE
{
	namespace Dataprep
	{
		namespace Private
		{
			/**
			 * This a utility object used to preserve the selection and the detailed object(s) across reimport or execution.
			 */
			class FScopeDataprepEditorSelectionCache
			{
			public:
				FScopeDataprepEditorSelectionCache(FDataprepEditor& InDataprepEditor)
					: DataprepEditor( InDataprepEditor )
					, AssetPaths()
					, WorldItemPaths()
					, DetailObjectSource( EDetailObjectsSource::Unknow )
				{
					TSet<UObject*> SelectedAssets;
					if ( DataprepEditor.AssetsTransientPackage )
					{

						// Cache selected Assets
						SelectedAssets =  DataprepEditor.AssetPreviewView->GetSelectedAssets();
						AssetPaths.Reserve( SelectedAssets.Num() );
						FString AssetPackagePath = DataprepEditor.AssetsTransientPackage->GetPathName();
						for ( const UObject* Asset : SelectedAssets )
						{
							if ( Asset )
							{
								FString AssetPath = Asset->GetPathName();
								if ( AssetPath.RemoveFromStart( AssetPackagePath ) )
								{
									AssetPaths.Add( AssetPath );
								}
							}
						}
					}

					if ( DataprepEditor.PreviewWorld )
					{
						// Cache the selected actors
						const TSet<TWeakObjectPtr<UObject>>& SelectedItems = DataprepEditor.GetWorldItemsSelection();
						WorldItemPaths.Reserve( SelectedItems.Num() );
						for ( const TWeakObjectPtr<UObject>& WeakObject : SelectedItems )
						{
							if ( const UObject* Object = WeakObject.Get() )
							{
								WorldItemPaths.Add( Object->GetPathName( DataprepEditor.PreviewWorld->GetCurrentLevel() ) );
							}
						}
					}

					if ( DataprepEditor.DetailsView )
					{
						DetailObjectSource = EDetailObjectsSource::Unknow;
						TArray<TWeakObjectPtr<UObject>> ShownObjects = DataprepEditor.DetailsView->GetObjectsShowInDetails();
						if ( ShownObjects.Num() )
						{
							if ( const UObject* ShownObject = ShownObjects[0].Get() )
							{
								if ( SelectedAssets.Contains( ShownObject ) )
								{
									DetailObjectSource = EDetailObjectsSource::Assets;
								}
								else
								{
									DetailObjectSource = EDetailObjectsSource::World;
								}
							}
						}
					}
				}

				~FScopeDataprepEditorSelectionCache()
				{
					
					if ( UObject* AssetsTransientPackage = DataprepEditor.AssetsTransientPackage )
					{
						TSet<UObject*> AssetSelection;
						AssetSelection.Reserve( AssetPaths.Num() );
						FString AssetPackagePath = DataprepEditor.AssetsTransientPackage->GetPathName();
						for ( const FString& AssetPath : AssetPaths )
						{
							if ( UObject* Asset = FindObject<UObject>( nullptr, *( AssetPackagePath + AssetPath ) ) )
							{
								AssetSelection.Add( Asset );
							}
						}

						if ( DataprepEditor.AssetPreviewView )
						{
							DataprepEditor.AssetPreviewView->SetSelectedAssets( AssetSelection, ESelectInfo::Direct );
						}

						if ( DetailObjectSource == EDetailObjectsSource::Assets )
						{
							DataprepEditor.SetDetailsObjects( AssetSelection, false );
						}
					}

					if ( UWorld* PreviewWorld = DataprepEditor.PreviewWorld )
					{
						TSet<TWeakObjectPtr<UObject>> ItemSelection;
						ItemSelection.Reserve( WorldItemPaths.Num() );

						for ( const FString& ItemPath : WorldItemPaths )
						{
							if ( UObject* Item = FindObjectFast<UObject>( PreviewWorld->GetCurrentLevel(), *ItemPath ) )
							{
								ItemSelection.Add( Item );
							}
						}

						DataprepEditor.SetWorldObjectsSelection( MoveTemp(ItemSelection),
							FDataprepEditor::EWorldSelectionFrom::Unknow,
							DetailObjectSource == EDetailObjectsSource::World
							);
					}
				}

			private:
				FDataprepEditor& DataprepEditor;

				TArray<FString> AssetPaths;
				TArray<FString> WorldItemPaths;

				enum class EDetailObjectsSource : uint8
				{
					Unknow,
					Assets,
					World
				};

				EDetailObjectsSource DetailObjectSource;
			};
		}
	}
}

class FTimeLogger
{
public:
	FTimeLogger(const FString& InText)
		: StartTime( FPlatformTime::Cycles64() )
		, Text( InText )
	{
		if( bLogTiming )
		{
			UE_LOGF( LogDataprepEditor, Log, "%ls ...", *Text );
		}
	}

	~FTimeLogger()
	{
		if( bLogTiming )
		{
			// Log time spent to import incoming file in minutes and seconds
			double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

			int ElapsedMin = int(ElapsedSeconds / 60.0);
			ElapsedSeconds -= 60.0 * (double)ElapsedMin;
			UE_LOGF( LogDataprepEditor, Log, "%ls took [%d min %.3f s]", *Text, ElapsedMin, ElapsedSeconds );
		}
	}

private:
	uint64 StartTime;
	FString Text;
};

FDataprepEditor::FDataprepEditor()
	: bWorldBuilt(false)
	, bIsFirstRun(false)
	, bPipelineChanged(false)
	, bIsDataprepInstance(false)
	, bIsActionMenuContextSensitive(true)
	, bSaveIntermediateBuildProducts(false)
	, PreviewWorld(nullptr)
	, AssetsTransientPackage(nullptr)
	, DataprepGraph(nullptr)
	, bIgnoreCloseRequest(false)
	, PreviewSystem( MakeShared<FDataprepPreviewSystem>() )
{
	FName UniqueWorldName = MakeUniqueObjectName(GetTransientPackage(), UWorld::StaticClass(), FName( *(LOCTEXT("PreviewWorld", "Preview").ToString()) ));
	PreviewWorld = NewObject<UWorld>(GetTransientPackage(), UniqueWorldName);
	PreviewWorld->WorldType = EWorldType::EditorPreview;

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(PreviewWorld->WorldType);
	WorldContext.SetCurrentWorld(PreviewWorld);

	PreviewWorld->InitializeNewWorld(UWorld::InitializationValues()
		.AllowAudioPlayback(false)
		.CreatePhysicsScene(false)
		.RequiresHitProxies(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.ShouldSimulatePhysics(false)
		.SetTransactional(false));

	for ( ULevel* Level : PreviewWorld->GetLevels() )
	{
		for ( AActor* Actor : Level->Actors )
		{
			DefaultActorsInPreviewWorld.Add( Actor );
		}
	}
}

FDataprepEditor::~FDataprepEditor()
{
	ResetBuildWorld();

	if( DataprepAssetInterfacePtr.IsValid() )
	{
		DataprepAssetInterfacePtr->GetOnChanged().RemoveAll( this );
	}

	if ( PreviewWorld )
	{
		GEngine->DestroyWorldContext( PreviewWorld );
		PreviewWorld->DestroyWorld( true );
		PreviewWorld = nullptr;
	}

	auto DeleteDirectory = [&](const FString& DirectoryToDelete)
	{
		const FString AbsolutePath = FPaths::ConvertRelativePathToFull( DirectoryToDelete );
		IFileManager::Get().DeleteDirectory( *AbsolutePath, false, true );
	};

	// Clean up temporary directories and data created for this session
	{

		DeleteDirectory( TempDir );

		FString PackagePathToDeleteOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename( GetTransientContentFolder(), PackagePathToDeleteOnDisk))
		{
			DeleteDirectory( PackagePathToDeleteOnDisk );
		}

	}

	// Clean up temporary directories associated to process if no session of Dataprep editor is running
	{
		auto IsDirectoryEmpty = [&](const TCHAR* Directory) -> bool
		{
			bool bDirectoryIsEmpty = true;
			IFileManager::Get().IterateDirectory(Directory, [&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
			{
				bDirectoryIsEmpty = false;
				return false;
			});

			return bDirectoryIsEmpty;
		};

		FString RootTempDir = FPaths::Combine( DataprepCorePrivateUtils::GetRootTemporaryDir(), FString::FromInt( FPlatformProcess::GetCurrentProcessId() ) );
		if(IsDirectoryEmpty( *RootTempDir ))
		{
			DeleteDirectory( RootTempDir );
		}

		const FString PackagePathToDelete = FPaths::Combine( DataprepCorePrivateUtils::GetRootPackagePath(), FString::FromInt( FPlatformProcess::GetCurrentProcessId() ) );
		FString PackagePathToDeleteOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(PackagePathToDelete, PackagePathToDeleteOnDisk))
		{
			if(IsDirectoryEmpty( *PackagePathToDeleteOnDisk ))
			{
				DeleteDirectory( PackagePathToDeleteOnDisk );
			}
		}
	}
}

FName FDataprepEditor::GetToolkitFName() const
{
	return FName("DataprepEditor");
}

FText FDataprepEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Dataprep Editor");
}

FString FDataprepEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Dataprep").ToString();
}

FLinearColor FDataprepEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FDataprepEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_DataprepEditor", "Data Preparation Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	InTabManager->RegisterTabSpawner(ScenePreviewTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabScenePreview))
		.SetDisplayName(LOCTEXT("ScenePreviewTab", "Scene Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Tabs.ScenePreview"));

	InTabManager->RegisterTabSpawner(AssetPreviewTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabAssetPreview))
		.SetDisplayName(LOCTEXT("AssetPreviewTab", "Asset Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Tabs.AssetPreview"));

	InTabManager->RegisterTabSpawner(SceneViewportTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabSceneViewport))
		.SetDisplayName(LOCTEXT("SceneViewportTab", "Scene Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Tabs.SceneViewport"));

	InTabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabDetails))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(DataprepAssetTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabDataprep))
		.SetDisplayName(LOCTEXT("DataprepAssetTab", "Dataprep"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(DataprepStatisticsTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabStatistics))
		.SetDisplayName(LOCTEXT("StatisticsTab", "Statistics"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer"));

	// Do not register tabs which are not pertinent to Dataprep instance
	if(!bIsDataprepInstance)
	{
		InTabManager->RegisterTabSpawner(PaletteTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabPalette))
			.SetDisplayName(LOCTEXT("PaletteTab", "Palette"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

		InTabManager->RegisterTabSpawner(DataprepGraphEditorTabId, FOnSpawnTab::CreateSP(this, &FDataprepEditor::SpawnTabGraphEditor))
			.SetDisplayName(LOCTEXT("GraphEditorTab", "Recipe Graph"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));
		
	}
}

void FDataprepEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterAllTabSpawners();
}

void FDataprepEditor::CleanUpTemporaryDirectories()
{
	const int32 CurrentProcessID = FPlatformProcess::GetCurrentProcessId();

	TSet<FString> TempDirectories;
	IFileManager::Get().IterateDirectory( *DataprepCorePrivateUtils::GetRootTemporaryDir(), [&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
	{
		if (bIsDirectory)
		{
			FString DirectoryName = FPaths::GetBaseFilename( FilenameOrDirectory );
			if(FCString::IsNumeric( *DirectoryName ))
			{

				int32 ProcessID = FCString::Atoi( *DirectoryName );
				if(ProcessID != CurrentProcessID)
				{
					FProcHandle ProcHandle = FPlatformProcess::OpenProcess( ProcessID );

					// Delete directories if process is not valid
					bool bDeleteDirectories = !ProcHandle.IsValid();

					// Process is valid, check if application associated with process id is UnrealEditor
					if(!bDeleteDirectories)
					{
						const FString ApplicationName = FPlatformProcess::GetApplicationName( ProcessID );
						bDeleteDirectories = !ApplicationName.StartsWith(TEXT("UnrealEditor"));
					}

					if(bDeleteDirectories)
					{
						FString PackagePathToDelete = FPaths::Combine( DataprepCorePrivateUtils::GetRootPackagePath(), DirectoryName );
						FString PackagePathToDeleteOnDisk;
						if (FPackageName::TryConvertLongPackageNameToFilename(PackagePathToDelete, PackagePathToDeleteOnDisk))
						{
							TempDirectories.Add( MoveTemp(PackagePathToDeleteOnDisk) );
						}

						TempDirectories.Emplace( FilenameOrDirectory );
					}
				}
			}
		}
		return true;
	});

	for(FString& TempDirectory : TempDirectories)
	{
		FString AbsolutePath = FPaths::ConvertRelativePathToFull( TempDirectory );
		IFileManager::Get().DeleteDirectory( *AbsolutePath, false, true );
	}
}

void FDataprepEditor::InitDataprepEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UDataprepAssetInterface* InDataprepAssetInterface)
{
	DataprepAssetInterfacePtr = TWeakObjectPtr<UDataprepAssetInterface>(InDataprepAssetInterface);
	check( DataprepAssetInterfacePtr.IsValid() );

	DataprepCorePrivateUtils::Analytics::DataprepEditorOpened( InDataprepAssetInterface );

	bIsDataprepInstance = InDataprepAssetInterface->IsA<UDataprepAssetInstance>();

	DataprepAssetInterfacePtr->GetOnChanged().AddSP( this, &FDataprepEditor::OnDataprepAssetChanged );

	if ( UDataprepAsset* DataprepAsset = Cast<UDataprepAsset>( InDataprepAssetInterface ) )
	{
		DataprepAsset->GetOnStepObjectsAboutToBeRemoved().AddSP( this, &FDataprepEditor::OnStepObjectsAboutToBeDeleted );
	}

	// Assign unique session identifier
	SessionID = FGuid::NewGuid().ToString();

	// Initialize Actions' context
	DataprepActionAsset::FCanExecuteNextStepFunc CanExecuteNextStepFunc = [this](UDataprepActionAsset* ActionAsset) -> bool
	{
		return this->OnCanExecuteNextStep(ActionAsset);
	};

	DataprepActionAsset::FActionsContextChangedFunc ActionsContextChangedFunc = [this](const UDataprepActionAsset* ActionAsset, bool bWorldChanged, bool bAssetsChanged, const TArray< TWeakObjectPtr<UObject> >& NewAssets)
	{
		this->OnActionsContextChanged(ActionAsset, bWorldChanged, bAssetsChanged, NewAssets);
	};

	ActionsContext = MakeShareable( new FDataprepActionContext() );

	ActionsContext->SetTransientContentFolder( GetTransientContentFolder() / DataprepAssetInterfacePtr->GetName() / TEXT("Pipeline") )
		.SetLogger( TSharedPtr<IDataprepLogger>( new FDataprepCoreUtils::FDataprepLogger ) )
		.SetCanExecuteNextStep( CanExecuteNextStepFunc )
		.SetActionsContextChanged( ActionsContextChangedFunc );

	// Create temporary directory to store transient data
	CleanUpTemporaryDirectories();
	TempDir = FPaths::Combine( DataprepCorePrivateUtils::GetRootTemporaryDir(), FString::FromInt( FPlatformProcess::GetCurrentProcessId() ), SessionID);
	IFileManager::Get().MakeDirectory(*TempDir);

	GEditor->RegisterForUndo(this);

	// Register our commands. This will only register them if not previously registered
	FDataprepEditorCommands::Register();

	BindCommands();

	CreateTabs();
	
	// Register outliner context menus
	RegisterCopyNameAndLabelMenu();
	RegisterSelectReferencedAssetsMenu();
	RegisterSelectReferencingActorsMenu();

	const TSharedRef<FTabManager::FLayout> Layout = bIsDataprepInstance ? CreateDataprepInstanceLayout() : CreateDataprepLayout();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, DataprepEditorAppIdentifier, Layout, bCreateDefaultToolbar, bCreateDefaultStandaloneMenu, InDataprepAssetInterface );

	ExtendMenu();
	ExtendToolBar();
	RegenerateMenusAndToolbars();

#ifdef DATAPREP_EDITOR_VERBOSE
	LogDataprepEditor.SetVerbosity( ELogVerbosity::Verbose );
#endif

	// Get notified when previewing is done to update outliner selection
	PreviewSystem->GetOnPreviewIsDoneProcessing().AddSP( this, &FDataprepEditor::SyncSelectionToPreviewSystem );
}

void FDataprepEditor::SyncSelectionToPreviewSystem()
{
	// Select actors passing filter
	TArray<UObject*> WorldActors;
	FDataprepCoreUtils::GetActorsFromWorld( PreviewWorld, WorldActors );

	TSet<TWeakObjectPtr<UObject>> SelectedActors;

	for ( UObject* Actor : WorldActors )
	{
		TSharedPtr< FDataprepPreviewProcessingResult > Result = PreviewSystem->GetPreviewDataForObject( Actor );
		if ( Result.IsValid() && Result->Status == EDataprepPreviewStatus::Pass )
		{
			SelectedActors.Add( Actor );
		}
	}

	SetWorldObjectsSelection( MoveTemp(SelectedActors) );

	// Select assets passing filter
	TArray<UObject*> WorldAssets;
	for ( TWeakObjectPtr<UObject>& WeakObjectPtr : Assets )
	{
		if ( UObject* Object = WeakObjectPtr.Get() )
		{
			WorldAssets.Add( Object );
		}
	}

	TSet< UObject* > SelectedAssets;

	for ( UObject* Asset : WorldAssets )
	{
		TSharedPtr< FDataprepPreviewProcessingResult > Result = PreviewSystem->GetPreviewDataForObject( Asset );
		if ( Result.IsValid() && Result->Status == EDataprepPreviewStatus::Pass )
		{
			SelectedAssets.Add( Asset );
		}
	}

	AssetPreviewView->SelectMatchingItems( SelectedAssets );
}

void FDataprepEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PreviewWorld);
	Collector.AddReferencedObject(AssetsTransientPackage);
	Collector.AddReferencedObject(DataprepGraph);
}

FString FDataprepEditor::GetReferencerName() const
{
	return "FDataprepEdtior";
}

bool FDataprepEditor::GetReferencerPropertyName(UObject* Object, FString& OutPropertyName) const
{
	if (Object == PreviewWorld)
	{
		OutPropertyName = "PreviewWorld";
	}
	else if (Object == AssetsTransientPackage)
	{
		OutPropertyName = "AssetsTransientPackage";
	}
	else if (Object == DataprepGraph)
	{
		OutPropertyName = "DataprepGraph";
	}
	else
	{
		return false;
	}

	return true;
}

void FDataprepEditor::BindCommands()
{
	const FDataprepEditorCommands& Commands = FDataprepEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();

	UICommandList->MapAction(
		Commands.SaveScene,
		FExecuteAction::CreateSP(this, &FDataprepEditor::OnSaveScene));

	UICommandList->MapAction(
		Commands.ExecutePipeline,
		FExecuteAction::CreateSP(this, &FDataprepEditor::OnExecutePipeline),
		FCanExecuteAction::CreateSP(this, &FDataprepEditor::CanExecutePipeline));

	UICommandList->MapAction(
		Commands.CommitWorld,
		FExecuteAction::CreateSP(this, &FDataprepEditor::OnCommitWorld),
		FCanExecuteAction::CreateSP(this, &FDataprepEditor::CanCommitWorld));
}

void FDataprepEditor::OnSaveScene()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::OnSaveScene);

}

void FDataprepEditor::OnBuildWorld()
{
	UE::Dataprep::Private::FScopeDataprepEditorSelectionCache SelectionCache( *this );

	UDataprepAssetInterface* DataprepAssetInterface = DataprepAssetInterfacePtr.Get();
	if (!ensureAlways(DataprepAssetInterface))
	{
		return;
	}

	if (!ensureAlways(PreviewWorld) || !ensureAlways(PreviewWorld->IsValidLowLevel()))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::OnBuildWorld);

	if (DataprepAssetInterface->GetProducers()->GetProducersCount() == 0)
	{
		ResetBuildWorld();
		return;
	}

	DataprepCorePrivateUtils::Analytics::ImportTriggered( DataprepAssetInterfacePtr.Get() );

	CleanPreviewWorld();

	AssetsTransientPackage = NewObject< UPackage >( nullptr, *GetTransientContentFolder(), RF_Transient );
	AssetsTransientPackage->FullyLoad();

	TSharedPtr< FDataprepCoreUtils::FDataprepFeedbackContext > FeedbackContext( new FDataprepCoreUtils::FDataprepFeedbackContext );

	TSharedPtr< IDataprepProgressReporter > ProgressReporter( new FDataprepCoreUtils::FDataprepProgressUIReporter( FeedbackContext.ToSharedRef() ) );
	{
		FTimeLogger TimeLogger( TEXT("Import") );

		FDataprepProducerContext Context;
		Context.SetWorld( PreviewWorld )
			.SetRootPackage( AssetsTransientPackage )
			.SetLogger( TSharedPtr< IDataprepLogger >( new FDataprepCoreUtils::FDataprepLogger ) )
			.SetProgressReporter( ProgressReporter );

		Assets = DataprepAssetInterface->GetProducers()->Produce( Context );
	}

	if ( ProgressReporter->IsWorkCancelled() )
	{
		// Flush the work that's already been done
		ResetBuildWorld();
		return;
	}

	UpdateDataForPreviewSystem();

	CachedAssets.Empty(Assets.Num());
	for(TWeakObjectPtr<UObject>& WeakObject : Assets)
	{
		if(UObject* Object = WeakObject.Get())
		{
			CachedAssets.Emplace( Object );
		}
	}

	TakeSnapshot();

	// Init stats
	PreExecuteStatsPtr = FDataprepStats::GenerateWorldStats(PreviewWorld);
	PostExecuteStatsPtr.Reset();
	RefreshStatsTab();

	UpdatePreviewPanels();
	SceneViewportView->FocusViewportOnScene();

	bWorldBuilt = true;
	bIsFirstRun = true;
}

void FDataprepEditor::OnDataprepAssetChanged( FDataprepAssetChangeType ChangeType )
{
	switch(ChangeType)
	{
		case FDataprepAssetChangeType::RecipeModified:
		{
			bPipelineChanged = true;
		}
		break;

		case FDataprepAssetChangeType::ProducerAdded:
		case FDataprepAssetChangeType::ProducerRemoved:
		case FDataprepAssetChangeType::ProducerModified:
		{
			// Just reset world for the time being
			ResetBuildWorld();
		}
		break;

		default:
			break;

	}
}

void FDataprepEditor::ResetBuildWorld()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::ResetBuildWorld);

	bWorldBuilt = false;
	CleanPreviewWorld();
	UpdatePreviewPanels();
	FDataprepCoreUtils::DeleteTemporaryFolders( GetTransientContentFolder() );
}

void FDataprepEditor::CleanPreviewWorld()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::CleanPreviewWorld);

	FTimeLogger TimeLogger( TEXT("CleanPreviewWorld") );

	// Destroy all actors in preview world
	for (ULevel* Level : PreviewWorld->GetLevels())
	{
		TArray<AActor*> LevelActors( Level->Actors );

		for( AActor* Actor : LevelActors )
		{
			if (IsValid(Actor) && !DefaultActorsInPreviewWorld.Contains(Actor))
			{
				PreviewWorld->EditorDestroyActor(Actor, true);

				// Since deletion can be delayed, rename to avoid future name collision
				// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
				Actor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		}
	}

	SceneViewportView->ClearScene();

	// Delete assets which are still in the transient content folder
	const FString TransientContentFolder( GetTransientContentFolder() );
	TArray<UObject*> ObjectsToDelete;
	for( FSoftObjectPath& SoftObjectPath : CachedAssets )
	{
		if(SoftObjectPath.GetLongPackageName().StartsWith(TransientContentFolder))
		{
			FSoftObjectPath PackagePath(SoftObjectPath.GetLongPackageName());
			UPackage* Package = Cast<UPackage>(PackagePath.ResolveObject());

			UObject* ObjectToDelete = StaticFindObjectFast(nullptr, Package, *SoftObjectPath.GetAssetName());
			if (!ObjectToDelete)
			{
				continue;
			}

			ObjectToDelete->ClearFlags(RF_Transactional);

			if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(ObjectToDelete))
			{
				MaterialInstanceConstant->SetParentEditorOnly(nullptr);
				MaterialInstanceConstant->PostEditChange();
			}

			FDataprepCoreUtils::MoveToTransientPackage( ObjectToDelete );
			ObjectsToDelete.Add( ObjectToDelete );
		}
	}

	// Disable warnings from LogStaticMesh because FDataprepCoreUtils::PurgeObjects is pretty verbose on harmless warnings
	ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
	LogStaticMesh.SetVerbosity( ELogVerbosity::Error );

	FDataprepCoreUtils::PurgeObjects( MoveTemp( ObjectsToDelete ) );

	// Restore LogStaticMesh verbosity
	LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );

	CachedAssets.Reset();
	Assets.Reset();

	PreviewWorld->CleanupActors();
}

void FDataprepEditor::OnExecutePipeline()
{
	UE::Dataprep::Private::FScopeDataprepEditorSelectionCache SelectionCache( *this );

	if( DataprepAssetInterfacePtr->GetConsumer() == nullptr )
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::OnExecutePipeline);

	DataprepCorePrivateUtils::Analytics::ExecuteTriggered( DataprepAssetInterfacePtr.Get() );

	// Clear selection in details tab since selected objects may be gone
	DetailsView->ShowDetailsObjects(TArray<UObject*>());

	if(!bIsFirstRun)
	{
		RestoreFromSnapshot();
	}

	// Remove any link between assets referenced in the preview world and the viewport
	SceneViewportView->ClearScene();

	// Trigger execution of data preparation operations on world attached to recipe
	{
		FTimeLogger TimeLogger( TEXT("ExecutePipeline") );

		// Some operation can indirectly call UAssetEditorSubsystem::CloseAllAssetEditors (eg. Remove Asset)
		// Editors can individually refuse this request: we ignore it during the pipeline traversal.
		TGuardValue<bool> IgnoreCloseRequestGuard(bIgnoreCloseRequest, true);

		TSharedPtr< FDataprepCoreUtils::FDataprepFeedbackContext > FeedbackContext(new FDataprepCoreUtils::FDataprepFeedbackContext);

		FFeedbackContext* OldGWarn = GWarn;
		GWarn = FeedbackContext.Get();

		ActionsContext->SetProgressReporter( TSharedPtr< IDataprepProgressReporter >( new FDataprepCoreUtils::FDataprepProgressUIReporter( FeedbackContext.ToSharedRef() ) ) );
		ActionsContext->SetWorld( PreviewWorld ).SetAssets( Assets );

		DataprepAssetInterfacePtr->ExecuteRecipe( ActionsContext );

		GWarn = OldGWarn;

		// Update list of assets with latest ones
		Assets = ActionsContext->Assets.Array();
	}

	if ( ActionsContext->ProgressReporterPtr->IsWorkCancelled() )
	{
		RestoreFromSnapshot();
	}

	UpdateDataForPreviewSystem();

	// Redraw 3D viewport
	SceneViewportView->UpdateScene();

	// Add assets which may have been created by actions
	for( TWeakObjectPtr<UObject>& Asset : Assets )
	{
		if( Asset.IsValid() )
		{
			CachedAssets.Emplace( Asset.Get() );
		}
	}

	// Indicate pipeline has been executed at least once
	bIsFirstRun = false;
	// Reset tracking of pipeline changes between execution
	bPipelineChanged = false;

	SceneOutliner->FullRefresh();

	// Get stats now that the pipeline has executed
	PostExecuteStatsPtr = FDataprepStats::GenerateWorldStats(PreviewWorld);
	RefreshStatsTab();
}

void FDataprepEditor::OnCommitWorld()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::OnCommitWorld);

	FTimeLogger TimeLogger( TEXT("Commit") );

	// Pipeline has not been executed, validate with user this is intentional
	if( bIsFirstRun && DataprepAssetInterfacePtr->HasActions())
	{
		const FText Title( LOCTEXT( "DataprepEditor_ProceedWithCommit", "Proceed with commit" ) );
		const FText Message( LOCTEXT( "DataprepEditor_ConfirmCommitPipelineNotExecuted", "The action pipeline has not been executed.\nDo you want to proceed with the commit anyway?" ) );

		if( FMessageDialog::Open( EAppMsgType::YesNo, Message, Title ) != EAppReturnType::Yes )
		{
			return;
		}
	}
	// Pipeline has changed without being executed, validate with user this is intentional
	else if( !bIsFirstRun && bPipelineChanged )
	{
		const FText Title( LOCTEXT( "DataprepEditor_ProceedWithCommit", "Proceed with commit" ) );
		const FText Message( LOCTEXT( "DataprepEditor_ConfirmCommitPipelineChanged", "The action pipeline has changed since last execution.\nDo you want to proceed with the commit anyway?" ) );

		if( FMessageDialog::Open( EAppMsgType::YesNo, Message, Title ) != EAppReturnType::Yes )
		{
			return;
		}
	}

	DataprepCorePrivateUtils::Analytics::CommitTriggered( DataprepAssetInterfacePtr.Get() );

	// Remove references to assets in 3D viewport before commit
	SceneViewportView->ClearScene();

	// Finalize assets
	TArray<TWeakObjectPtr<UObject>> ValidAssets( MoveTemp(Assets) );

	FDataprepConsumerContext Context;
	Context.SetWorld( PreviewWorld )
		.SetAssets( ValidAssets )
		.SetTransientContentFolder( GetTransientContentFolder() )
		.SetLogger( TSharedPtr<IDataprepLogger>( new FDataprepCoreUtils::FDataprepLogger ) )
		.SetProgressReporter( TSharedPtr< IDataprepProgressReporter >( new FDataprepCoreUtils::FDataprepProgressUIReporter() ) );

	if( !DataprepAssetInterfacePtr->RunConsumer( Context ) )
	{
		UE_LOGF( LogDataprepEditor, Error, "Consumer failed..." );

		// Restore Dataprep's import data
		RestoreFromSnapshot();

		// Restore 3D viewport
		SceneViewportView->UpdateScene();

		// Indicates that the pipeline has not been run on the data
		bIsFirstRun = true;

		return;
	}

	ResetBuildWorld();

	UpdateDataForPreviewSystem();

	PreExecuteStatsPtr.Reset();
	PostExecuteStatsPtr.Reset();
	RefreshStatsTab();
}

void FDataprepEditor::ExtendMenu()
{
	IDataprepEditorModule* DataprepEditorModule = &FModuleManager::LoadModuleChecked<IDataprepEditorModule>("DataprepEditor");
	AddMenuExtender(DataprepEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FDataprepEditor::ExtendToolBar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FDataprepEditor* ThisEditor)
		{
			ToolbarBuilder.BeginSection("Run");
			{
				ToolbarBuilder.AddToolBarButton(FDataprepEditorCommands::Get().ExecutePipeline);
				ToolbarBuilder.AddToolBarButton(FDataprepEditorCommands::Get().CommitWorld);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	FDataprepEditor* ThisEditor = this;

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ToolkitCommands,
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, ThisEditor)
	);

	AddToolbarExtender(ToolbarExtender);

	IDataprepEditorModule* DataprepEditorModule = &FModuleManager::LoadModuleChecked<IDataprepEditorModule>("DataprepEditor");
	AddToolbarExtender(DataprepEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FDataprepEditor::CreateTabs()
{
	CreateAssetPreviewTab();

	CreateGraphEditor();

	DataprepAssetView = SNew( SDataprepAssetView, DataprepAssetInterfacePtr.Get() )
		.DataprepImportProducersDelegate( FDataprepImportProducers::CreateSP(this, &FDataprepEditor::OnBuildWorld) )
		.DataprepImportProducersEnabledDelegate( FDataprepImportProducersEnabled::CreateSP(this, &FDataprepEditor::CanBuildWorld) );

	// Create 3D viewport
	SceneViewportView = SNew( SDataprepEditorViewport, SharedThis(this) )
	.WorldToPreview( PreviewWorld );

	CreateScenePreviewTab();

	// Create Details Panel
	CreateDetailsViews();

	AssetPreviewView->OnKeyDown().BindSP( SceneViewportView.Get(), &SDataprepEditorViewport::OnKeyDown );
	ScenePreviewView->OnKeyDown().BindRaw( SceneViewportView.Get(), &SDataprepEditorViewport::OnKeyDown );
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabScenePreview(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == ScenePreviewTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataprepEditor_ScenePreviewTab_Title", "Scene Preview"))
		[
			ScenePreviewView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabAssetPreview(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == AssetPreviewTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataprepEditor_AssetPreviewTab_Title", "Asset Preview"))
		[
			SNew(SBorder)
			.Padding(2.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				AssetPreviewView.ToSharedRef()
			]
		];
}

void FDataprepEditor::CreateAssetPreviewTab()
{
	AssetPreviewView = SNew(AssetPreviewWidget::SAssetsPreviewWidget);
	AssetPreviewView->OnSelectionChanged().AddLambda(
		[this](TSet< UObject* > Selection)
		{
			SetDetailsObjects(MoveTemp(Selection), false);
		}
	);

	AssetPreviewView->OnContextMenu().BindLambda(
		[this](TSet< UObject* > Selection) -> TSharedPtr<SWidget>
		{
			UDataprepEditorContextMenuContext* ContextObject = NewObject<UDataprepEditorContextMenuContext>();
			ContextObject->SelectedObjects = Selection.Array();
			ContextObject->DataprepAsset = GetDataprepAsset();

			UToolMenus* ToolMenus = UToolMenus::Get();
			FToolMenuContext MenuContext( ContextObject );
			return ToolMenus->GenerateWidget( "DataprepEditor.AssetContextMenu", MenuContext );
		});
}

void FDataprepEditor::RegisterCopyNameAndLabelMenu()
{
	TFunction<void(const FName&, const FName&)> RegisterMenu = [](const FName& MenuNameToExtend, const FName& MenuSection)
	{
		UToolMenus* ToolMenus = UToolMenus::Get();

		UToolMenu* Menu = ToolMenus->ExtendMenu(MenuNameToExtend);

		if (!Menu)
		{
			return;
		}

		FToolMenuSection& AssetActionsSection = Menu->FindOrAddSection(MenuSection);

		AssetActionsSection.AddDynamicEntry("GetCopyName", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UDataprepEditorContextMenuContext* Context = InSection.FindContext<UDataprepEditorContextMenuContext>();
			if (Context)
			{
				if (Context->SelectedObjects.Num() == 1)
				{
					InSection.AddMenuEntry(
						"CopyName",
						LOCTEXT("CopyNameLabel", "Copy Name"),
						LOCTEXT("CopyNameTooltip", "Copy name to clipboard"),
						FSlateIcon(),
						FExecuteAction::CreateLambda([Obj = Context->SelectedObjects[0]]()
						{
							FPlatformApplicationMisc::ClipboardCopy(*Obj->GetName());
						}));


					if (AActor* Actor = Cast<AActor>(Context->SelectedObjects[0]))
					{
						InSection.AddMenuEntry(
							"CopyLabel",
							LOCTEXT("CopyLabelLabel", "Copy Label"),
							LOCTEXT("CopyLabelTooltip", "Copy actor label to clipboard"),
							FSlateIcon(),
							FExecuteAction::CreateLambda([Actor]()
							{
								FPlatformApplicationMisc::ClipboardCopy(*Actor->GetActorLabel());
							}));
					}
				}
			}
		}));
	};

	RegisterMenu("DataprepEditor.AssetContextMenu", "AssetActions");
	RegisterMenu("DataprepEditor.SceneOutlinerContextMenu", "SceneOutlinerActions");
}

void FDataprepEditor::RegisterSelectReferencedAssetsMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->ExtendMenu("DataprepEditor.SceneOutlinerContextMenu");

	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("SceneOutlinerActions");

	Section.AddDynamicEntry("SelectAssets", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		UDataprepEditorContextMenuContext* Context = InSection.FindContext<UDataprepEditorContextMenuContext>();
		if (!Context)
		{
			return;
		}

		TSet<AActor*> SelectedActors;
		for (UObject* Obj : Context->SelectedObjects)
		{
			if (AActor* Actor = Cast<AActor>(Obj))
			{
				SelectedActors.Add(Actor);
			}
		}

		if (SelectedActors.Num() == 0)
		{
			return;
		}

		TSet<UObject*> AssetSelection = FDataprepEditorUtils::GetReferencedAssets(SelectedActors);

		InSection.AddMenuEntry(
			"CopyLabel",
			LOCTEXT("SelectReferencedAssetsLabel", "Select Referenced Assets"),
			LOCTEXT("SelectReferencedAssetsTooltip", "Select assets referenced by the selected actors"),
			FSlateIcon(),
			FExecuteAction::CreateLambda([this, AssetSelection]()
			{
				AssetPreviewView->SelectMatchingItems(AssetSelection);
			}));
	}));
}

void FDataprepEditor::RegisterSelectReferencingActorsMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->ExtendMenu("DataprepEditor.AssetContextMenu");

	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("AssetActions");

	Section.AddDynamicEntry("SelectActors", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		UDataprepEditorContextMenuContext* Context = InSection.FindContext<UDataprepEditorContextMenuContext>();
		if (!Context || Context->SelectedObjects.Num() == 0)
		{
			return;
		}

		TSet<UObject*> ReferencedAssets(Context->SelectedObjects);

		InSection.AddMenuEntry(
			"SelectReferencingActors",
			LOCTEXT("SelectReferencingActorsLabel", "Select Referencing Actors"),
			LOCTEXT("SelectReferencingActorsTooltip", "Select actors that reference the selected assets"),
			FSlateIcon(),
			FExecuteAction::CreateLambda([this, ReferencedAssets]()
			{
				TSet<TWeakObjectPtr<UObject>> Actors = FDataprepEditorUtils::GetActorsReferencingAssets(PreviewWorld, ReferencedAssets);
				if (Actors.Num() > 0)
				{
					SetWorldObjectsSelection(MoveTemp(Actors));
				}
			}));
	}));
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabPalette(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == PaletteTabId);

	if(!bIsDataprepInstance)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("PaletteTab", "Palette"))
			[
				SNew(SDataprepPalette)
			];
	}

	return SNew(SDockTab);
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabDataprep(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == DataprepAssetTabId);

	return SNew(SDockTab)
	.Label(LOCTEXT("DataprepEditor_DataprepTab_Title", "Dataprep"))
	[
		SNew(SBorder)
		.Padding(2.f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			DataprepAssetView.ToSharedRef()
		]
	];
}

void FDataprepEditor::RefreshStatsTab()
{
	if (!StatsTabPtr.IsValid())
	{
		return;
	}

	check(StatsWidgetPtr.IsValid());

	TSharedPtr<SDataprepStats> StatsWidget = StatsWidgetPtr.Pin();

	if (PreExecuteStatsPtr.IsValid())
	{
		{
			FDataprepStatPtr StatPtr = StatsWidget->GetStat(StatNameDrawCalls);
			StatPtr->PreExecuteCount = TAttribute<int32>::Create(TAttribute<int32>::FGetter::CreateLambda([this]()
				{
					return SceneViewportView->GetDrawCallsAverage();
				}));
			StatsWidget->SetStats(PreExecuteStatsPtr, true);
		}

		if (PostExecuteStatsPtr.IsValid())
		{
			// Freeze live updates of draw calls stat, we want to compare this value to the new one
			FDataprepStatPtr StatPtr = StatsWidget->GetStat(StatNameDrawCalls);
			StatPtr->PreExecuteCount = SceneViewportView->GetDrawCallsAverage();
			StatPtr->PostExecuteCount = TAttribute<int32>::Create(TAttribute<int32>::FGetter::CreateLambda([this]()
			{
				return SceneViewportView->GetDrawCallsAverage();
			}));
			StatsWidget->SetStats(PostExecuteStatsPtr, false);
		}
		else
		{
			StatsWidget->ClearStats(false, true);
		}
	}
	else
	{
		// Clear the stats
		StatsWidget->ClearStats(true, true);

		// Reset the draw calls stat too
		FDataprepStatPtr StatPtr = StatsWidget->GetStat(StatNameDrawCalls);
		StatPtr->PreExecuteCount = TAttribute<int32>();
		StatPtr->PostExecuteCount = TAttribute<int32>();
	}
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabStatistics(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == DataprepStatisticsTabId);

	TSharedPtr<SDataprepStats> StatsWidget = SNew(SDataprepStats);

	TSharedRef<SDockTab> StatsTab = SNew(SDockTab)
		.Label(LOCTEXT("DataprepEditor_StatisticsTab_Title", "Statistics"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(StatsWidget, SDataprepStats)
			]
		];

	const FName CategoryAssets("Assets");
	const FName CategoryScene("Scene (World)");

	StatsWidget->AddStat(StatNameDrawCalls, CategoryScene, FDataprepStat(LOCTEXT("DataprepEditor_Stats_DrawCalls", "Draw Calls")));
	StatsWidget->AddStat(FDataprepStats::StatNameTriangles, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Triangles", "Triangles")));
	StatsWidget->AddStat(FDataprepStats::StatNameVertices, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Vertices", "Vertices")));
	StatsWidget->AddStat(FDataprepStats::StatNameNaniteTriangles, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_NaniteTriangles", "Nanite Triangles")));
	StatsWidget->AddStat(FDataprepStats::StatNameNaniteVertices, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_NaniteVertices", "Nanite Vertices")));
	StatsWidget->AddStat(FDataprepStats::StatNameTextures, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Textures", "Textures")));
	StatsWidget->AddStat(FDataprepStats::StatNameTextureSize, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_TextureSize", "Max Texture Size")));
	StatsWidget->AddStat(FDataprepStats::StatNameMeshes, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Meshes", "Meshes")));
	StatsWidget->AddStat(FDataprepStats::StatNameSkeletalMeshes, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_SkelMeshes", "Skeletal Meshes")));
	StatsWidget->AddStat(FDataprepStats::StatNameMaterials, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Materials", "Materials")));
	StatsWidget->AddStat(FDataprepStats::StatNameLights, CategoryAssets, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Lights", "Lights")));
	StatsWidget->AddStat(FDataprepStats::StatNameActors, CategoryScene, FDataprepStat(LOCTEXT("DataprepEditor_Stats_Actors", "Actors")));
	StatsWidget->AddStat(FDataprepStats::StatNameActorComponents, CategoryScene, FDataprepStat(LOCTEXT("DataprepEditor_Stats_ActorComps", "Actor Components")));
	
	StatsWidgetPtr = StatsWidget;
	StatsTabPtr = StatsTab;

	RefreshStatsTab();

	return StatsTab;
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabSceneViewport( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == SceneViewportTabId );

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		.Label( LOCTEXT("DataprepEditor_SceneViewportTab_Title", "Viewport") )
		[
			SceneViewportView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDataprepEditor::SpawnTabGraphEditor(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == DataprepGraphEditorTabId);

	if(!bIsDataprepInstance)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("DataprepEditor_GraphEditorTab_Title", "Recipe Graph"))
			[
				GraphEditor.ToSharedRef()
			];
	}

	return SNew(SDockTab);
}

TSharedRef<FTabManager::FLayout> FDataprepEditor::CreateDataprepLayout()
{
	return FTabManager::NewLayout("Standalone_DataprepEditor_Layout_v0.10")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(0.75f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->AddTab(AssetPreviewTabId, ETabState::OpenedTab)
							->SetHideTabWell( true )
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.55f)
							->AddTab(SceneViewportTabId, ETabState::OpenedTab)
							->SetHideTabWell(true)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.25f)
							->AddTab(ScenePreviewTabId, ETabState::OpenedTab)
							->SetHideTabWell( true )
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.15f)
							->AddTab(PaletteTabId, ETabState::OpenedTab)
							->SetHideTabWell( true )
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.85f)
							->AddTab(DataprepGraphEditorTabId, ETabState::OpenedTab)
							->SetHideTabWell( true )
						)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(DataprepAssetTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.7f)
						->AddTab(DetailsTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
				)
			)
		);
}

TSharedRef<FTabManager::FLayout> FDataprepEditor::CreateDataprepInstanceLayout()
{
	return FTabManager::NewLayout("Standalone_DataprepEditor_InstanceLayout_v0.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(ScenePreviewTabId, ETabState::OpenedTab)
						->SetHideTabWell( true )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(AssetPreviewTabId, ETabState::OpenedTab)
						->SetHideTabWell( true )
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.6f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(1.f)
						->AddTab(SceneViewportTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(DataprepAssetTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(DetailsTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
				)
			)
		);
}

void FDataprepEditor::UpdatePreviewPanels(bool bInclude3DViewport)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::UpdatePreviewPanels);

	{
		FTimeLogger TimeLogger( TEXT("UpdatePreviewPanels") );

		AssetPreviewView->ClearAssetList();
		FString SubstitutePath = DataprepAssetInterfacePtr->GetOutermost()->GetName();
		if(DataprepAssetInterfacePtr->GetConsumer() != nullptr && !DataprepAssetInterfacePtr->GetConsumer()->GetTargetContentFolder().IsEmpty())
		{
			SubstitutePath = DataprepAssetInterfacePtr->GetConsumer()->GetTargetContentFolder();
		}
		AssetPreviewView->SetAssetsList( Assets, GetTransientContentFolder(), SubstitutePath );
	}

	if(bInclude3DViewport)
	{
		SceneViewportView->UpdateScene();
	}
}

bool FDataprepEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	const int ActorCount = PreviewWorld->GetActorCount();
	if( InCloseReason != EAssetEditorCloseReason::AssetForceDeleted &&  bWorldBuilt && !bIgnoreCloseRequest && ActorCount > DefaultActorsInPreviewWorld.Num() )
	{
		// World was imported and is not empty -- show warning message
		const FText Title( LOCTEXT( "DataprepEditor_ProceedWithClose", "Proceed with close") );
		const FText Message( LOCTEXT( "DataprepEditor_ConfirmClose", "Imported data was not committed! Closing the editor will discard imported data.\nDo you want to close anyway?" ) );

		if (FMessageDialog::Open(EAppMsgType::YesNo, Message, Title) == EAppReturnType::Yes)
		{
			ResetBuildWorld();
			return true;
		}
		
		return false;
	}
	return !bIgnoreCloseRequest;
}

bool FDataprepEditor::CanBuildWorld()
{
	return DataprepAssetInterfacePtr->GetProducers()->GetProducersCount() > 0;
}

bool FDataprepEditor::CanExecutePipeline()
{
	return bWorldBuilt && DataprepAssetInterfacePtr->GetConsumer() != nullptr;
}

bool FDataprepEditor::CanCommitWorld()
{
	// Execution of pipeline is not required. User can directly commit result of import
	return bWorldBuilt && DataprepAssetInterfacePtr->GetConsumer() != nullptr;
}

FString FDataprepEditor::GetTransientContentFolder()
{
	return FPaths::Combine( DataprepCorePrivateUtils::GetRootPackagePath(), FString::FromInt( FPlatformProcess::GetCurrentProcessId() ), SessionID );
}

bool FDataprepEditor::OnCanExecuteNextStep(UDataprepActionAsset* ActionAsset)
{
	// #ueent_todo: Make this action configurable by user
	UpdatePreviewPanels(false);
	return true;
}

void FDataprepEditor::OnActionsContextChanged( const UDataprepActionAsset* ActionAsset, bool bWorldChanged, bool bAssetsChanged, const TArray< TWeakObjectPtr<UObject> >& NewAssets )
{
	if(bAssetsChanged)
	{
		Assets = NewAssets;
	}
}

void FDataprepEditor::RefreshColumnsForPreviewSystem()
{
	if ( PreviewSystem->HasObservedObjects() )
	{
		SceneOutliner->RemoveColumn( FSceneOutlinerBuiltInColumnTypes::ActorInfo() );
		FSceneOutlinerColumnInfo ColumnInfo(ESceneOutlinerColumnVisibility::Visible, 100, FCreateSceneOutlinerColumn::CreateLambda([Preview = PreviewSystem](ISceneOutliner& InSceneOutliner) -> TSharedRef< ISceneOutlinerColumn >
			{
				return MakeShared<FDataprepPreviewOutlinerColumn>( InSceneOutliner, Preview );
			} ) );
		SceneOutliner->AddColumn( FDataprepPreviewOutlinerColumn::ColumnID, ColumnInfo );

		AssetPreviewView->AddColumn( MakeShared<FDataprepPreviewAssetColumn>(PreviewSystem) );
	}
	else
	{
		SceneOutliner->RemoveColumn( FDataprepPreviewOutlinerColumn::ColumnID );
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		FSceneOutlinerColumnInfo* ActorInfoColumPtr = SceneOutlinerModule.DefaultColumnMap.Find( FSceneOutlinerBuiltInColumnTypes::ActorInfo() );
		if ( ActorInfoColumPtr )
		{
			SceneOutliner->AddColumn( FSceneOutlinerBuiltInColumnTypes::ActorInfo(), *ActorInfoColumPtr );
		}

		AssetPreviewView->RemoveColumn( FDataprepPreviewAssetColumn::ColumnID );
	}
}

void FDataprepEditor::UpdateDataForPreviewSystem()
{
	TArray<UObject*> ObjectsForThePreviewSystem;
	FDataprepCoreUtils::GetActorsFromWorld( PreviewWorld, ObjectsForThePreviewSystem );
	ObjectsForThePreviewSystem.Reserve( Assets.Num() );
	for ( TWeakObjectPtr<UObject>& WeakObjectPtr : Assets )
	{
		if ( UObject* Object = WeakObjectPtr.Get() )
		{
			ObjectsForThePreviewSystem.Add( Object );
		}
	}
	PreviewSystem->UpdateDataToProcess( MakeArrayView( ObjectsForThePreviewSystem ) );
}

bool FDataprepEditor::IsPreviewingStep(const UDataprepParameterizableObject* StepObject) const 
{
	return PreviewSystem->IsObservingStepObject( StepObject );
}

int32 FDataprepEditor::GetCountOfPreviewedSteps() const
{
	return PreviewSystem->GetObservedStepsCount();
}

void FDataprepEditor::OnStepObjectsAboutToBeDeleted(const TArrayView<UDataprepParameterizableObject*>& StepObjects)
{
	for ( UDataprepParameterizableObject* StepObject : StepObjects )
	{
		if ( IsPreviewingStep( StepObject ) )
		{
			ClearPreviewedObjects();
			break;
		}
	}
}

void FDataprepEditor::PostUndo(bool bSuccess)
{
	if ( bSuccess )
	{
		PreviewSystem->EnsureValidityOfTheObservedObjects();
		RefreshColumnsForPreviewSystem();
	}
}

void FDataprepEditor::PostRedo(bool bSuccess)
{
	if ( bSuccess )
	{
		PreviewSystem->EnsureValidityOfTheObservedObjects();
		RefreshColumnsForPreviewSystem();
	}
}

void FDataprepEditor::SetPreviewedObjects(const TArrayView<UDataprepParameterizableObject*>& ObservedObjects)
{
	PreviewSystem->SetObservedObjects( ObservedObjects );
	
	if ( SDataprepGraphEditor* RawGraphEditor = GraphEditor.Get() )
	{
		// Refresh the graph
		RawGraphEditor->NotifyGraphChanged();
	}

	RefreshColumnsForPreviewSystem();
}

void FDataprepEditor::ClearPreviewedObjects()
{
	SetPreviewedObjects( MakeArrayView<UDataprepParameterizableObject*>( nullptr, 0 ) );
}

#undef LOCTEXT_NAMESPACE
