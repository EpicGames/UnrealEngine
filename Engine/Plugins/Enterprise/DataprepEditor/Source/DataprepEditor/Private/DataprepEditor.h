// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepEditorUtils.h"
#include "DataprepAssetInterface.h"

#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "SceneOutlinerFwd.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Layout/SBorder.h"

class FDataprepPreviewSystem;
class SDataprepStats;
namespace ESelectInfo { enum Type : int; }
struct FDataprepStats;

class UDataprepParameterizableObject;

class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class IMessageLogListing;
class IMessageToken;
class SDataprepGraphEditor;;
class SGraphEditor;
class SGraphNodeDetailsWidget;
class SInspectorView;
class SWidget;
class UDataprepGraph;
class UEdGraphNode;

namespace UE
{
	namespace Dataprep
	{
		namespace Private
		{
			class FScopeDataprepEditorSelectionCache;
		}
	}
}

namespace AssetPreviewWidget
{
	class SAssetsPreviewWidget;
}

class SDataprepScenePreviewView : public SBorder
{
public:

	FDataprepEditorUtils::FOnKeyDown& OnKeyDown() { return OnKeyDownDelegate; }

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( OnKeyDownDelegate.IsBound() )
		{
			return OnKeyDownDelegate.Execute( MyGeometry, InKeyEvent );
		}
		return FReply::Unhandled();
	}

private:
	FDataprepEditorUtils::FOnKeyDown OnKeyDownDelegate;
};

/** Tuple linking an asset package path, a unique identifier and the UClass of the asset*/
typedef TTuple< FString, UClass*, EObjectFlags > FSnapshotDataEntry;

struct FDataprepSnapshot
{
	bool bIsValid;
	TArray<FSnapshotDataEntry> DataEntries;

	FDataprepSnapshot() : bIsValid(false) {}
};

typedef TTuple< UClass*, FText, FText > DataprepEditorClassDescription;

class FDataprepEditor : public FAssetEditorToolkit, public FEditorUndoClient, public FNotifyHook, public FGCObject
{
public:
	FDataprepEditor();
	virtual ~FDataprepEditor();

	DECLARE_MULTICAST_DELEGATE(FOnDataprepAssetProducerChanged);

	FOnDataprepAssetProducerChanged& OnDataprepAssetProducerChanged()
	{
		return DataprepAssetProducerChangedDelegate;
	}

	DECLARE_MULTICAST_DELEGATE(FOnDataprepAssetConsumerChanged);

	FOnDataprepAssetConsumerChanged& OnDataprepAssetConsumerChanged()
	{
		return DataprepAssetConsumerChangedDelegate;
	}

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	void InitDataprepEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UDataprepAssetInterface* InDataprepAssetInterface);

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	UDataprepAssetInterface* GetDataprepAsset() const
	{
		return DataprepAssetInterfacePtr.Get();
	}

	/** Gets or sets the flag for context sensitivity in the graph action menu */
	bool& GetIsContextSensitive() { return bIsActionMenuContextSensitive; }

	/** Return the selected object from the scene outliner / world preview */
	const TSet<TWeakObjectPtr<UObject>>& GetWorldItemsSelection() const { return WorldItemsSelection; };

	enum class EWorldSelectionFrom : uint8
	{
		SceneOutliner,
		Viewport,
		Unknow,
	};

	/** Set the selection of the world items */
	void SetWorldObjectsSelection(TSet<TWeakObjectPtr<UObject>>&& NewSelection, EWorldSelectionFrom SelectionFrom = EWorldSelectionFrom::Unknow, bool bSetAsDetailsObject = true);

	/** Setup the preview system to observe those steps */
	void SetPreviewedObjects(const TArrayView<UDataprepParameterizableObject*>& ObservedObjects);

	/** Clear any ongoing preview from the step preview system for this editor */
	void ClearPreviewedObjects();

	/** Is the preview system observing this step */
	bool IsPreviewingStep(const UDataprepParameterizableObject* StepObject) const;

	/** Return the number of steps the preview system is currently previewing */
	int32 GetCountOfPreviewedSteps() const;

	/** Select all actors and assets that have status Pass from the preview system */
	void SyncSelectionToPreviewSystem();

	/** Return a weak pointer to asset preview view */
	TWeakPtr<AssetPreviewWidget::SAssetsPreviewWidget> GetAssetPreviewView() { return AssetPreviewView; }

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	virtual bool GetReferencerPropertyName(UObject* Object, FString& OutPropertyName) const override;
	/** Handles change to selection in SceneOutliner */
	void OnSceneOutlinerSelectionChanged(FSceneOutlinerTreeItemPtr ItemPtr, ESelectInfo::Type SelectionMode);

	/** Returns content folder under which all assets are stored after execution of all producers */
	FString GetTransientContentFolder();

	UWorld* GetWorld()
	{
		return PreviewWorld;
	}

private:
	void BindCommands();
	void OnSaveScene();
	void OnBuildWorld();
	void ResetBuildWorld();
	void CleanPreviewWorld();
	void OnExecutePipeline();
	void OnCommitWorld();

	/** Updates asset preview, scene outliner and 3D viewport if applicable */
	void UpdatePreviewPanels(bool bInclude3dViewport = true);

	void CreateTabs();

	// Scene outliner context menus
	void RegisterCopyNameAndLabelMenu();
	void RegisterSelectReferencedAssetsMenu();
	void RegisterSelectReferencingActorsMenu();

	void CreateScenePreviewTab();

	void CreateAssetPreviewTab();

	TSharedRef<SDockTab> SpawnTabAssetPreview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabScenePreview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabPalette(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabDataprep(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabSceneViewport( const FSpawnTabArgs& Args );
	TSharedRef<SDockTab> SpawnTabStatistics(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabGraphEditor(const FSpawnTabArgs & Args);

	TSharedRef<FTabManager::FLayout> CreateDataprepLayout();
	TSharedRef<FTabManager::FLayout> CreateDataprepInstanceLayout();

	void TryInvokingDetailsTab(bool bFlash);

	/** Extends the toolbar menu to include static mesh editor options */
	void ExtendMenu();

	/** Builds the Data Prep Editor toolbar. */
	void ExtendToolBar();

	void CreateDetailsViews();

	/** Gets a multi-cast delegate which is called whenever the graph object is changed to a different graph. */
	//FOnGraphChanged& OnGraphChanged();

	/** Called whenever the selected nodes in the graph editor changes. */
	void OnPipelineEditorSelectionChanged(const TSet<UObject*>& SelectedNodes);

	/** 
	 * Change the objects display by the details panel
	 * @param bCanEditProperties Should the details view let the user modify the objects.
	 */
	void SetDetailsObjects(const TSet<UObject*>& Objects, bool bCanEditProperties);

	bool CanExecutePipeline();
	bool CanBuildWorld();
	bool CanCommitWorld();

	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;

	/** Create a snapshot of the world and tracked assets */
	void TakeSnapshot();

	/** Recreate preview world from snapshot */
	void RestoreFromSnapshot( bool bUpdateViewport = false );

	/** Handles changes in the Dataprep asset */
	void OnDataprepAssetChanged(FDataprepAssetChangeType ChangeType);

	/** Remove all temporary data remaining from previous runs of the Dataprep editor */
	void CleanUpTemporaryDirectories();

	bool OnCanExecuteNextStep(UDataprepActionAsset* ActionAsset);

	/** Handles change to the content passed to an action */
	void OnActionsContextChanged( const UDataprepActionAsset* ActionAsset, bool bWorldChanged, bool bAssetsChanged, const TArray< TWeakObjectPtr<UObject> >& NewAssets );

	/** Refresh the columns of the scene preview (outliner) and the asset preview */
	void RefreshColumnsForPreviewSystem();

	/** Update the data used by the preview system */
	void UpdateDataForPreviewSystem();

	void OnStepObjectsAboutToBeDeleted(const TArrayView<UDataprepParameterizableObject*>& StepObject);

	TSharedPtr<SWidget> OnSceneOutlinerContextMenuOpening();

	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	void CreateGraphEditor();

	void RefreshStatsTab();

	// A scoped object responsible of trying to conserve the selection across imports
	friend UE::Dataprep::Private::FScopeDataprepEditorSelectionCache;

private:
	bool bWorldBuilt;
	bool bIsFirstRun;
	bool bPipelineChanged;
	bool bIsDataprepInstance;
	TWeakObjectPtr<UDataprepAssetInterface> DataprepAssetInterfacePtr;

	FOnDataprepAssetProducerChanged DataprepAssetProducerChangedDelegate;
	FOnDataprepAssetConsumerChanged DataprepAssetConsumerChangedDelegate;

	TWeakPtr<SDockTab> DetailsTabPtr;
	TSharedPtr<class SDataprepEditorViewport> SceneViewportView;
	TSharedPtr<AssetPreviewWidget::SAssetsPreviewWidget> AssetPreviewView;
	TSharedPtr<SDataprepScenePreviewView> ScenePreviewView;
	TSharedPtr<SGraphNodeDetailsWidget> DetailsView;
	TSharedPtr<class SDataprepAssetView > DataprepAssetView;
	TSharedPtr<SDataprepGraphEditor> GraphEditor;

	TSharedPtr<class ISceneOutliner> SceneOutliner;

	/** Command list for the pipeline editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;
	bool bIsActionMenuContextSensitive;
	bool bSaveIntermediateBuildProducts;

	/** 
	 * Track the user selection from the scene outliner
	 * This is referred to the garbage collector as a week reference
	 */
	TSet<TWeakObjectPtr<UObject>> WorldItemsSelection;

	/**
	 * All assets tracked for this editor.
	 */
	TArray<TWeakObjectPtr<UObject>> Assets;
	TSet<FSoftObjectPath> CachedAssets;

	/**
	 * The world used to preview the inputs
	 */
	TObjectPtr<UWorld> PreviewWorld;

	/**
	 * The package that contains the assets of a dataprep import
	 */
	TObjectPtr<UPackage> AssetsTransientPackage;

	/**
	 * The graph used to manipulate actions and steps
	 */
	TObjectPtr<UDataprepGraph> DataprepGraph;

	TSet<class AActor*> DefaultActorsInPreviewWorld;

	/** flag raised to prevent this editor to be closed */
	bool bIgnoreCloseRequest;

	/** Array of UClasses deriving from UDataprepContentConsumer */
	TArray< DataprepEditorClassDescription > ConsumerDescriptions;

	/** Temporary folder used to store content from snapshot */
	FString TempDir;

	/** Unique identifier assigned to each opened Dataprep editor to avoid name collision on cached data */
	FString SessionID;

	/** Structure to hold on the content of the latest call to OnBuildWorld */
	FDataprepSnapshot ContentSnapshot;

	/** Helper member to record classes of assets' sub-objects */
	TMap<FString, UClass*> SnapshotClassesMap;

	/** Steps Preview System */
	TSharedRef<FDataprepPreviewSystem> PreviewSystem;

	TSharedPtr<FDataprepActionContext> ActionsContext;

	TSharedPtr<FDataprepStats> PreExecuteStatsPtr;
	TSharedPtr<FDataprepStats> PostExecuteStatsPtr;
	TWeakPtr<SDataprepStats> StatsWidgetPtr;
	TWeakPtr<SDockTab> StatsTabPtr;

	/**	The tab ids for all the tabs used */
	static const FName ScenePreviewTabId;
	static const FName AssetPreviewTabId;
	static const FName PaletteTabId;
	static const FName DetailsTabId;
	static const FName DataprepAssetTabId;
	static const FName SceneViewportTabId;
	static const FName DataprepStatisticsTabId;
	static const FName DataprepGraphEditorTabId;

	static const FName StatNameDrawCalls;
};
