// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorActions.h"

#define LOCTEXT_NAMESPACE "DataprepEditorCommands"

void FDataprepEditorCommands::RegisterCommands()
{
	// Temp code for the nodes development
	UI_COMMAND(CompileGraph, "Compile", "Compile the Dataprep graph when dirty.", EUserInterfaceActionType::ToggleButton, FInputChord());
	// end of temp code for nodes development

	UI_COMMAND(SaveScene, "Save Datasmith Scene", "To be filled.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowDatasmithSceneSettings, "Scene Settings", "Edit the scene settings.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ExecutePipeline, "Execute", "Execute the Dataprep graph on the imported data", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CommitWorld, "Commit", "Copy data from the Dataprep Editor to the project", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
