// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

#include "DataprepEditorStyle.h"

/**
* Unreal DataPrep editor actions
*/
class FDataprepEditorCommands : public TCommands<FDataprepEditorCommands>
{

public:
	FDataprepEditorCommands() : TCommands<FDataprepEditorCommands>
		(
			"DataprepEditor", // Context name for fast lookup
			NSLOCTEXT("Contexts", "DataprepEditor", "Dataprep Editor"), // Localized context name for displaying
			"EditorViewport",  // Parent
			FDataprepEditorStyle::GetStyleSetName() // Icon Style Set
			)
	{
	}

	/**
	* DataPrep Editor Commands
	*/

	/**  */
	TSharedPtr< FUICommandInfo > SaveScene;
	TSharedPtr< FUICommandInfo > ShowDatasmithSceneSettings;
	TSharedPtr< FUICommandInfo > ExecutePipeline;
	TSharedPtr< FUICommandInfo > CommitWorld;


	// Temp code for the nodes development
	TSharedPtr< FUICommandInfo > CompileGraph;
	// end of temp code for nodes development

	/**
	* Initialize commands
	*/
	virtual void RegisterCommands() override;

public:
};
