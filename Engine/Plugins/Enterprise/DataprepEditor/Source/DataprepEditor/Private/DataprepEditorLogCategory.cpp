// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorLogCategory.h"

DEFINE_LOG_CATEGORY(LogDataprepEditor)

