// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorModule.h"

#include "DataprepEditorUtils.h"
#include "DataprepEditorStyle.h"
#include "ToolMenu.h"
#include "Widgets/DataprepGraph/SDataprepGraphEditor.h"
#include "Widgets/DataprepWidgets.h"
#include "Widgets/SDataprepEditorViewport.h"
#include "Widgets/SDataprepProducersWidget.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"


// Temporary include remove when the new graph is in place


const FName DataprepEditorAppIdentifier = FName(TEXT("DataprepEditorApp"));

#define LOCTEXT_NAMESPACE "DataprepEditorModule"

class FDataprepEditorModule : public IDataprepEditorModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		FDataprepEditorStyle::Initialize();

		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		// Register the details customizer
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
		PropertyModule.RegisterCustomClassLayout( UDataprepAssetProducers::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic( &FDataprepAssetProducersDetails::MakeDetails ) );

		SDataprepGraphEditor::RegisterFactories();

		SDataprepEditorViewport::LoadDefaultSettings();

		UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FDataprepEditorModule::RegisterMenus));

		FDataprepEditorUtils::RegisterBlueprintCallbacks(this);
	}

	virtual void ShutdownModule() override
	{
		UToolMenus::UnRegisterStartupCallback(this);
		UToolMenus::UnregisterOwner(this);

		FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);

		SDataprepEditorViewport::ReleaseDefaultMaterials();

		SDataprepGraphEditor::UnRegisterFactories();

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		FDataprepEditorStyle::Shutdown();

		// Register the details customizer
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
		PropertyModule.UnregisterCustomClassLayout( TEXT("DataprepAssetProducers") );
	}

	void RegisterMenus()
	{
		// Allow cleanup when module unloads
		FToolMenuOwnerScoped OwnerScoped(this);
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("DataprepEditor.AssetContextMenu");
			Menu->AddSection("AssetActions", LOCTEXT("AssetActionsMenuHeading", "Asset Actions"));
		}
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("DataprepEditor.SceneOutlinerContextMenu");
			Menu->AddSection("SceneOutlinerActions", LOCTEXT("SceneOutlinerMenuHeading", "Scene Outliner"));
		}
	}

	/** Gets the extensibility managers for outside entities to extend datasmith data prep editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	virtual TSharedRef<SWidget> CreateDataprepProducersWidget(UDataprepAssetProducers* AssetProducers) override
	{
		return AssetProducers ? SNew(SDataprepProducersWidget, AssetProducers) : SNullWidget::NullWidget;
	}

	virtual TSharedRef<SWidget> CreateDataprepDetailsView(UObject* ObjectToDetail) override
	{
		if(ObjectToDetail)
		{
			return SNew(SDataprepDetailsView).Object( ObjectToDetail );
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};

IMPLEMENT_MODULE(FDataprepEditorModule, DataprepEditor)

#undef LOCTEXT_NAMESPACE
