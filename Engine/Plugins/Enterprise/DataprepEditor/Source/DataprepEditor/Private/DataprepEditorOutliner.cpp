// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorMenu.h"

#include "ISceneOutlinerColumn.h"
#include "SceneOutlinerModule.h"
#include "DataprepEditorOutlinerMode.h"
#include "ToolMenuContext.h"
#include "WorldTreeItem.h"
#include "FolderTreeItem.h"

#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Widgets/SDataprepEditorViewport.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "DataprepEditor"

namespace DataprepEditorSceneOutlinerUtils
{
	/** Specifics for the scene outliner */

	/**
	 * Use this struct to get the selection from the scene outliner
	 */
	struct FGetSelectionFromSceneOutliner
	{
		mutable TSet<TWeakObjectPtr<UObject>> Selection;

		void TryAdd(const ISceneOutlinerTreeItem& Item) const
		{
			if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
			{
				Selection.Add(ActorItem->Actor);
			}
			else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
			{
				Selection.Add(ComponentItem->Component);
			}
		}
	};

	void SetVisibility(ISceneOutlinerTreeItem& Item, bool bIsVisible, TWeakPtr<SDataprepEditorViewport> Viewport)
	{
		if (FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (Actor)
			{
				// Save the actor to the transaction buffer to support undo/redo, but do
				// not call Modify, as we do not want to dirty the actor's package and
				// we're only editing temporary, transient values
				SaveToTransactionBuffer(Actor, false);
				Actor->SetIsTemporarilyHiddenInEditor(!bIsVisible);

				Viewport.Pin()->SetActorVisibility(Actor, bIsVisible);

				// Apply the same visibility to the actors children
				for (auto& ChildPtr : ActorItem->GetChildren())
				{
					auto Child = ChildPtr.Pin();
					if (Child.IsValid())
					{
						SetVisibility(*Child, bIsVisible, Viewport);
					}
				}
			}
		}
		else if (Item.IsA<FWorldTreeItem>() || Item.IsA<FFolderTreeItem>())
		{
			for (auto& ChildPtr : Item.GetChildren())
			{
				auto Child = ChildPtr.Pin();
				if (Child.IsValid())
				{
					SetVisibility(*Child, bIsVisible, Viewport);
				}
			}
		}
	}

	class FVisibilityDragDropOp : public FDragDropOperation
	{
	public:

		DRAG_DROP_OPERATOR_TYPE(FVisibilityDragDropOp, FDragDropOperation)

		/** Flag which defines whether to hide destination actors or not */
		bool bHidden;

		/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
		TUniquePtr<FScopedTransaction> UndoTransaction;

		/** The widget decorator to use */
		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNullWidget::NullWidget;
		}

		/** Create a new drag and drop operation out of the specified flag */
		static TSharedRef<FVisibilityDragDropOp> New(const bool _bHidden, TUniquePtr<FScopedTransaction>& ScopedTransaction)
		{
			TSharedRef<FVisibilityDragDropOp> Operation = MakeShareable(new FVisibilityDragDropOp);

			Operation->bHidden = _bHidden;
			Operation->UndoTransaction = MoveTemp(ScopedTransaction);

			Operation->Construct();
			return Operation;
		}
	};

	class FPreviewSceneOutlinerGutter : public ISceneOutlinerColumn
	{
	public:
		FPreviewSceneOutlinerGutter(ISceneOutliner& Outliner, TWeakPtr<SDataprepEditorViewport> InViewport)
		{
			WeakSceneViewport = InViewport;
			WeakOutliner = StaticCastSharedRef<ISceneOutliner>(Outliner.AsShared());
		}

		virtual ~FPreviewSceneOutlinerGutter() {}

		static FName GetID() { return FName("PreviewGutter"); }

		virtual void Tick(double InCurrentTime, float InDeltaTime) override
		{
			VisibilityCache.VisibilityInfo.Empty();
		}

		virtual FName GetColumnID() override { return GetID(); }

		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override
		{
			return SHeaderRow::Column(GetColumnID())
				.FixedWidth(16.f)
				[
					SNew(SSpacer)
				];
		}

		virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

		virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override {}

		/** Check whether the specified item is visible */
		bool IsItemVisible(const ISceneOutlinerTreeItem& Item) { return VisibilityCache.GetVisibility(Item); }

		TWeakPtr<SDataprepEditorViewport> GetViewport() const
		{
			return WeakSceneViewport;
		}

	private:
		/** Weak pointer back to the scene outliner - required for setting visibility on current selection. */
		TWeakPtr<ISceneOutliner> WeakOutliner;

		TWeakPtr<SDataprepEditorViewport> WeakSceneViewport;

		/** Get and cache visibility for items. Cached per-frame to avoid expensive recursion. */
		FSceneOutlinerVisibilityCache VisibilityCache;
	};

	/** Widget responsible for managing the visibility for a single actor */
	class SVisibilityWidget : public SImage
	{
	public:
		SLATE_BEGIN_ARGS(SVisibilityWidget) {}
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, TWeakPtr<FPreviewSceneOutlinerGutter> InWeakColumn, TWeakPtr<ISceneOutliner> InWeakOutliner, TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem)
		{
			WeakTreeItem = InWeakTreeItem;
			WeakOutliner = InWeakOutliner;
			WeakColumn = InWeakColumn;

			SImage::Construct(
				SImage::FArguments()
				.Image(this, &SVisibilityWidget::GetBrush)
			);
		}

	private:

		/** Start a new drag/drop operation for this widget */
		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				return FReply::Handled().BeginDragDrop(FVisibilityDragDropOp::New(!IsVisible(), UndoTransaction));
			}
			else
			{
				return FReply::Unhandled();
			}
		}

		/** If a visibility drag drop operation has entered this widget, set its actor to the new visibility state */
		virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			auto VisibilityOp = DragDropEvent.GetOperationAs<FVisibilityDragDropOp>();
			if (VisibilityOp.IsValid())
			{
				SetIsVisible(!VisibilityOp->bHidden);
			}
		}

		FReply HandleClick()
		{
			auto Outliner = WeakOutliner.Pin();
			auto TreeItem = WeakTreeItem.Pin();
			auto Column = WeakColumn.Pin();

			if (!Outliner.IsValid() || !TreeItem.IsValid() || !Column.IsValid())
			{
				return FReply::Unhandled();
			}

			// Open an undo transaction
			UndoTransaction.Reset(new FScopedTransaction(LOCTEXT("SetActorVisibility", "Set Actor Visibility")));

			const auto& Tree = Outliner->GetTree();

			const bool bVisible = !IsVisible();

			// We operate on all the selected items if the specified item is selected
			if (Tree.IsItemSelected(TreeItem.ToSharedRef()))
			{
				auto Viewport = WeakColumn.Pin()->GetViewport();

				for (auto& SelectedItem : Tree.GetSelectedItems())
				{
					if (IsVisible(SelectedItem, Column) != bVisible)
					{
						DataprepEditorSceneOutlinerUtils::SetVisibility(*SelectedItem, bVisible, Viewport);
					}
				}

				GEditor->RedrawAllViewports();
			}
			else
			{
				SetIsVisible(bVisible);
			}

			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}

		virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
		{
			return HandleClick();
		}

		/** Called when the mouse button is pressed down on this widget */
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
			{
				return FReply::Unhandled();
			}

			return HandleClick();
		}

		/** Process a mouse up message */
		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				UndoTransaction.Reset();
				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

		/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
		virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override
		{
			UndoTransaction.Reset();
		}

		/** Get the brush for this widget */
		const FSlateBrush* GetBrush() const
		{
			if (IsVisible())
			{
				static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");
				static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
				return IsHovered() ? FAppStyle::GetBrush(NAME_VisibleHoveredBrush) :
					FAppStyle::GetBrush(NAME_VisibleNotHoveredBrush);
			}
			else
			{
				static const FName NAME_NotVisibleHoveredBrush = TEXT("Level.NotVisibleHighlightIcon16x");
				static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");
				return IsHovered() ? FAppStyle::GetBrush(NAME_NotVisibleHoveredBrush) :
					FAppStyle::GetBrush(NAME_NotVisibleNotHoveredBrush);
			}
		}

		/** Check if the specified item is visible */
		static bool IsVisible(const FSceneOutlinerTreeItemPtr& Item, const TSharedPtr<FPreviewSceneOutlinerGutter>& Column)
		{
			return Column.IsValid() && Item.IsValid() ? Column->IsItemVisible(*Item) : false;
		}

		/** Check if our wrapped tree item is visible */
		bool IsVisible() const
		{
			return IsVisible(WeakTreeItem.Pin(), WeakColumn.Pin());
		}

		/** Set the actor this widget is responsible for to be hidden or shown */
		void SetIsVisible(const bool bVisible)
		{
			TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin();
			TSharedPtr<ISceneOutliner> Outliner = WeakOutliner.Pin();

			if (TreeItem.IsValid() && Outliner.IsValid() && IsVisible() != bVisible)
			{
				DataprepEditorSceneOutlinerUtils::SetVisibility(*TreeItem, bVisible, WeakColumn.Pin()->GetViewport());

				Outliner->Refresh();

				GEditor->RedrawAllViewports();
			}
		}

		/** The tree item we relate to */
		TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem;

		/** Reference back to the outliner so we can set visibility of a whole selection */
		TWeakPtr<ISceneOutliner> WeakOutliner;

		/** Weak pointer back to the column */
		TWeakPtr<FPreviewSceneOutlinerGutter> WeakColumn;

		/** Scoped undo transaction */
		TUniquePtr<FScopedTransaction> UndoTransaction;
	};

	const TSharedRef<SWidget> FPreviewSceneOutlinerGutter::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SVisibilityWidget, SharedThis(this), WeakOutliner, TreeItem)
			];
	}

	/** End of specifics for the scene outliner */
}

void FDataprepEditor::CreateScenePreviewTab()
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	
	FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([SpecifiedWorld = TWeakObjectPtr<UWorld>(PreviewWorld), DataprepEditor = StaticCastSharedRef<FDataprepEditor>(AsShared())](SSceneOutliner* Outliner)
		{
			return new FDataprepEditorOutlinerMode(Outliner, DataprepEditor, SpecifiedWorld);
		});

	FSceneOutlinerInitializationOptions SceneOutlinerOptions;
	SceneOutlinerOptions.ModeFactory = ModeFactory;

	SceneOutliner = SceneOutlinerModule.CreateSceneOutliner(SceneOutlinerOptions);

	// Add our custom visibility gutter

	auto CreateColumn = [this](ISceneOutliner& Outliner)
	{
		return TSharedRef< ISceneOutlinerColumn >(MakeShareable(new DataprepEditorSceneOutlinerUtils::FPreviewSceneOutlinerGutter(Outliner, SceneViewportView.ToSharedRef())));
	};

	FSceneOutlinerColumnInfo ColumnInfo;
	ColumnInfo.Visibility = ESceneOutlinerColumnVisibility::Visible;
	ColumnInfo.PriorityIndex = 0;
	ColumnInfo.Factory.BindLambda([&](ISceneOutliner& Outliner)
	{
		return TSharedRef< ISceneOutlinerColumn >(MakeShareable(new DataprepEditorSceneOutlinerUtils::FPreviewSceneOutlinerGutter(Outliner, SceneViewportView.ToSharedRef())));
	});

	SceneOutliner->AddColumn(DataprepEditorSceneOutlinerUtils::FPreviewSceneOutlinerGutter::GetID(), ColumnInfo);

	// Add the default columns
	const FSharedSceneOutlinerData& SharedData = SceneOutliner->GetSharedData();
	for (auto& DefaultColumn : SceneOutlinerModule.DefaultColumnMap)
	{
		SceneOutliner->AddColumn(DefaultColumn.Key, DefaultColumn.Value);
	}

	SAssignNew(ScenePreviewView, SDataprepScenePreviewView)
		.Padding(2.f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SceneOutliner.ToSharedRef()
			]
		];
}

TSharedPtr<SWidget> FDataprepEditorOutlinerMode::CreateContextMenu()
{
	DataprepEditorSceneOutlinerUtils::FGetSelectionFromSceneOutliner Selector;

	for (FSceneOutlinerTreeItemPtr Item : SceneOutliner->GetTree().GetSelectedItems())
	{
		Selector.TryAdd(*Item);
	}

	TSet<UObject*> SelectedActors;

	for (TWeakObjectPtr<UObject> ObjectPtr : Selector.Selection)
	{
		if (AActor* Actor = Cast<AActor>(ObjectPtr.Get()))
		{
			SelectedActors.Add(Actor);
		}
	}

	TSharedPtr<FDataprepEditor> DataprepEditor = DataprepEditorPtr.Pin();

	if (!DataprepEditor.IsValid() || SelectedActors.Num() == 0)
	{
		return TSharedPtr<SWidget>();
	}

	// Build context menu

	UDataprepEditorContextMenuContext* ContextObject = NewObject<UDataprepEditorContextMenuContext>();
	ContextObject->SelectedObjects = SelectedActors.Array();
	ContextObject->DataprepAsset = DataprepEditor->GetDataprepAsset();

	UToolMenus* ToolMenus = UToolMenus::Get();
	FToolMenuContext MenuContext( nullptr, nullptr, ContextObject );
	return ToolMenus->GenerateWidget( "DataprepEditor.SceneOutlinerContextMenu", MenuContext );
}

void FDataprepEditor::OnSceneOutlinerSelectionChanged(FSceneOutlinerTreeItemPtr ItemPtr, ESelectInfo::Type SelectionMode)
{
	DataprepEditorSceneOutlinerUtils::FGetSelectionFromSceneOutliner Selector;

	for (FSceneOutlinerTreeItemPtr Item : SceneOutliner->GetTree().GetSelectedItems())
	{
		Selector.TryAdd(*Item);
	}

	SetWorldObjectsSelection(MoveTemp(Selector.Selection), EWorldSelectionFrom::SceneOutliner);
}

void FDataprepEditor::SetWorldObjectsSelection(TSet<TWeakObjectPtr<UObject>>&& NewSelection, EWorldSelectionFrom SelectionFrom /* = EWorldSelectionFrom::Unknow */,  bool bSetAsDetailsObject /* = true */)
{
	WorldItemsSelection.Empty(NewSelection.Num());
	WorldItemsSelection.Append(MoveTemp(NewSelection));

	if (SelectionFrom != EWorldSelectionFrom::SceneOutliner)
	{
		DataprepEditorSceneOutlinerUtils::FSynchroniseSelectionToSceneOutliner Selector(StaticCastSharedRef<FDataprepEditor>(AsShared()));
		SceneOutliner->SetSelection(Selector);
	}

	if (SelectionFrom != EWorldSelectionFrom::Viewport)
	{
		TArray<AActor*> Actors;
		Actors.Reserve(WorldItemsSelection.Num());

		for (TWeakObjectPtr<UObject> ObjectPtr : WorldItemsSelection)
		{
			if (AActor* Actor = Cast<AActor>(ObjectPtr.Get()))
			{
				Actors.Add(Actor);
			}
		}

		SceneViewportView->SelectActors(Actors);
	}

	if ( bSetAsDetailsObject )
	{
		TSet<UObject*> Objects;
		Objects.Reserve(WorldItemsSelection.Num());
		for (const TWeakObjectPtr<UObject>& ObjectPtr : WorldItemsSelection)
		{
			Objects.Add(ObjectPtr.Get());
		}

		SetDetailsObjects(Objects, false);
	}
}

#undef LOCTEXT_NAMESPACE
