// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"
#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "DataprepEditor.h"
#include "Misc/CoreDelegates.h"
#include "SSceneOutliner.h"

namespace DataprepEditorSceneOutlinerUtils
{
	/**
	* Use this struct to match the scene outliers selection to a dataprep editor selection
	*/
	struct FSynchroniseSelectionToSceneOutliner
	{
		FSynchroniseSelectionToSceneOutliner(TWeakPtr<FDataprepEditor> InDataprepEditor)
			: DataprepEditorPtr(InDataprepEditor)
		{
		};

		bool operator()(const ISceneOutlinerTreeItem& Item) const
		{
			if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
			{
				if (const FDataprepEditor* DataprepEditor = DataprepEditorPtr.Pin().Get())
				{
					return DataprepEditor->GetWorldItemsSelection().Contains(ActorItem->Actor);
				}
			}
			else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
			{
				if (const FDataprepEditor* DataprepEditor = DataprepEditorPtr.Pin().Get())
				{
					return DataprepEditor->GetWorldItemsSelection().Contains(ComponentItem->Component);
				}
			}

			return false;
		}
	private:
		TWeakPtr<FDataprepEditor> DataprepEditorPtr;
	};
}

class FDataprepEditorOutlinerMode : public FActorMode
{
public:
	FDataprepEditorOutlinerMode(SSceneOutliner* InSceneOutliner, TWeakPtr<FDataprepEditor> InDataprepEditor, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay = nullptr)
		: FActorMode(FActorModeParams(InSceneOutliner, InSpecifiedWorldToDisplay, true, true))
		, DataprepEditorPtr(InDataprepEditor)
	{
		FCoreDelegates::OnActorLabelChanged.AddRaw(this, &FDataprepEditorOutlinerMode::OnActorLabelChanged);
	}

	~FDataprepEditorOutlinerMode()
	{
		FCoreDelegates::OnActorLabelChanged.RemoveAll(this);
	}

	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override { return false; }
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }

	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override
	{
		auto DataprepEditor = DataprepEditorPtr.Pin();
		if (DataprepEditor)
		{
			DataprepEditor->OnSceneOutlinerSelectionChanged(Item, SelectionType);
		}
	}

	virtual TSharedPtr<SWidget> CreateContextMenu() override;

private:
	void OnActorLabelChanged(AActor* InChangedActor)
	{
		if (!ensure(InChangedActor))
		{
			return;
		}
		
		if (!DataprepEditorPtr.IsValid())
		{
			return;
		}

		if (InChangedActor->GetWorld() != DataprepEditorPtr.Pin()->GetWorld())
		{
			return;
		}

		if (SceneOutliner && IsActorDisplayable(InChangedActor))
		{
			// Force create the item otherwise the outliner may not be notified of a change to the item if it is filtered out
			if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FActorTreeItem>(InChangedActor, true))
			{
				SceneOutliner->OnItemLabelChanged(Item);
			}
		}
	}

private:
	TWeakPtr<FDataprepEditor> DataprepEditorPtr;
};
