// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "DataprepEditorModule.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FDataprepEditorStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define BOX_PLUGIN_BRUSH( RelativePath, ... ) FSlateBoxBrush( FDataprepEditorStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )

TSharedPtr<FSlateStyleSet> FDataprepEditorStyle::StyleSet;

void FDataprepEditorStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	StyleSet->Set("DataprepEditor.Producer", new IMAGE_PLUGIN_BRUSH("Icons/Producer24", Icon20x20));
	StyleSet->Set("DataprepEditor.Producer.Selected", new IMAGE_PLUGIN_BRUSH("Icons/Producer24", Icon20x20));

	StyleSet->Set("DataprepEditor.SaveScene", new IMAGE_PLUGIN_BRUSH("Icons/SaveScene", Icon40x40));
	StyleSet->Set("DataprepEditor.SaveScene.Small", new IMAGE_PLUGIN_BRUSH("Icons/SaveScene", Icon20x20));
	StyleSet->Set("DataprepEditor.SaveScene.Selected", new IMAGE_PLUGIN_BRUSH("Icons/SaveScene", Icon40x40));
	StyleSet->Set("DataprepEditor.SaveScene.Selected.Small", new IMAGE_PLUGIN_BRUSH("Icons/SaveScene", Icon20x20));

	StyleSet->Set("DataprepEditor.ShowDataprepSettings", new IMAGE_PLUGIN_BRUSH("Icons/IconOptions", Icon40x40));
	StyleSet->Set("DataprepEditor.ShowDatasmithSceneSettings", new IMAGE_PLUGIN_BRUSH("Icons/IconOptions", Icon40x40));

	StyleSet->Set("DataprepEditor.CommitWorld", new IMAGE_PLUGIN_BRUSH("Icons/CommitWorld", Icon40x40));
	StyleSet->Set("DataprepEditor.CommitWorld.Small", new IMAGE_PLUGIN_BRUSH("Icons/CommitWorld", Icon20x20));
	StyleSet->Set("DataprepEditor.CommitWorld.Selected", new IMAGE_PLUGIN_BRUSH("Icons/CommitWorld", Icon40x40));
	StyleSet->Set("DataprepEditor.CommitWorld.Selected.Small", new IMAGE_PLUGIN_BRUSH("Icons/CommitWorld", Icon20x20));

	StyleSet->Set("DataprepEditor.ExecutePipeline", new IMAGE_PLUGIN_BRUSH("Icons/ExecutePipeline", Icon40x40));
	StyleSet->Set("DataprepEditor.ExecutePipeline.Small", new IMAGE_PLUGIN_BRUSH("Icons/ExecutePipeline", Icon20x20));
	StyleSet->Set("DataprepEditor.ExecutePipeline.Selected", new IMAGE_PLUGIN_BRUSH("Icons/ExecutePipeline", Icon40x40));
	StyleSet->Set("DataprepEditor.ExecutePipeline.Selected.Small", new IMAGE_PLUGIN_BRUSH("Icons/ExecutePipeline", Icon20x20));

	StyleSet->Set("DataprepEditor.Pipeline.GroupActions", new IMAGE_PLUGIN_BRUSH("Icons/GroupActions", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.UngroupActions", new IMAGE_PLUGIN_BRUSH("Icons/UngroupActions", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.EnableActions", new IMAGE_PLUGIN_BRUSH("Icons/EnableActions", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.DisableActions", new IMAGE_PLUGIN_BRUSH("Icons/DisableActions", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.EnableActionGroup", new IMAGE_PLUGIN_BRUSH("Icons/EnableActionGroup", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.DisableActionGroup", new IMAGE_PLUGIN_BRUSH("Icons/DisableActionGroup", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.PreviewFilter", new IMAGE_PLUGIN_BRUSH("Icons/PreviewFilter", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.InversePreviewFilter", new IMAGE_PLUGIN_BRUSH("Icons/InversePreviewFilter", Icon16x16));
	StyleSet->Set("DataprepEditor.Pipeline.ClearPreviewFilter", new IMAGE_PLUGIN_BRUSH("Icons/ClearPreviewFilter", Icon16x16));

	StyleSet->Set("DataprepEditor.TrackNode.Slot", new IMAGE_PLUGIN_BRUSH("CircleBox", Icon22x22));

	StyleSet->Set("DataprepEditor.Node.Body", new BOX_PLUGIN_BRUSH("Node_Body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f)));

	StyleSet->Set( "DataprepEditor.SectionFont", DEFAULT_FONT( "Bold", 10 ) );

	// Dataprep action UI
	{
		StyleSet->Set("Dataprep.Background.Black", FLinearColor(FColor(26, 26, 26)) );
		StyleSet->Set("Dataprep.TextSeparator.Color", FLinearColor(FColor(120, 120, 120, 255)) );
		StyleSet->Set("Dataprep.TextSeparatorActionNode.Color", FLinearColor(FColor(100, 100, 100, 255)) );

		StyleSet->Set("DataprepEditor.SoftwareCursor_Grab", new IMAGE_PLUGIN_BRUSH("Icons/cursor_grab", Icon20x20));
		StyleSet->Set("DataprepEditor.SoftwareCursor_Hand", new IMAGE_PLUGIN_BRUSH("Icons/cursor_hand", Icon20x20));

		StyleSet->Set("DataprepAction.Outter.Regular.Padding", FMargin(1.f, 2.f, 4.f, 2.f));
		StyleSet->Set("DataprepAction.Outter.Selected.Padding", FMargin(0.f));
		StyleSet->Set("DataprepAction.Body.Padding", FMargin(6.f, 2.f, 2.f, 2.f));
		StyleSet->Set("DataprepAction.Steps.Padding", FMargin(11.f, 2.f, 6.f, 2.f));
		StyleSet->Set("DataprepAction.DragAndDrop", FLinearColor(FColor(212, 212, 59)) );

		StyleSet->Set("DataprepAction.Group.OutlineColor", FLinearColor(FColor(0, 102, 255)));

		StyleSet->Set("DataprepAction.OutlineColor", FLinearColor(FColor(10, 177, 51)));
		StyleSet->Set("DataprepAction.BackgroundColor", FLinearColor(FColor(61, 61, 61)));

		{
			FTextBlockStyle TilteTextStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
			TilteTextStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 11));
			StyleSet->Set("DataprepAction.TitleTextStyle", TilteTextStyle);

			FEditableTextBoxStyle TitleEditableText = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("LegacyViewportMenu").EditableTextStyle;
			TitleEditableText.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 11));

			StyleSet->Set( "DataprepAction.TitleInlineEditableText", FInlineEditableTextBlockStyle()
				.SetTextStyle(TilteTextStyle)
				.SetEditableTextBoxStyle(TitleEditableText)
			);
		}

		StyleSet->Set("DataprepAction.EmptyStep.Background.Hovered", FLinearColor(FColor(66, 66, 66)));
		StyleSet->Set("DataprepAction.EmptyStep.Background.Normal", FLinearColor(FColor(57, 57, 57)));
		StyleSet->Set("DataprepAction.EmptyStep.Outer.Hovered", FLinearColor(FColor(117, 117, 117)));
		StyleSet->Set("DataprepAction.EmptyStep.Outer.Normal", FLinearColor(FColor(85, 85, 85)));
		StyleSet->Set("DataprepAction.EmptyStep.Text.Hovered", FLinearColor(FColor(230, 230, 230)));
		StyleSet->Set("DataprepAction.EmptyStep.Text.Normal", FLinearColor(FColor(117, 117, 117)));

		StyleSet->Set("DataprepAction.EmptyStep.Bottom.Padding", FMargin(0.f, 0.f, 0.f, 5.f));

		StyleSet->Set("DataprepActionSteps.BackgroundColor", FLinearColor(FColor(26, 26, 26)));

		StyleSet->Set("DataprepActionStep.BackgroundColor", FLinearColor(FColor(93, 93, 93)) );
		StyleSet->Set("DataprepActionStep.DragAndDrop", FLinearColor(FColor(212, 212, 59)) );
		StyleSet->Set("DataprepActionStep.Selected", FLinearColor(FColor(1, 202, 252)) );
		StyleSet->Set("DataprepActionStep.Filter.OutlineColor", FLinearColor(FColor(220, 125, 67)));
		StyleSet->Set("DataprepActionStep.Operation.OutlineColor", FLinearColor(FColor(67, 177, 220)) );
		StyleSet->Set("DataprepActionStep.SelectionTransform.OutlineColor", FLinearColor(FColor(255, 88, 255)) );
		StyleSet->Set("DataprepActionStep.Separator.Color", FLinearColor(FColor(182, 219, 192)) );

		StyleSet->Set("DataprepActionStep.Outter.Disabled.Padding", FMargin(5.f, 3.f, 6.f, 3.f));
		StyleSet->Set("DataprepActionStep.Outter.Regular.Padding", FMargin(5.f, 3.f, 10.f, 3.f));
		StyleSet->Set("DataprepActionStep.Outter.Selected.Padding", FMargin(5.f, 0.f, 3.f, 0.f));
		StyleSet->Set("DataprepActionStep.Padding", FMargin(10.f, 3.f, 6.f, 3.f));

		StyleSet->Set("DataprepActionStep.DnD.Outter.Padding", FMargin(0.f, 5.f, 0.f, 5.f));
		StyleSet->Set("DataprepActionStep.DnD.Inner.Padding", FMargin(5.f, 5.f, 5.f, 5.f));

		StyleSet->Set("DataprepActionBlock.ContentBackgroundColor.Old", FLinearColor(0.11f, 0.11f, 0.11f));
		{
			FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
			TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 10));
			StyleSet->Set("DataprepActionBlock.TitleTextBlockStyle", TilteTextBlockStyle);
		}


		StyleSet->Set("Graph.ActionStepNode.PreviewColor", FLinearColor(0.822786f, 0.715693f, 0.0f, 1.f));

		{
			FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
			TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Italic", 7))
				.SetShadowOffset(FVector2D::ZeroVector)
				.SetColorAndOpacity(GetColor("Graph.ActionStepNode.PreviewColor"))
				.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.7f));
			StyleSet->Set("DataprepActionBlock.PreviewTextBlockStyle", TilteTextBlockStyle);
		}

		StyleSet->Set("DataprepEditor.ActionStepNode.ArrowNext", new IMAGE_PLUGIN_BRUSH("Triangle", FVector2D(30.0f, 10.0f)));
	}

	// DataprepGraphEditor
	{
		StyleSet->Set("Graph.TrackEnds.BackgroundColor", FLinearColor(0.05f, 0.05f, 0.05f, 0.2f));
		StyleSet->Set("Graph.TrackInner.BackgroundColor", FLinearColor(FColor(50, 50, 50, 200)));
		
		StyleSet->Set("Graph.ActionNode.BackgroundColor", FLinearColor(0.115861f, 0.115861f, 0.115861f));
		{
			FTextBlockStyle GraphActionNodeTitle = FTextBlockStyle(/*NormalText*/)
			.SetColorAndOpacity( FLinearColor(230.0f/255.0f,230.0f/255.0f,230.0f/255.0f) )
			.SetFont( FCoreStyle::GetDefaultFontStyle("Bold", 14) );
			StyleSet->Set( "Graph.ActionNode.Title", GraphActionNodeTitle );

			FEditableTextBoxStyle GraphActionNodeTitleEditableText = FEditableTextBoxStyle()
			.SetTextStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(GraphActionNodeTitle.Font);
			StyleSet->Set( "Graph.ActionNode.NodeTitleEditableText", GraphActionNodeTitleEditableText );

			StyleSet->Set( "Graph.ActionNode.TitleInlineEditableText", FInlineEditableTextBlockStyle()
				.SetTextStyle(GraphActionNodeTitle)
				.SetEditableTextBoxStyle(GraphActionNodeTitleEditableText)
			);
		}

		StyleSet->Set("Graph.ActionNode.Margin", FMargin(2.f, 0.0f, 2.f, 0.0f));
		StyleSet->Set("Graph.ActionNode.DesiredSize", FVector2D(300.f, 300.f));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

#undef IMAGE_PLUGIN_BRUSH
#undef BOX_PLUGIN_BRUSH
#undef RootToContentDir

void FDataprepEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FDataprepEditorStyle::GetStyleSetName()
{
	static FName StyleName("DataprepEditorStyle");
	return StyleName;
}

FString FDataprepEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString BaseDir = IPluginManager::Get().FindPlugin(DATAPREPEDITOR_MODULE_NAME)->GetBaseDir() + TEXT("/Resources");
	return (BaseDir / RelativePath) + Extension;
}
