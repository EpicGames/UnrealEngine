// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class ISlateStyle;

/** Contains a collection of named properties (StyleSet) that guide the appearance of Datasmith related UI. */
class FDataprepEditorStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static TSharedPtr<ISlateStyle> Get() { return StyleSet; }

	static FName GetStyleSetName();

	static float GetFloat( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetFloat( PropertyName, Specifier );
	}

	static FVector2D GetVector( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetVector( PropertyName, Specifier );
	}

	static const FLinearColor& GetColor( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetColor( PropertyName, Specifier );
	}

	static const FMargin& GetMargin( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetMargin( PropertyName, Specifier );
	}

	static const FSlateBrush* GetBrush( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetBrush( PropertyName, Specifier );
	}

	template< typename WidgetStyleType >            
	static const WidgetStyleType& GetWidgetStyle( FName PropertyName, const ANSICHAR* Specifier = nullptr )
	{
		return StyleSet->GetWidgetStyle<WidgetStyleType>( PropertyName, Specifier );
	}

private:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
