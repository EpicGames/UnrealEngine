// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditorUtils.h"

#include "Components/StaticMeshComponent.h"
#include "DataprepAsset.h"
#include "DataprepOperation.h"
#include "Engine/Texture.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Materials/MaterialInterface.h"
#include "IAssetTools.h"
#include "Widgets/Parameterization/SDataprepLinkToParameter.h"

#include "AssetToolsModule.h"
#include "Dialogs/DlgPickPath.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorUtilityBlueprintFactory.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Interfaces/IMainFrameModule.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_Event.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "SchemaActions/DataprepMenuActionCollectorUtils.h"
#include "ScopedTransaction.h"
#include "SelectionSystem/DataprepFilter.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "DataprepEditorUtils"

namespace DataprepEditorUtils
{
	struct FFilterCategory
	{
		FText Name;
		UClass* FetcherClass;
	};

	class SPickFetcherClassDialog : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SPickFetcherClassDialog) {}
			SLATE_ARGUMENT(TArray<TSharedPtr<FFilterCategory>>, Items)
		SLATE_END_ARGS()

		bool IsDialogConfirmed() const { return bDialogConfirmed; }

		int32 GetSelectedCategoryIndex() const { return SelectedCategoryIndex; }

		virtual void Construct(const FArguments& InArgs)
		{
			Items = InArgs._Items;

			SelectedCategoryIndex = Items.Num() > 0 ? 0 : -1;

			ChildSlot 
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(0)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(30, 30, 4, 6)
					[
						SNew(STextBlock).Text(LOCTEXT("CategoryLabel", "Category"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Bottom)
					.Padding(4, 3, 30, 4)
					[
						SNew(SComboBox< TSharedPtr<FFilterCategory> >)
						.ContentPadding(FMargin(6.0f, 2.0f))
						.OptionsSource(&Items)
						.OnGenerateWidget_Lambda([](TSharedPtr<FFilterCategory> Item)
						{
							return SNew(STextBlock).Text(Item->Name);
						})
						.Content()
						[
							SNew(STextBlock)
							.Text(this, &SPickFetcherClassDialog::GetItemText)
						]
						.OnSelectionChanged_Lambda([this](TSharedPtr<FFilterCategory> NewItem, ESelectInfo::Type SelectType)
						{
							if (!NewItem.IsValid() || Items.Num() < 1)
							{
								return;
							}
							SelectedCategoryIndex = Items.Find(NewItem);
						})
					]				
				]

				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1)
					[
						SNew(SBox)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(4, 3)
					[
						SNew(SButton)
						.Text(LOCTEXT("ChooseButtonLabel", "Choose"))
						.ContentPadding(FMargin(8, 2, 8, 2))
						.OnClicked(this, &SPickFetcherClassDialog::OnConfirmClicked)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(4, 3)
					[
						SNew(SButton)
						.ContentPadding(FMargin(8, 2, 8, 2))
						.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
						.OnClicked(this, &SPickFetcherClassDialog::OnCancelClicked)
					]
				]
			];
		}

	private:
		int32 SelectedCategoryIndex = -1;
		bool bDialogConfirmed = false;

		TArray<TSharedPtr<FFilterCategory>> Items;

		FText GetItemText() const
		{
			return SelectedCategoryIndex >= 0 ? Items[SelectedCategoryIndex]->Name : FText::FromString( "No filters" );
		}

		FReply OnConfirmClicked()
		{
			bDialogConfirmed = true;
			CloseDialog();
			return FReply::Handled();
		}

		FReply OnCancelClicked()
		{
			bDialogConfirmed = false;
			CloseDialog();
			return FReply::Handled();
		}

		void CloseDialog()
		{
			TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

			if (ContainingWindow.IsValid())
			{
				ContainingWindow->RequestDestroyWindow();
			}
		}
	};

	bool CreateEditorUtilityBlueprint(UClass* ParentClass)
	{
		TSharedRef<SDlgPickPath> PickContentPathDlg =
			SNew(SDlgPickPath)
			.Title(LOCTEXT("DataprepPalette_ChooseContentPath", "Choose Location for new asset"))
			.DefaultPath(FText::FromString("/Game"));

		if (PickContentPathDlg->ShowModal() == EAppReturnType::Ok)
		{
			UClass* FactoryClass = UEditorUtilityBlueprintFactory::StaticClass();
			UEditorUtilityBlueprintFactory* NewFactory = NewObject<UEditorUtilityBlueprintFactory>(GetTransientPackage(), FactoryClass);
			NewFactory->ParentClass = ParentClass;

			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

			FString PackageName = PickContentPathDlg->GetPath().ToString();
			FString AssetName;
			FString TempPackageName;

			AssetTools.CreateUniqueAssetName(PackageName / NewFactory->GetDefaultNewAssetName(), FString(), TempPackageName, AssetName);

			UClass* AssetClass = NewFactory->GetSupportedClass();
		
			if (AssetClass)
			{
				AssetTools.CreateAsset(AssetName, PackageName, AssetClass, NewFactory, FName("ContentBrowserNewAsset"));
				return true;
			}
		}
		return false;
	}

	void OnNewUserFilterCreated(UBlueprint* InBlueprint)
	{
		check(InBlueprint && InBlueprint->SkeletonGeneratedClass);

		FBlueprintEditorUtils::ConformImplementedInterfaces(InBlueprint);

		UFunction* OverrideFunc = nullptr;
		UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(InBlueprint, FName("Fetch"), &OverrideFunc);
		check(OverrideFunc);

		// Implement the function graph
		UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, FName("Fetch"), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

		FBlueprintEditorUtils::AddFunctionGraph(InBlueprint, NewGraph, /*bIsUserCreated=*/ false, OverrideFuncClass);

		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewGraph);
	}

	void OnNewUserOperationCreated(UBlueprint* InBlueprint)
	{
		check(InBlueprint && InBlueprint->SkeletonGeneratedClass);

		FBlueprintEditorUtils::ConformImplementedInterfaces(InBlueprint);

		UFunction* OverrideFunc = nullptr;
		UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(InBlueprint, FName("OnExecution"), &OverrideFunc);
		check(OverrideFunc);
		UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(InBlueprint);
		if (EventGraph && UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(OverrideFunc))
		{
			// Add to event graph
			FName EventName = OverrideFunc->GetFName();
			UK2Node_Event* NewEventNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Event>(
				EventGraph,
				EventGraph->GetGoodPlaceForNewNode(),
				EK2NewNodeFlags::SelectNewNode,
				[EventName, OverrideFuncClass](UK2Node_Event* NewInstance)
				{
					NewInstance->EventReference.SetExternalMember(EventName, OverrideFuncClass);
					NewInstance->bOverrideFunction = true;
				}
			);

			UEdGraphPin* FromPin = NewEventNode->FindPin(FName("InContext"));

			TArrayView < UEdGraphPin* > Pins = MakeArrayView( &FromPin, 1 );

			FEdGraphSchemaAction_K2NewNode::CreateNode( 
				EventGraph, 
				Pins, 
				EventGraph->GetGoodPlaceForNewNode(),
				[](UEdGraph* InParentGraph)->UK2Node*
				{
					return NewObject<UK2Node_BreakStruct>( InParentGraph );
				},
				[FromPin](UK2Node* NewNode)
				{
					UScriptStruct* StructType = Cast<UScriptStruct>(FromPin->PinType.PinSubCategoryObject.Get());
					Cast<UK2Node_BreakStruct>(NewNode)->StructType = StructType;
				},
				EK2NewNodeFlags::None);


			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject( NewEventNode );
		}
	}
}

void FDataprepParametrizationActionData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DataprepAsset);
	Collector.AddReferencedObject(Object);
}

bool FDataprepParametrizationActionData::IsValid() const
{
	return DataprepAsset && Object && PropertyChain.Num() > 0;
}

void FDataprepEditorUtils::PopulateMenuForParameterization(FMenuBuilder& MenuBuilder, UDataprepAsset& DataprepAsset, UDataprepParameterizableObject& Object, const TArray<FDataprepPropertyLink>& PropertyChain)
{
	TSharedRef<FDataprepParametrizationActionData> ActionData = MakeShared<FDataprepParametrizationActionData>( DataprepAsset, Object, PropertyChain );

	MenuBuilder.BeginSection( NAME_None, LOCTEXT("ParametrizationMenuSection", "Parameterization") );
	{
		FName ParameterName = DataprepAsset.GetNameOfParameterForObjectProperty( &Object, PropertyChain );

		FNewMenuDelegate BindToParamerizationDelegate;
		

		BindToParamerizationDelegate.BindLambda( [ActionData, ParameterName](FMenuBuilder& BindToParamerizationDelegateMenu)
			{
				FUIAction DoNothing;
				BindToParamerizationDelegateMenu.AddWidget( SNew(SDataprepLinkToParameter, ActionData), FText(), true, false );
			});


		MenuBuilder.AddSubMenu( LOCTEXT("LinkToParameterLabel", "Link To Parameter"), LOCTEXT("LinkToParameterTooltip", "Link this property to a existing parameter or a new one")
			, BindToParamerizationDelegate, true, FSlateIcon(), false );

		if ( !ParameterName.IsNone() )
		{ 
			
			FUIAction RemoveBinding;
			FText UnlinkFromParameterLabel = LOCTEXT("UnlinkFromParameterLabel", "Remove Link To Parameter");
			RemoveBinding.ExecuteAction.BindLambda( [ActionData, UnlinkFromParameterLabel]()
				{
					if ( ActionData->IsValid() )
					{
						FScopedTransaction ScopedTransaction( UnlinkFromParameterLabel );
						ActionData->DataprepAsset->RemoveObjectPropertyFromParameterization( ActionData->Object, ActionData->PropertyChain );
					}
				});

			MenuBuilder.AddMenuEntry( UnlinkFromParameterLabel, TAttribute<FText>(), FSlateIcon(), RemoveBinding );
		}
	}
	MenuBuilder.EndSection();
}

FSlateFontInfo FDataprepEditorUtils::GetGlyphFont()
{
	return FAppStyle::Get().GetFontStyle("FontAwesome.11");
}

TSharedPtr<SWidget> FDataprepEditorUtils::MakeContextMenu(const TSharedPtr<FDataprepParametrizationActionData>& ParameterizationActionData)
{
	if ( ParameterizationActionData && ParameterizationActionData->IsValid() )
	{
		FMenuBuilder MenuBuilder( true, nullptr );
		PopulateMenuForParameterization( MenuBuilder, *ParameterizationActionData->DataprepAsset,
			*ParameterizationActionData->Object, ParameterizationActionData->PropertyChain );
		return MenuBuilder.MakeWidget();
	}

	return {};
}

void FDataprepEditorUtils::RegisterBlueprintCallbacks(void* InModule)
{
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(InModule, UDataprepFetcher::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateStatic(&DataprepEditorUtils::OnNewUserFilterCreated));
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(InModule, UDataprepOperation::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateStatic(&DataprepEditorUtils::OnNewUserOperationCreated));
}

bool FDataprepEditorUtils::CreateUserDefinedFilter()
{
	// Get available categories
	TArray<TSharedPtr<DataprepEditorUtils::FFilterCategory>> FilterCategories;

	TArray<UClass*> FilterClasses = DataprepMenuActionCollectorUtils::GetNativeChildClasses( *UDataprepFilter::StaticClass() );

	for ( UClass* FilterClass : FilterClasses )
	{
		if ( FilterClass )
		{
			UDataprepFilter* Filter = FilterClass->GetDefaultObject< UDataprepFilter >();
			if ( Filter )
			{
				UClass* FetcherClass = Filter->GetAcceptedFetcherClass().Get();
				if ( FetcherClass )
				{
					TSharedPtr< DataprepEditorUtils::FFilterCategory > Category = MakeShared<DataprepEditorUtils::FFilterCategory>();
					Category->Name = FetcherClass->GetDisplayNameText();
					Category->FetcherClass = FetcherClass;
					FilterCategories.Add( Category );
				}
			}
		}
	}

	TSharedRef< DataprepEditorUtils::SPickFetcherClassDialog > FetcherPickDialog = SNew( DataprepEditorUtils::SPickFetcherClassDialog ).Items( FilterCategories );

	// Create SWindow that contains SPickFetcherClassDialog
	const FVector2D WindowSize( 300.f, 130.f );
	TSharedRef< SWindow > DialogWindow =
		SNew( SWindow )
		.Title( LOCTEXT( "GenericAssetDialogWindowHeader", "Choose Filter Category" ) )
		.ClientSize( WindowSize );
	DialogWindow->SetContent( FetcherPickDialog );

	// Launch dialog and block thread until user finishes with it
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked< IMainFrameModule >( TEXT( "MainFrame" ) );
	const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
	if ( MainFrameParentWindow.IsValid() )
	{
		FSlateApplication::Get().AddModalWindow( DialogWindow, MainFrameParentWindow.ToSharedRef() );
	}
	else
	{
		ensureMsgf(false, TEXT("Could not create \"Pick Fetcher Class\" dialog, this should never happen."));
	}

	if ( !FetcherPickDialog->IsDialogConfirmed() )
	{
		return false;
	}

	UClass* SelectedFetcherClass = FilterCategories[FetcherPickDialog->GetSelectedCategoryIndex()]->FetcherClass;

	return DataprepEditorUtils::CreateEditorUtilityBlueprint( SelectedFetcherClass );
}

bool FDataprepEditorUtils::CreateUserDefinedOperation()
{
	return DataprepEditorUtils::CreateEditorUtilityBlueprint( UDataprepEditingOperation::StaticClass() );
}

TSet<UObject*> FDataprepEditorUtils::GetReferencedAssets(const TSet<AActor*>& InActors)
{
	auto GetReferencedMeshesAndMaterials = [](const AActor* Actor, TSet<UObject*>& ActorMeshes, TSet<UObject*>& ActorMaterials)
	{
		TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();

			if (Mesh)
			{
				ActorMeshes.Add(Mesh);

				for (FStaticMaterial& StaticMaterial : Mesh->GetStaticMaterials())
				{
					if (StaticMaterial.MaterialInterface)
					{
						ActorMaterials.Add(StaticMaterial.MaterialInterface);
					}
				}
			}

			for (UMaterialInterface* MeshComponentMaterialInterface : StaticMeshComponent->OverrideMaterials)
			{
				if (MeshComponentMaterialInterface)
				{
					ActorMaterials.Add(MeshComponentMaterialInterface);
				}
			}
		}
	};

	TSet<UObject*> StaticMeshes;
	TSet<UObject*> Materials;
	TSet<UObject*> Textures;

	for (AActor* Actor : InActors)
	{
		GetReferencedMeshesAndMaterials(Actor, StaticMeshes, Materials);
	}

	for (UObject* Material : Materials)
	{
		TArray<UTexture*> MaterialTextures;
		Cast<UMaterialInterface>(Material)->GetUsedTextures(MaterialTextures);
		for (UTexture* Texture : MaterialTextures)
		{
			if (Texture)
			{
				Textures.Add(Texture);
			}
		}
	}

	TSet<UObject*> Assets;
	
	Assets.Append(StaticMeshes);
	Assets.Append(Materials);
	Assets.Append(Textures);

	return MoveTemp(Assets);
}

TSet<TWeakObjectPtr<UObject>> FDataprepEditorUtils::GetActorsReferencingAssets(UWorld* InWorld, const TSet<UObject*>& InAssets)
{
	TSet<UMaterialInterface*> Materials;
	TSet<UStaticMesh*> StaticMeshes;
	TSet<UTexture*> Textures;

	for (UObject* Asset : InAssets)
	{
		if (!ensure(Asset) || !IsValid(Asset))
		{
			continue;
		}

		if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Asset))
		{
			StaticMeshes.Add(StaticMesh);
		}
		else if (UTexture* Texture = Cast< UTexture >(Asset))
		{
			Textures.Add(Texture);
		}
		else if (UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >(Asset))
		{
			Materials.Add(MaterialInterface);
		}
	}

	auto DoesMaterialUseTexture = [](const UMaterialInterface* Material, const UTexture* CheckTexture)
	{
		TArray<UTexture*> Textures;
		Material->GetUsedTextures(Textures);
		for (int32 i = 0; i < Textures.Num(); i++)
		{
			if (Textures[i] == CheckTexture)
			{
				return true;
			}
		}
		return false;
	};

	auto GetReferencedMeshesAndMaterials = [](const AActor* Actor, TSet<UStaticMesh*>& ActorMeshes, TSet<UMaterialInterface*>& ActorMaterials)
	{
		TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();

			if (Mesh)
			{
				ActorMeshes.Add(Mesh);

				for (FStaticMaterial& StaticMaterial : Mesh->GetStaticMaterials())
				{
					ActorMaterials.Add(StaticMaterial.MaterialInterface);
				}
			}

			for (UMaterialInterface* MeshComponentMaterialInterface : StaticMeshComponent->OverrideMaterials)
			{
				ActorMaterials.Add(MeshComponentMaterialInterface);
			}
		}
	};

	// Iterate world actors and check their assets
	TSet<TWeakObjectPtr<UObject>> ResultActorSet;

	const EActorIteratorFlags Flags = EActorIteratorFlags::SkipPendingKill;
	for (TActorIterator<AActor> It(InWorld, AActor::StaticClass(), Flags); It; ++It)
	{
		AActor* Actor = *It;
		if (!IsValid(Actor))
		{
			continue;
		}

		TSet<UMaterialInterface*> ActorMaterials;
		TSet<UStaticMesh*> ActorMeshes;

		GetReferencedMeshesAndMaterials(Actor, ActorMeshes, ActorMaterials);

		// Check for referencing the different asset types.

		for (UStaticMesh* StaticMesh : ActorMeshes)
		{
			if (StaticMeshes.Contains(StaticMesh))
			{
				ResultActorSet.Add(Actor);
				break;
			}
		}
		if (!ResultActorSet.Contains(Actor))
		{
			// Check materials
			for (UMaterialInterface* ActorMaterial : ActorMaterials)
			{
				if (Materials.Contains(ActorMaterial))
				{
					ResultActorSet.Add(Actor);
					break;
				}
			}
		}
		if (!ResultActorSet.Contains(Actor))
		{
			// Check the textures
			for (UMaterialInterface* ActorMaterial : ActorMaterials)
			{
				for (UTexture* Texture : Textures)
				{
					if (DoesMaterialUseTexture(ActorMaterial, Texture))
					{
						ResultActorSet.Add(Actor);
						break;
					}
				}
			}
		}
	}

	return MoveTemp(ResultActorSet);
}

#undef LOCTEXT_NAMESPACE
