// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Parameterization/DataprepParameterizationUtils.h"

#include "UObject/GCObject.h"

class FReply;
struct FGeometry;
struct FKeyEvent;
struct FSlateFontInfo;

class FMenuBuilder;
class SWidget;
class UDataprepAsset;
class UDataprepParameterizableObject;

struct FDataprepParametrizationActionData : public FGCObject
{
	FDataprepParametrizationActionData(UDataprepAsset& InDataprepAsset, UDataprepParameterizableObject& InObject, const TArray<FDataprepPropertyLink>& InPropertyChain)
		: FGCObject()
		, DataprepAsset(&InDataprepAsset)
		, Object(&InObject)
		, PropertyChain(InPropertyChain)
	{}

	TObjectPtr<UDataprepAsset> DataprepAsset;
	TObjectPtr<UDataprepParameterizableObject> Object;
	TArray<FDataprepPropertyLink> PropertyChain;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDataprepParametrizationActionData");
	}

	bool IsValid() const;
};

class FDataprepEditorUtils
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnKeyDown, const FGeometry& , const FKeyEvent& )

	/**
	 * Populate a menu builder with the section made for the parameterization
	 * @param DataprepAsset the asset that own the object
	 * @param Object The Object on which we want to modify the parametrization binding
	 * @param PropertyChain The property chain is the property path from the class of the object to the property that we want to edit
	 */
	static void PopulateMenuForParameterization(FMenuBuilder& MenuBuilder, UDataprepAsset& DataprepAsset, UDataprepParameterizableObject& Object, const TArray<FDataprepPropertyLink>& PropertyChain);

	static FSlateFontInfo GetGlyphFont();

	/**
	 * Make a context menu widget to manage parameterization link of a property
	 */
	static TSharedPtr<SWidget> MakeContextMenu(const TSharedPtr<FDataprepParametrizationActionData>& ParameterizationActionData);

	static void RegisterBlueprintCallbacks(void* InModule);

	/**
	 * Create new user-defined filter, based on blueprint
	 * @return Returns true if the filter was successfully created, false if cancelled
	 */
	static bool CreateUserDefinedFilter();

	/**
	 * Create new user-defined operation, based on blueprint
	 * @return Returns true if the operation was successfully created, false if cancelled
	 */
	static bool CreateUserDefinedOperation();

	/**
	 * Get assets referenced by a set of actors.
	 * @param InActors	Actors to check for assets
	 */
	static TSet<UObject*> GetReferencedAssets(const TSet<AActor*>& InActors);

	/**
	 * Get all actors that reference any asset in the input assets list
	 * @param InAssets The set of referenced assets to follow
	 */
	static TSet<TWeakObjectPtr<UObject>> GetActorsReferencingAssets(UWorld* InWorld, const TSet<UObject*>& InAssets);
};
