// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepGraph.h"

#include "DataprepEditor.h"
#include "DataprepGraph/DataprepGraphSchema.h"
#include "Widgets/DataprepGraph/SDataprepGraphEditor.h"

#include "DataprepAsset.h"

#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

void UDataprepGraph::Initialize(UDataprepAsset* InDataprepAsset)
{
	DataprepAssetPtr = InDataprepAsset;

	// Add recipe graph editor node which will be used as a start point to populate 
	UDataprepGraphRecipeNode* RecipeNode = Cast<UDataprepGraphRecipeNode>(CreateNode(UDataprepGraphRecipeNode::StaticClass(), false));
	RecipeNode->SetEnabledState(ENodeEnabledState::Disabled, true);
}

void FDataprepEditor::CreateGraphEditor()
{
	if(UDataprepAsset* DataprepAsset = Cast<UDataprepAsset>(DataprepAssetInterfacePtr.Get()) )
	{
		FGraphAppearanceInfo AppearanceInfo;
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "DATAPREP");

		// Create the title bar widget
		TSharedRef<SWidget> TitleBarWidget =
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
			.HAlign(HAlign_Fill)
			.Padding( 4.f )
			[
				SNew(SConstraintCanvas)
				+ SConstraintCanvas::Slot()
			.Anchors( FAnchors(0.5) )
			.Alignment( FVector2D(0.5,0.5) )
			.AutoSize( true )
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DataprepRecipeEditor_TitleBar_Label", "Dataprep Recipe"))
			.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
			]
		];

		FName UniqueGraphName = MakeUniqueObjectName( GetTransientPackage(), UWorld::StaticClass(), FName( *(LOCTEXT("DataprepGraph", "Graph").ToString()) ) );
		DataprepGraph = NewObject< UDataprepGraph >(GetTransientPackage(), UniqueGraphName);
		DataprepGraph->Schema = UDataprepGraphSchema::StaticClass();

		DataprepGraph->Initialize( DataprepAsset );

		GraphEditor = SNew(SDataprepGraphEditor, DataprepAsset)
			.Appearance(AppearanceInfo)
			.TitleBar(TitleBarWidget)
			.GraphToEdit(DataprepGraph)
			.DataprepEditor( StaticCastSharedRef<FDataprepEditor>( this->AsShared() ) );
	}
}

#undef LOCTEXT_NAMESPACE
