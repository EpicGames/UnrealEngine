// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "EdGraph/EdGraph.h"

#include "UObject/Package.h"
#include "DataprepGraph.generated.h"

class SGraphEditor;
class SGraphNode;
class FSlateRect;
class UDataprepAsset;
class UDataprepGraphActionNode;
class UToolMenu;

/**
 * The UDataprepGraph class is used to display the pipeline of a Dataprep asset
 * in a SDataprepGraphEditor.
 */
UCLASS()
class DATAPREPEDITOR_API UDataprepGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	/** Associate Dataprep asset with graph and add root node */
	void Initialize(UDataprepAsset* InDataprepAsset);

	const UDataprepAsset* GetDataprepAsset() const { return DataprepAssetPtr.Get(); }
	UDataprepAsset* GetDataprepAsset() { return DataprepAssetPtr.Get(); }

private:
	/** Dataprep asset associated with this graph */
	TWeakObjectPtr<UDataprepAsset> DataprepAssetPtr;
};

/**
 * The UDataprepGraphRecipeNode is the root graph node from which the associated widget,
 * SDataprepGraphTrackNode, will add all the action nodes and their content
 */
UCLASS()
class UDataprepGraphRecipeNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	/** Get/Set associated widget */
	TSharedPtr<SGraphNode> GetWidget() { return Widget.Pin(); }
	void SetWidget(TSharedPtr<SGraphNode> InWidget) { Widget = InWidget; }

	// UEdGraphNode implementation
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override { return false; }
	// End UEdGraphNode implementation

private:
	/**
	 * Associated widget displayed in the Dataprep graph editor
	 */
	TWeakPtr<SGraphNode> Widget;
};
