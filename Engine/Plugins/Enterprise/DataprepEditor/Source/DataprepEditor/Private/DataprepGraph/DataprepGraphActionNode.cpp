// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepGraph/DataprepGraphActionNode.h"

// Dataprep includes
#include "DataprepAsset.h"

// Engine includes
#include "Kismet2/Kismet2NameValidators.h"

#define LOCTEXT_NAMESPACE "DataprepGraphActionNode"

UDataprepGraphActionNode::UDataprepGraphActionNode()
	: DataprepActionAsset(nullptr)
	, ExecutionOrder(INDEX_NONE)
{
	bCanRenameNode = true;
	ActionTitle = LOCTEXT("DefaultActionNodeTitle", "New Action").ToString();
}

void UDataprepGraphActionNode::Initialize(TWeakObjectPtr<UDataprepAsset> InDataprepAssetPtr, UDataprepActionAsset* InDataprepActionAsset, int32 InExecutionOrder)
{
	if(InDataprepActionAsset)
	{
		DataprepAssetPtr = InDataprepAssetPtr;
		DataprepActionAsset = InDataprepActionAsset;
		ActionTitle = DataprepActionAsset->GetLabel();
		ExecutionOrder = InExecutionOrder;
		NodeWidth = InDataprepActionAsset->GetAppearance()->NodeSize.X;
		NodeHeight = InDataprepActionAsset->GetAppearance()->NodeSize.Y;
	}
	else
	{
		ensure(false);
	}
}

FLinearColor UDataprepGraphActionNode::GetNodeTitleColor() const
{
	return FLinearColor(0.0036765f, 0.3864294f, 0.2501584f);
}

FText UDataprepGraphActionNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString( ActionTitle );
}

void UDataprepGraphActionNode::OnRenameNode(const FString& NewName)
{
	ActionTitle = NewName;
	if ( DataprepActionAsset )
	{
		DataprepActionAsset->SetLabel( *NewName );
	}
}

void UDataprepGraphActionNode::DestroyNode()
{
	if ( DataprepActionAsset )
	{
		Modify();

		DataprepActionAsset->NotifyDataprepSystemsOfRemoval();

		// Force the transaction system to restore the action
		DataprepActionAsset = nullptr;
	}
 
	Super::DestroyNode();
}

TSharedPtr<class INameValidatorInterface> UDataprepGraphActionNode::MakeNameValidator() const
{
	// The name doesn't matter
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

void UDataprepGraphActionNode::ResizeNode(const FVector2f& NewSize)
{
	if ( DataprepActionAsset )
	{
		DataprepAssetPtr->Modify();
		DataprepActionAsset->Modify();
		DataprepActionAsset->GetAppearance()->Modify();
		DataprepActionAsset->GetAppearance()->NodeSize = NewSize;
	}
}

UDataprepGraphActionStepNode::UDataprepGraphActionStepNode()
	: DataprepActionAsset(nullptr)
	, StepIndex(INDEX_NONE)
{
	bCanRenameNode = false;
}

const UDataprepActionStep* UDataprepGraphActionStepNode::GetDataprepActionStep() const
{
	return DataprepActionAsset ? DataprepActionAsset->GetStep(StepIndex).Get() : nullptr;
}

UDataprepActionStep* UDataprepGraphActionStepNode::GetDataprepActionStep()
{
	return DataprepActionAsset ? DataprepActionAsset->GetStep(StepIndex).Get() : nullptr;
}

FLinearColor UDataprepGraphActionStepNode::GetNodeTitleColor() const
{
	return FLinearColor(0.0036765f, 0.3864294f, 0.2501584f);
}

FText UDataprepGraphActionStepNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static FText DummyTitle;
	return DummyTitle;
}

void UDataprepGraphActionStepNode::DestroyNode()
{
	if ( DataprepActionAsset )
	{
		Modify();

		//DataprepActionAsset->NotifyDataprepSystemsOfRemoval();

		// Force the transaction system to restore the action
		DataprepActionAsset = nullptr;
	}

	Super::DestroyNode();
}

UDataprepGraphActionGroupNode::UDataprepGraphActionGroupNode()
	: ExecutionOrder(INDEX_NONE)
{
	bCanRenameNode = false;
	NodeTitle = LOCTEXT("DefaultActionGroupNodeTitle", "Action Group").ToString();
}

void UDataprepGraphActionGroupNode::Initialize(TWeakObjectPtr<UDataprepAsset> InDataprepAssetPtr, TArray<UDataprepActionAsset*>& InActions, int32 InExecutionOrder) 
{
	DataprepAssetPtr = InDataprepAssetPtr;
	Actions = InActions;
	ExecutionOrder = InExecutionOrder;

	NodeWidth = 0;
	NodeHeight = 0;

	for ( UDataprepActionAsset* Action : Actions )
	{
		NodeWidth = FMath::Max( static_cast<int32>( Action->GetAppearance()->NodeSize.X ), NodeWidth );
		NodeHeight = FMath::Max( static_cast<int32>( Action->GetAppearance()->NodeSize.Y ), NodeHeight );
	}
}

FText UDataprepGraphActionGroupNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString( NodeTitle );
}

void UDataprepGraphActionGroupNode::ResizeNode(const FVector2f& NewSize)
{
	DataprepAssetPtr->Modify();

	// Resize grouped actions
	for ( UDataprepActionAsset* Action : Actions )
	{
		Action->Modify();
		Action->GetAppearance()->Modify();
		Action->GetAppearance()->NodeSize = NewSize;
	}
}

TSharedPtr<class INameValidatorInterface> UDataprepGraphActionGroupNode::MakeNameValidator() const
{
	// The name doesn't matter
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

int32 UDataprepGraphActionGroupNode::GetGroupId() const 
{
	if (Actions.Num() == 0)
	{
		return INDEX_NONE;
	}
	return Actions[0]->GetAppearance()->GroupId;
}

bool UDataprepGraphActionGroupNode::IsGroupEnabled() const 
{
	if (Actions.Num() == 0)
	{
		return true;
	}
	return Actions[0]->GetAppearance()->bGroupIsEnabled;
}

#undef LOCTEXT_NAMESPACE
