// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"

#include "DataprepGraphActionNode.generated.h"


class FBlueprintActionDatabaseRegistrar;
class UDataprepActionAsset;
class UDataprepActionStep;
class UDataprepAsset;

/**
 * The UDataprepGraphActionStepNode class is used as the UEdGraphNode associated
 * to an SGraphNode in order to display the action's steps in a SDataprepGraphEditor.
 */
UCLASS(MinimalAPI)
class UDataprepGraphActionStepNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	UDataprepGraphActionStepNode();

	// Begin EdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void DestroyNode() override;
	// End EdGraphNode interface


	void Initialize(UDataprepActionAsset* InDataprepActionAsset, int32 InStepIndex)
	{
		DataprepActionAsset = InDataprepActionAsset;
		StepIndex = InStepIndex;
	}

	const UDataprepActionAsset* GetDataprepActionAsset() const { return DataprepActionAsset; }
	UDataprepActionAsset* GetDataprepActionAsset() { return DataprepActionAsset; }

	int32 GetStepIndex() const { return StepIndex; }

	const UDataprepActionStep* GetDataprepActionStep() const;
	UDataprepActionStep* GetDataprepActionStep();

protected:
	UPROPERTY()
	TObjectPtr<UDataprepActionAsset> DataprepActionAsset;

	UPROPERTY()
	int32 StepIndex;

	// Is this node currently driving the filter preview
	bool bIsPreviewed = false;
};

/**
 * The UDataprepGraphActionNode class is used as the UEdGraphNode associated
 * to an SGraphNode in order to display actions in a SDataprepGraphEditor.
 */
UCLASS(MinimalAPI)
class UDataprepGraphActionNode final : public UEdGraphNode
{
	GENERATED_BODY()

public:
	UDataprepGraphActionNode();

	// Begin EdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void DestroyNode() override;
	TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual void ResizeNode(const FVector2f& NewSize) override;
	// End EdGraphNode interface

	void Initialize(TWeakObjectPtr<UDataprepAsset> InDataprepAssetPtr, UDataprepActionAsset* InDataprepActionAsset, int32 InExecutionOrder);

	const UDataprepActionAsset* GetDataprepActionAsset() const { return DataprepActionAsset; }
	UDataprepActionAsset* GetDataprepActionAsset() { return DataprepActionAsset; }

	int32 GetExecutionOrder() const { return ExecutionOrder; }

protected:
	UPROPERTY()
	FString ActionTitle;

	UPROPERTY()
	TObjectPtr<UDataprepActionAsset> DataprepActionAsset;

	UPROPERTY()
	TWeakObjectPtr<UDataprepAsset> DataprepAssetPtr;

	UPROPERTY()
	int32 ExecutionOrder;
};

UCLASS(MinimalAPI)
class UDataprepGraphActionGroupNode final : public UEdGraphNode
{
	GENERATED_BODY()

public:
	UDataprepGraphActionGroupNode();

	// Begin EdGraphNode interface
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual void ResizeNode(const FVector2f& NewSize) override;
	// End EdGraphNode interface

	void Initialize(TWeakObjectPtr<UDataprepAsset> InDataprepAssetPtr, TArray<UDataprepActionAsset*>& InActions, int32 InExecutionOrder);

	int32 GetExecutionOrder() const { return ExecutionOrder; }

	int32 GetGroupId() const;

	bool IsGroupEnabled() const;

	UDataprepActionAsset* GetAction(int32 Index) const
	{
		check(Index < Actions.Num());
		return Actions[Index];
	}

	UDataprepAsset* GetDataprepAsset()
	{
		return DataprepAssetPtr.Get();
	}

	int32 GetActionsCount() const
	{
		return Actions.Num();
	}

protected:
	UPROPERTY()
	int32 ExecutionOrder;

	UPROPERTY()
	FString NodeTitle;

	UPROPERTY()
	TArray<TObjectPtr<UDataprepActionAsset>> Actions;

	UPROPERTY()
	TWeakObjectPtr<UDataprepAsset> DataprepAssetPtr;
};
