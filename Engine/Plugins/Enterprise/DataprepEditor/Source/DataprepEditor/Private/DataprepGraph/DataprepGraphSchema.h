// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "EdGraph/EdGraphSchema.h"

#include "DataprepGraphSchema.generated.h"

UCLASS()
class UDataprepGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	// UEdGraphSchema interface
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override {}
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	//virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	// End of UEdGraphSchema interface
};
