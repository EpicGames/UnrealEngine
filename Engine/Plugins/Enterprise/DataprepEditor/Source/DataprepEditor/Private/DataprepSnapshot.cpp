// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditor.h"

#include "DataprepCoreUtils.h"
#include "DataprepEditorLogCategory.h"

#include "ActorEditorUtils.h"
#include "Async/Async.h"
#include "Engine/Level.h"
#include "Engine/Texture.h"
#include "Engine/StaticMesh.h"
#include "Exporters/Exporter.h"
#include "Factories/LevelFactory.h"
#include "GameFramework/WorldSettings.h"
#include "Materials/MaterialFunction.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialInstance.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "UnrealExporter.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DataprepEditor"

enum class EDataprepAssetClass : uint8 {
	EDataprep,
	ETexture,
	EMaterialFunction,
	EMaterialFunctionInstance,
	EMaterial,
	EMaterialInstance,
	EStaticMesh,
	EOther,
	EMaxClasses
};

namespace DataprepSnapshotUtil
{
	typedef TArray<uint8, TSizedDefaultAllocator<64>> FSnapshotBuffer;

	const TCHAR* SnapshotExtension = TEXT(".dpc");

	// #ueent_todo: Remove those temporary fixes for large data saving/loading with implementation from Core
	class FSnapshotWriter : public FMemoryArchive
	{
	public:
		FSnapshotWriter( FSnapshotBuffer& InBytes, bool bIsPersistent = false, bool bSetOffset = false, const FName InArchiveName = NAME_None )
			: FMemoryArchive()
			, Bytes(InBytes)
			, ArchiveName(InArchiveName)
		{
			this->SetIsSaving(true);
			this->SetIsPersistent(bIsPersistent);
			if (bSetOffset)
			{
				Offset = InBytes.Num();
			}
		}

		virtual void Serialize(void* Data, int64 Num) override
		{
			const int64 NumBytesToAdd = Offset + Num - Bytes.Num();
			if( NumBytesToAdd > 0 )
			{
				const int64 NewArrayCount = Bytes.Num() + NumBytesToAdd;
				Bytes.AddUninitialized( NumBytesToAdd );
			}

			check((Offset + Num) <= Bytes.Num());

			if( Num )
			{
				FMemory::Memcpy( &Bytes[Offset], Data, Num );
				Offset+=Num;
			}
		}
		/**
		* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
		* is in when a loading error occurs.
		*
		* This is overridden for the specific Archive Types
		**/
		virtual FString GetArchiveName() const override { return TEXT("FSnapshotWriter"); }

		int64 TotalSize() override
		{
			return Bytes.Num();
		}

	protected:

		FSnapshotBuffer&	Bytes;

		/** Archive name, used to debugging, by default set to NAME_None. */
		const FName ArchiveName;
	};

	class FSnapshotReader : public FMemoryArchive
	{
	public:
		/**
		* Returns the name of the Archive.  Useful for getting the name of the package a struct or object
		* is in when a loading error occurs.
		*
		* This is overridden for the specific Archive Types
		**/
		virtual FString GetArchiveName() const override
		{
			return TEXT("FSnapshotReader");
		}

		virtual int64 TotalSize() override
		{
			return FMath::Min(Bytes.Num(), LimitSize);
		}

		void Serialize( void* Data, int64 Num )
		{
			if (Num && !ArIsError)
			{
				// Only serialize if we have the requested amount of data
				if (Offset + Num <= TotalSize())
				{
					FMemory::Memcpy( Data, &Bytes[Offset], Num );
					Offset += Num;
				}
				else
				{
					ArIsError = true;
				}
			}
		}

		explicit FSnapshotReader( const FSnapshotBuffer& InBytes, bool bIsPersistent = false )
			: Bytes    (InBytes)
			, LimitSize(INT64_MAX)
		{
			this->SetIsLoading(true);
			this->SetIsPersistent(bIsPersistent);
		}

		/** With this method it's possible to attach data behind some serialized data. */
		void SetLimitSize(int64 NewLimitSize)
		{
			LimitSize = NewLimitSize;
		}

	private:
		const FSnapshotBuffer&	Bytes;
		int64					LimitSize;
	};

	/**
	 * Extends FObjectAndNameAsStringProxyArchive to support FLazyObjectPtr.
	 */
	struct FSnapshotCustomArchive : public FObjectAndNameAsStringProxyArchive
	{
		FSnapshotCustomArchive(FArchive& InInnerArchive)
			:	FObjectAndNameAsStringProxyArchive(InInnerArchive, false)
		{
			// Set archive as transacting to persist all data including data in memory
			SetIsTransacting( true );
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Obj) override
		{
			// Copied from FArchiveUObject::SerializeLazyObjectPtr
			// Note that archive is transacting
			if (IsLoading())
			{
				// Reset before serializing to clear the internal weak pointer. 
				Obj.Reset();
			}
			InnerArchive << Obj.GetUniqueID();

			return *this;
		}
	};

	void RemoveSnapshotFiles(const FString& RootDir)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DataprepSnapshotUtil::RemoveSnapshotFiles);

		TArray<FString> FileNames;
		IFileManager::Get().FindFiles( FileNames, *RootDir, SnapshotExtension );
		for ( FString& FileName : FileNames )
		{
			IFileManager::Get().Delete( *FPaths::Combine( RootDir, FileName ), false );
		}

	}

	FString BuildAssetFileName(const FString& RootPath, const FString& AssetPath )
	{
		static FString FileNamePrefix( TEXT("stream_") );

		FString PackageFileName = FileNamePrefix + FString::Printf( TEXT("%08x"), GetTypeHash( AssetPath ) );
		return FPaths::ConvertRelativePathToFull( FPaths::Combine( RootPath, PackageFileName ) + SnapshotExtension );
	}

	// #ueent_dataprep: Revisit serialization of snapshot in 5.0
	void SerializeObject(FSnapshotCustomArchive& Ar, UObject* Object, TArray< UObject* >& SubObjects)
	{
		bool bIsStaticMesh = Cast<UStaticMesh>(Object) != nullptr;

		// Overwrite persistent flag if Object is a static mesh.
		// This will skip the persistence of the MeshDescription on static mesh and geometrical sub-objects

		// Serialize SubObjects' content
		for(UObject* SubObject : SubObjects)
		{
			if (SubObject && (!bIsStaticMesh || !SubObject->HasAnyFlags(RF_DefaultSubObject)))
			{
				// Do not set the archive as persistent if the subobject is mesh description's bulk data.
				// The combination of transient object and persistence is not accepted by such object during serialization
				Ar.SetIsPersistent(bIsStaticMesh && SubObject->StaticClass()->IsChildOf(UMeshDescriptionBaseBulkData::StaticClass()));
				SubObject->Serialize(Ar);
			}
		}

		// Overwrite transacting flag if Object is a static mesh.
		// This will skip the persistence of the MeshDescription
		// Set archive as persistent if dealing with a static mesh. Otherwise nothingis serialized
		Ar.SetIsPersistent(bIsStaticMesh);
		Ar.SetIsTransacting(!bIsStaticMesh);

		// Serialize object
		Object->Serialize(Ar);
	}

	void WriteSnapshotData(UObject* Object, FSnapshotBuffer& OutSerializedData)
	{
		// Helper struct to identify dependency of a UObject on other UObject(s) except given one (its outer)
		struct FObjectDependencyAnalyzer : public FArchiveUObject
		{
			FObjectDependencyAnalyzer(UObject* InSourceObject, const TSet<UObject*>& InValidObjects)
				: SourceObject( InSourceObject )
				, ValidObjects( InValidObjects )
			{ }

			virtual FArchive& operator<<(UObject*& Obj) override
			{
				if(Obj != nullptr)
				{
					// Limit serialization to sub-object of source object
					if( Obj == SourceObject->GetOuter() || Obj->IsA<UPackage>() || (Obj->HasAnyFlags(RF_Public) && Obj->GetOuter()->IsA<UPackage>()))
					{
						return FArchiveUObject::operator<<(Obj);
					}
					// Stop serialization when a dependency is found or has been found
					else if( Obj != SourceObject && !DependentObjects.Contains( Obj ) && ValidObjects.Contains( Obj ) )
					{
						DependentObjects.Add( Obj );
					}
				}

				return *this;
			}

			UObject* SourceObject;
			const TSet<UObject*>& ValidObjects;
			TSet<UObject*> DependentObjects;
		};

		FSnapshotWriter MemAr(OutSerializedData);
		FSnapshotCustomArchive Ar(MemAr);

		// Collect sub-objects depending on input object including nested objects
		TArray< UObject* > SubObjectsArray;
		GetObjectsWithOuter( Object, SubObjectsArray, EGetObjectsFlags::IncludeNestedObjects);

		// Sort array of sub-objects based on their inter-dependency
		{
			// Create and initialize graph of dependency between sub-objects
			TMap< UObject*, TSet<UObject*> > SubObjectDependencyGraph;
			SubObjectDependencyGraph.Reserve( SubObjectsArray.Num() );

			for(UObject* SubObject : SubObjectsArray)
			{
				SubObjectDependencyGraph.Add( SubObject );
			}

			// Build graph of dependency: each entry contains the set of sub-objects to create before itself
			TSet<UObject*> SubObjectsSet(SubObjectsArray);
			for(UObject* SubObject : SubObjectsArray)
			{
				FObjectDependencyAnalyzer Analyzer( SubObject, SubObjectsSet );
				SubObject->Serialize( Analyzer );

				if (!Analyzer.DependentObjects.IsEmpty())
				{
					TSet<UObject*>& SubObjectEntry = SubObjectDependencyGraph.FindOrAdd(SubObject);
					for (UObject* DependentObject : Analyzer.DependentObjects)
					{
						// Skip cyclical dependency
						if (!SubObjectDependencyGraph.Contains(DependentObject))
						{
							SubObjectEntry.Add(DependentObject);
						}
					}
				}
			}

			// Sort array of sub-objects: first objects do not depend on ones below
			int32 Count = SubObjectsArray.Num();
			SubObjectsArray.Empty(Count);

			while(Count != SubObjectsArray.Num())
			{
				for(auto& Entry : SubObjectDependencyGraph)
				{
					if(Entry.Value.Num() == 0)
					{
						UObject* SubObject = Entry.Key;

						SubObjectDependencyGraph.Remove( SubObject );

						SubObjectsArray.Add(SubObject);

						bool bNoEndlessLoop = SubObjectDependencyGraph.IsEmpty();
						for(auto& SubEntry : SubObjectDependencyGraph)
						{
							if(SubEntry.Value.Num() > 0)
							{
								SubEntry.Value.Remove(SubObject);
							}

							bNoEndlessLoop |= SubEntry.Value.Num() == 0;
						}

						ensure(bNoEndlessLoop);

						break;
					}
				}
			}
		}

		// Serialize size of array
		int32 SubObjectsCount = SubObjectsArray.Num();
		MemAr << SubObjectsCount;

		// Serialize class and path of each sub-object
		for (int32 Index = 0; Index < SubObjectsArray.Num(); ++Index)
		{
			const UObject* SubObject = SubObjectsArray[Index];

			UClass* SubObjectClass = SubObject->GetClass();

			FString ClassName = SubObjectClass->GetPathName();
			MemAr << ClassName;

			int32 ObjectFlags = SubObject->GetFlags();
			MemAr << ObjectFlags;
		}

		// Serialize sub-objects' outer path and name
		// Done in reverse order since a sub-object can be the outer of another sub-object
		// it depends on. Not the opposite
		for (int32 Index = 0; Index < SubObjectsArray.Num(); ++Index)
		{
			const UObject* SubObject = SubObjectsArray[Index];

			FSoftObjectPath SoftPath( SubObject->GetOuter() );

			FString SoftPathString = SoftPath.ToString();
			MemAr << SoftPathString;

			FString SubObjectName = SubObject->GetName();
			MemAr << SubObjectName;
		}

		SerializeObject(Ar, Object, SubObjectsArray);
	}

	void ReadSnapshotData(UObject* Object, const FSnapshotBuffer& InSerializedData, TMap<FString, UClass*>& InClassesMap, TArray<UObject*>& ObjectsToDelete)
	{
		bool bIsStaticMesh = Cast<UStaticMesh>(Object) != nullptr;

		// Remove all objects created by default that InObject is dependent on
		// This method must obviously be called just after the InObject is created
		auto RemoveDefaultDependencies = [&ObjectsToDelete, bIsStaticMesh](UObject* InObject)
		{
			TArray< UObject* > ObjectsWithOuter;
			GetObjectsWithOuter( InObject, ObjectsWithOuter, EGetObjectsFlags::IncludeNestedObjects);

			for(UObject* ObjectWithOuter : ObjectsWithOuter)
			{
				// Do not delete default sub-objects
				if (!bIsStaticMesh || !ObjectWithOuter->HasAnyFlags(RF_DefaultSubObject))
				{
					FDataprepCoreUtils::MoveToTransientPackage( ObjectWithOuter );
					ObjectsToDelete.Add( ObjectWithOuter );
				}
			}
		};

		RemoveDefaultDependencies( Object );

		FSnapshotReader MemAr(InSerializedData);
		FSnapshotCustomArchive Ar(MemAr);

		// Deserialize count of sub-objects
		int32 SubObjectsCount = 0;
		MemAr << SubObjectsCount;

		// Create empty sub-objects based on class and patch
		TArray< UObject* > SubObjectsArray;
		SubObjectsArray.SetNumZeroed(SubObjectsCount);

		// Create root name to avoid name collision
		FString RootName = FGuid::NewGuid().ToString();

		for (int32 Index = 0; Index < SubObjectsArray.Num(); ++Index)
		{
			FString ClassName;
			MemAr << ClassName;

			UClass* SubObjectClass = UClass::TryFindTypeSlow<UClass>(ClassName);
			check( SubObjectClass );

			int32 ObjectFlags;
			MemAr << ObjectFlags;

			// Temporary - Do not create default sub-object for static mesh
			// #ueent_dataprep: TODO: Implement a more robust serialization process for transient UProperties
			if (bIsStaticMesh && (ObjectFlags & RF_DefaultSubObject))
			{
				continue;
			}

			FString SubObjectName = RootName + FString::FromInt(Index);

			UObject* SubObject = NewObject<UObject>( Object, SubObjectClass, *SubObjectName, EObjectFlags(ObjectFlags) );
			SubObjectsArray[Index] = SubObject;

			RemoveDefaultDependencies( SubObject );
		}

		// Restore sub-objects' outer if original outer differs from Object
		// Restore sub-objects' name 
		for(int32 Index = 0; Index < SubObjectsArray.Num(); ++Index)
		{
			FString SoftPathString;
			MemAr << SoftPathString;

			FString SubObjectName;
			MemAr << SubObjectName;

			const FSoftObjectPath SoftPath( SoftPathString );

			UObject* NewOuter = SoftPath.ResolveObject();
			ensure( NewOuter );

			if (UObject* SubObject = SubObjectsArray[Index])
			{
				if( NewOuter != SubObject->GetOuter() )
				{
					FDataprepCoreUtils::RenameObject( SubObject, nullptr, NewOuter );
				}

				if( SubObjectName != SubObject->GetName() && SubObject->Rename( *SubObjectName, nullptr, REN_Test ))
				{
					FDataprepCoreUtils::RenameObject( SubObject, *SubObjectName );
				}
			}
		}

		SerializeObject(Ar, Object, SubObjectsArray);

		if(UTexture* Texture = Cast<UTexture>(Object))
		{
			Texture->UpdateResource();
		}
	}
}

class FDataprepExportObjectInnerContext : public FExportObjectInnerContext
{
public:
	FDataprepExportObjectInnerContext( UWorld* World)
		//call the empty version of the base class
		: FExportObjectInnerContext(false)
	{
		// For each object . . .
		for ( TObjectIterator<UObject> It; It ; ++It )
		{
			UObject* InnerObj = *It;
			UObject* OuterObj = InnerObj->GetOuter();

			// By default assume object does not need to be copied
			bool bObjectMustBeCopied = false;

			UObject* TestParent = OuterObj;
			while (TestParent)
			{
				AActor* TestParentAsActor = Cast<AActor>(TestParent);

				const bool bIsValidActor = IsValid(TestParentAsActor) &&
					TestParentAsActor->GetWorld() == World &&
					TestParentAsActor->IsEditable() &&
					!TestParentAsActor->IsTemplate() &&
					!FActorEditorUtils::IsABuilderBrush(TestParentAsActor) &&
					!TestParentAsActor->IsA(AWorldSettings::StaticClass());

				if ( bIsValidActor )
				{
					// Track actor so it will be processed during the copy
					SelectedActors.Add(TestParentAsActor);

					bObjectMustBeCopied = true;
					break;
				}

				TestParent = TestParent->GetOuter();
			}

			if (bObjectMustBeCopied)
			{
				InnerList* Inners = ObjectToInnerMap.Find( OuterObj );
				if ( Inners )
				{
					// Add object to existing inner list.
					Inners->Add( InnerObj );
				}
				else
				{
					// Create a new inner list for the outer object.
					InnerList& InnersForOuterObject = ObjectToInnerMap.Add( OuterObj, InnerList() );
					InnersForOuterObject.Add( InnerObj );
				}
			}
		}
	}

	virtual bool IsObjectSelected(const UObject* InObj) const override
	{
		const AActor* Actor = Cast<AActor>(InObj);
		return Actor && SelectedActors.Contains(Actor);
	}

	virtual int32 GetObjectNumber() const override { return SelectedActors.Num(); }

	/** Set of actors marked as selected so they get included in the copy */
	TSet<const AActor*> SelectedActors;
};

void FDataprepEditor::TakeSnapshot()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::TakeSnapshot);

	uint64 StartTime = FPlatformTime::Cycles64();
	UE_LOGF( LogDataprepEditor, Verbose, "Taking snapshot..." );

	FScopedSlowTask SlowTask( 100.0f, LOCTEXT("SaveSnapshot_Title", "Creating snapshot of world content ...") );
	SlowTask.MakeDialog(false);

	// Clean up temporary folder with content of previous snapshot(s)
	SlowTask.EnterProgressFrame( 10.0f, LOCTEXT("SaveSnapshot_Cleanup", "Snapshot : Cleaning previous content ...") );
	{
		DataprepSnapshotUtil::RemoveSnapshotFiles( TempDir );
		ContentSnapshot.DataEntries.Empty( Assets.Num() );
		SnapshotClassesMap.Reset();
	}

	// Sort assets to serialize and deserialize them according to their dependency
	// Texture first, then Material, then ...
	// Note: This classification must be updated as type of assets are added
	auto GetAssetClassEnum = [&](const UClass* AssetClass)
	{
		if (AssetClass->IsChildOf(UStaticMesh::StaticClass()))
		{
			return EDataprepAssetClass::EStaticMesh;
		}
		else if (AssetClass->IsChildOf(UMaterialFunction::StaticClass()))
		{
			return EDataprepAssetClass::EMaterialFunction;
		}
		else if (AssetClass->IsChildOf(UMaterialFunctionInstance::StaticClass()))
		{
			return EDataprepAssetClass::EMaterialFunctionInstance;
		}
		else if (AssetClass->IsChildOf(UMaterial::StaticClass()))
		{
			return EDataprepAssetClass::EMaterial;
		}
		else if (AssetClass->IsChildOf(UMaterialInstance::StaticClass()))
		{
			return EDataprepAssetClass::EMaterialInstance;
		}
		else if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			return EDataprepAssetClass::ETexture;
		}

		return EDataprepAssetClass::EOther;
	};

	Assets.Sort([&](const TWeakObjectPtr<UObject>& A, const TWeakObjectPtr<UObject>& B)
	{
		EDataprepAssetClass AValue = A.IsValid() ? GetAssetClassEnum(A->GetClass()) : EDataprepAssetClass::EMaxClasses;
		EDataprepAssetClass BValue = B.IsValid() ? GetAssetClassEnum(B->GetClass()) : EDataprepAssetClass::EMaxClasses;
		return AValue < BValue;
	});

	// Cache the asset's path, class and flags
	for( const TWeakObjectPtr<UObject>& AssetPtr : Assets )
	{
		if(UObject* AssetObject = AssetPtr.Get())
		{
			FSoftObjectPath AssetPath( AssetObject );
			ContentSnapshot.DataEntries.Emplace(AssetPath.GetAssetPathString(), AssetObject->GetClass(), AssetObject->GetFlags());
		}
	}

	TAtomic<bool> bGlobalIsValid(true);
	{
		FText Message = LOCTEXT("SaveSnapshot_SaveAssets", "Snapshot : Caching assets ...");
		SlowTask.EnterProgressFrame( 40.0f, Message );

		FScopedSlowTask SlowSaveAssetTask( (float)Assets.Num(), Message );
		SlowSaveAssetTask.MakeDialog(false);

		TArray<TFuture<bool>> AsyncTasks;
		AsyncTasks.Reserve( Assets.Num() );

		for (TWeakObjectPtr<UObject> AssetObjectPtr : Assets)
		{
			AsyncTasks.Emplace(
				Async(
					EAsyncExecution::LargeThreadPool,
					[this, AssetObjectPtr, &bGlobalIsValid]()
					{
						if(UObject* AssetObject = AssetObjectPtr.Get())
						{
							if ( bGlobalIsValid.Load(EMemoryOrder::Relaxed) )
							{
								EObjectFlags ObjectFlags = AssetObject->GetFlags();
								AssetObject->ClearFlags(RF_Transient);
								AssetObject->SetFlags(RF_Public);

								FSoftObjectPath AssetPath( AssetObject );
								const FString AssetPathString = AssetPath.GetAssetPathString();
								UE_LOGF( LogDataprepEditor, Verbose, "Saving asset %ls", *AssetPathString );

								bool bLocalIsValid = false;

								// Serialize asset
								{
									DataprepSnapshotUtil::FSnapshotBuffer SerializedData;
									DataprepSnapshotUtil::WriteSnapshotData( AssetObject, SerializedData );

									FString AssetFilePath = DataprepSnapshotUtil::BuildAssetFileName( this->TempDir, AssetPathString );

									bLocalIsValid = FFileHelper::SaveArrayToFile( SerializedData, *AssetFilePath );
								}

								// Ensure DefaultSubojbect flag persists through the clearing of flags
								if (AssetObject->HasAllFlags(RF_DefaultSubObject))
								{
									ObjectFlags |= RF_DefaultSubObject;
								}

								AssetObject->ClearFlags( RF_AllFlags );
								AssetObject->SetFlags( ObjectFlags );

								return bLocalIsValid;
							}
							else
							{
								return false;
							}
						}

						return true;
					}
				)
			);
		}

		for (int32 Index = 0; Index < AsyncTasks.Num(); ++Index)
		{
			if(UObject* AssetObject = Assets[Index].Get())
			{
				SlowSaveAssetTask.EnterProgressFrame();

				const FSoftObjectPath AssetPath( AssetObject );
				const FString AssetPathString = AssetPath.GetAssetPathString();

				// Wait the result of the async task
				if(!AsyncTasks[Index].Get())
				{
					UE_LOGF( LogDataprepEditor, Log, "Failed to save %ls", *AssetPathString );

					bGlobalIsValid = false;
					break;
				}
				else
				{
					UE_LOGF( LogDataprepEditor, Verbose, "Asset %ls successfully saved", *AssetPathString );
				}
			}
		}
	}

	ContentSnapshot.bIsValid = bGlobalIsValid;

	// Serialize world if applicable
	if(ContentSnapshot.bIsValid)
	{
		FText Message = LOCTEXT("SaveSnapshot_World", "Snapshot : caching level ...");
		SlowTask.EnterProgressFrame( 50.0f, Message );
		UE_LOGF( LogDataprepEditor, Verbose, "Saving preview world" );

		FScopedSlowTask SlowSaveAssetTask( (float)PreviewWorld->GetCurrentLevel()->Actors.Num(), Message );
		SlowSaveAssetTask.MakeDialog(false);

		PreviewWorld->ClearFlags(RF_Transient);
		{
			// Code inspired from UUnrealEdEngine::edactCopySelected
			FStringOutputDevice Ar;
			uint32 ExportFlags = PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified | PPF_IncludeTransient;
			const FDataprepExportObjectInnerContext Context( PreviewWorld );
			UExporter::ExportToOutputDevice( &Context, PreviewWorld, nullptr, Ar, TEXT("copy"), 0, ExportFlags);

			// Save text into file
			FString PackageFilePath = DataprepSnapshotUtil::BuildAssetFileName( TempDir, GetTransientContentFolder() / SessionID ) + TEXT(".asc");
			ContentSnapshot.bIsValid &= FFileHelper::SaveStringToFile( Ar, *PackageFilePath );
		}
		PreviewWorld->SetFlags(RF_Transient);

		if(ContentSnapshot.bIsValid)
		{
			UE_LOGF( LogDataprepEditor, Verbose, "Level successfully saved" );
		}
		else
		{
			UE_LOGF( LogDataprepEditor, Warning, "Failed to save level" );
		}
	}

	if(!ContentSnapshot.bIsValid)
	{
		DataprepSnapshotUtil::RemoveSnapshotFiles( TempDir );
		ContentSnapshot.DataEntries.Empty();
		return;
	}

	ContentSnapshot.DataEntries.Sort([&](const FSnapshotDataEntry& A, const FSnapshotDataEntry& B)
	{
		return GetAssetClassEnum( A.Get<1>() ) < GetAssetClassEnum( B.Get<1>() );
	});

	// Log time spent to import incoming file in minutes and seconds
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	UE_LOGF( LogDataprepEditor, Verbose, "Snapshot taken in [%d min %.3f s]", ElapsedMin, ElapsedSeconds );
}

void FDataprepEditor::RestoreFromSnapshot(bool bUpdateViewport)
{
	// Snapshot is not usable, rebuild the world from the producers
	if ( !ContentSnapshot.bIsValid )
	{
		UE_LOGF( LogDataprepEditor, Log, "Snapshot is invalid. Running the producers..." );
		OnBuildWorld();
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDataprepEditor::RestoreFromSnapshot);

	// Clean up all assets and world content
	{
		CleanPreviewWorld();
		Assets.Reset(ContentSnapshot.DataEntries.Num());
	}

	FScopedSlowTask SlowTask( 100.0f, LOCTEXT("RestoreFromSnapshot_Title", "Restoring world initial content ...") );
	SlowTask.MakeDialog(false);

	uint64 StartTime = FPlatformTime::Cycles64();
	UE_LOGF( LogDataprepEditor, Verbose, "Restoring snapshot..." );

	TMap<FString, UPackage*> PackagesCreated;
	PackagesCreated.Reserve(ContentSnapshot.DataEntries.Num());

	UPackage* RootPackage = NewObject< UPackage >( nullptr, *GetTransientContentFolder(), RF_Transient );
	RootPackage->FullyLoad();

	uint32 PortFlags = PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified | PPF_IncludeTransient | PPF_Copy;
	TArray<UObject*> ObjectsToDelete;

	SlowTask.EnterProgressFrame( 40.0f, LOCTEXT("RestoreFromSnapshot_Assets", "Restoring assets ...") );
	{
		TArray<UMaterialInterface*> MaterialInterfaces;

		FScopedSlowTask SubSlowTask( ContentSnapshot.DataEntries.Num(), LOCTEXT("RestoreFromSnapshot_Assets", "Restoring assets ...") );
		SubSlowTask.MakeDialog(false);

		for (FSnapshotDataEntry& DataEntry : ContentSnapshot.DataEntries)
		{
			const FSoftObjectPath ObjectPath( DataEntry.Get<0>() );
			const FString PackageToLoadPath = ObjectPath.GetLongPackageName();
			const FString AssetName = ObjectPath.GetAssetName();

			SubSlowTask.EnterProgressFrame( 1.0f, FText::Format( LOCTEXT("RestoreFromSnapshot_OneAsset", "Restoring asset {0}"), FText::FromString( ObjectPath.GetAssetName() ) ) );
			UE_LOGF( LogDataprepEditor, Verbose, "Loading asset %ls", *ObjectPath.GetAssetPathString() );

			if(PackagesCreated.Find(PackageToLoadPath) == nullptr)
			{
				UPackage* PackageCreated = NewObject< UPackage >( nullptr, *PackageToLoadPath, RF_Transient );
				PackageCreated->FullyLoad();
				PackageCreated->MarkPackageDirty();

				PackagesCreated.Add( PackageToLoadPath, PackageCreated );
			}

			// Duplicate sub-objects to delete after all assets are read
			UPackage* Package = PackagesCreated[PackageToLoadPath];
			UObject* Asset = NewObject<UObject>( Package, DataEntry.Get<1>(), *AssetName, DataEntry.Get<2>() );

			{
				DataprepSnapshotUtil::FSnapshotBuffer SerializedData;
				FString AssetFilePath = DataprepSnapshotUtil::BuildAssetFileName( TempDir, ObjectPath.GetAssetPathString() );
				if(FFileHelper::LoadFileToArray(SerializedData, *AssetFilePath))
				{
					DataprepSnapshotUtil::ReadSnapshotData( Asset, SerializedData, SnapshotClassesMap, ObjectsToDelete );
				}
				else
				{
					UE_LOGF( LogDataprepEditor, Error, "Failed to restore asset %ls", *ObjectPath.GetAssetPathString() );
				}
			}

			Assets.Add( Asset );

			UE_LOGF( LogDataprepEditor, Verbose, "Asset %ls loaded", *ObjectPath.GetAssetPathString() );
		}

		FDataprepCoreUtils::PurgeObjects( MoveTemp( ObjectsToDelete ) );
	}

	// Make sure all assets have RF_Public flag set so the actors in the level can find the assets they are referring to
	// Cache boolean representing if RF_Public flag was set or not on asset
	TArray<bool> AssetFlags;
	AssetFlags.AddDefaulted( Assets.Num() );

	for(int32 Index = 0; Index < Assets.Num(); ++Index)
	{
		if( UObject* Asset = Assets[Index].Get() )
		{
			AssetFlags[Index] = bool(Asset->GetFlags() & RF_Public);
			Asset->SetFlags( RF_Public );
		}
	}

	SlowTask.EnterProgressFrame( 60.0f, LOCTEXT("RestoreFromSnapshot_Level", "Restoring level ...") );
	{
		// Code inspired from UUnrealEdEngine::edactPasteSelected
		ULevel* WorldLevel = PreviewWorld->GetCurrentLevel();

		FString PackageFilePath = DataprepSnapshotUtil::BuildAssetFileName( TempDir, GetTransientContentFolder() / SessionID ) + TEXT(".asc");
		const bool bBSPAutoUpdate = GetDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate;
		GetMutableDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate = false;

		// Load the text file to a string
		FString FileBuffer;
		verify( FFileHelper::LoadFileToString(FileBuffer, *PackageFilePath) );

		// Set the GWorld to the preview world since ULevelFactory::FactoryCreateText uses GWorld
		UWorld* PrevGWorld = GWorld;
		GWorld = PreviewWorld;

		// Cache and disable recording of transaction
		TGuardValue<decltype(GEditor->Trans)> NormalTransactor( GEditor->Trans, nullptr );

		// Cache and disable warnings from LogExec because ULevelFactory::FactoryCreateText is pretty verbose on harmless warnings
		ELogVerbosity::Type PrevLogExecVerbosity = LogExec.GetVerbosity();
		LogExec.SetVerbosity( ELogVerbosity::Error );

		// Cache and disable Editor's selection
		TGuardValue<bool> EdSelectionLock( GEdSelectionLock, true );

		const TCHAR* Paste = *FileBuffer;
		ULevelFactory* Factory = NewObject<ULevelFactory>();
		Factory->FactoryCreateText( ULevel::StaticClass(), WorldLevel, WorldLevel->GetFName(), RF_Transactional, NULL, TEXT("paste"), Paste, Paste + FileBuffer.Len(), GWarn );

		// Restore LogExec verbosity
		LogExec.SetVerbosity( PrevLogExecVerbosity );

		// Reinstate old BSP update setting, and force a rebuild - any levels whose geometry has changed while pasting will be rebuilt
		GetMutableDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate = bBSPAutoUpdate;

		// Restore GWorld
		GWorld = PrevGWorld;
	}
	UE_LOGF( LogDataprepEditor, Verbose, "Level loaded" );

	// Restore RF_Public on each asset
	for(int32 Index = 0; Index < Assets.Num(); ++Index)
	{
		if( UObject* Asset = Assets[Index].Get() )
		{
			if( !AssetFlags[Index] )
			{
				Asset->ClearFlags( RF_Public );
			}
		}
	}

	{
		TSharedPtr< IDataprepProgressReporter > ProgressReporter( new FDataprepCoreUtils::FDataprepProgressUIReporter() );
		FDataprepCoreUtils::BuildAssets( Assets, ProgressReporter );
	}

	// Log time spent to import incoming file in minutes and seconds
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	UE_LOGF( LogDataprepEditor, Verbose, "Preview world restored in [%d min %.3f s]", ElapsedMin, ElapsedSeconds );

	// Update preview panels to reflect restored content
	UpdatePreviewPanels( bUpdateViewport );
}

#undef LOCTEXT_NAMESPACE
