// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepStats.h"

#include "Components/LightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/Level.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureCube.h"
#include "Model.h"
#include "LandscapeComponent.h"
#include "LandscapeProxy.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/NaniteResources.h"
#include "ReferencedAssetsUtils.h"
#include "UObject/UObjectIterator.h"

const FName FDataprepStats::StatNameTriangles(TEXT("Triangles"));
const FName FDataprepStats::StatNameVertices(TEXT("Vertices"));
const FName FDataprepStats::StatNameNaniteTriangles(TEXT("NaniteTriangles"));
const FName FDataprepStats::StatNameNaniteVertices(TEXT("NaniteVertices"));
const FName FDataprepStats::StatNameTextures(TEXT("Textures"));
const FName FDataprepStats::StatNameTextureSize(TEXT("TextureSize"));
const FName FDataprepStats::StatNameMeshes(TEXT("Meshes"));
const FName FDataprepStats::StatNameSkeletalMeshes(TEXT("SkeletalMeshes"));
const FName FDataprepStats::StatNameMaterials(TEXT("Materials"));
const FName FDataprepStats::StatNameLights(TEXT("Lights"));
const FName FDataprepStats::StatNameActors(TEXT("Actors"));
const FName FDataprepStats::StatNameActorComponents(TEXT("ActorComponents"));

class DataprepStatsGenerator : public FFindReferencedAssets
{
public:
	DataprepStatsGenerator(UWorld* InWorld) : World(InWorld)
	{
		check(World);
	}

	FDataprepStats GenerateStats()
	{
		// Reset stats
		Stats = FDataprepStats{};

		Textures.Empty();
		Materials.Empty();
		Meshes.Empty();
		SkeletalMeshes.Empty();
		Actors.Empty();
		Lights.Empty();

		BuildReferencingData();

		for (int32 RefIndex = 0; RefIndex < Referencers.Num(); RefIndex++)
		{
			const TSet<UObject*> &AssetList = Referencers[RefIndex].AssetList;

			// Look at each referenced asset
			for(TSet<UObject*>::TConstIterator SetIt(AssetList); SetIt; ++SetIt)
			{
				UObject* Asset = *SetIt;

				if (UTexture* CurrentTexture = Cast<UTexture>(Asset))
				{
					if (IsTextureValidForStats(CurrentTexture))
					{
						RegisterTexture(CurrentTexture, RefIndex);
					}
				}
				else if (UPrimitiveComponent* ReferencedComponent = Cast<UPrimitiveComponent>(Asset))
				{
					RegisterComponent(ReferencedComponent, RefIndex);
				}
			}
		}

		Stats.Set(FDataprepStats::StatNameActors, Actors.Num());
		Stats.Set(FDataprepStats::StatNameLights, Lights.Num());
		Stats.Set(FDataprepStats::StatNameMaterials, Materials.Num());
		Stats.Set(FDataprepStats::StatNameMeshes, Meshes.Num());
		Stats.Set(FDataprepStats::StatNameSkeletalMeshes, SkeletalMeshes.Num());
		Stats.Set(FDataprepStats::StatNameTextures, Textures.Num());

		return Stats;
	}

private:
	UWorld* World;

	FDataprepStats Stats;

	TSet<const UTexture*> Textures;
	TSet<const UMaterialInterface*> Materials;
	TSet<const UStaticMesh*> Meshes;
	TSet<const USkeletalMesh*> SkeletalMeshes;
	TSet<AActor*> Actors;
	TSet<ULightComponent*> Lights;

	void GetWorldActors()
	{
		Actors.Empty(Actors.Num());

		for (ULevel* Level : World->GetLevels())
		{
			for (AActor* Actor : Level->Actors)
			{
				const bool bIsValidActor = IsValid(Actor);

				if (bIsValidActor)
				{
					Stats.AddCount(FDataprepStats::StatNameActorComponents, Actor->GetComponents().Num());
					Actors.Add( Actor );
				}
			}
		}
	}

	bool IsTextureValidForStats(const UTexture* Texture)
	{
		return Texture && (Texture->IsA( UTexture2D::StaticClass() ) || Texture->IsA( UTextureCube::StaticClass() ) );
	}

	void BuildReferencingData()
	{
		// Don't check for BSP mats if the list mode needs something to be selected
		{
			TSet<UObject*> BspMats;
			// materials to a temp list
			for (int32 Index = 0; Index < World->GetModel()->Surfs.Num(); Index++)
			{
				// No point showing the default material
				if (World->GetModel()->Surfs[Index].Material != NULL)
				{
					BspMats.Add(World->GetModel()->Surfs[Index].Material);
				}
			}
			// If any BSP surfaces are selected
			if (BspMats.Num() > 0)
			{
				FReferencedAssets* Referencer = new(Referencers) FReferencedAssets(World->GetModel());

				// Now copy the array
				Referencer->AssetList = BspMats;
				ReferenceGraph.Add(World->GetModel(), ObjectPtrWrap(BspMats));
			}
		}

		if (World->GetOutermost() == GetTransientPackage())
		{
			// Do not ignore the transient package as our world lives there.
			IgnorePackages.Remove(GetTransientPackage());
		}

		// this is the maximum depth to use when searching for references
		const int32 MaxRecursionDepth = 0;

		// Mark all objects so we don't get into an endless recursion
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			// Skip the level, world, and any packages that should be ignored
			if ( ShouldSearchForAssets(*It, ObjectPtrDecay(IgnoreClasses), ObjectPtrDecay(IgnorePackages), false) )
			{
				It->Mark(OBJECTMARK_TagExp);
			}
			else
			{
				It->UnMark(OBJECTMARK_TagExp);
			}
		}

		// Get the objects to search for texture references
		GetWorldActors();

		for( AActor* Actor : Actors )
		{
			// Create a new entry for this actor
			FReferencedAssets* Referencer = new(Referencers) FReferencedAssets(Actor);

			// Add to the list of referenced assets
			FFindAssetsArchive(Actor, Referencer->AssetList, &ReferenceGraph, MaxRecursionDepth, false, false);
		}
	}

	void GetWorldLights()
	{
		for (TObjectIterator<ULightComponent> LightIt; LightIt; ++LightIt)
		{
			ULightComponent* const Light = *LightIt;

			const bool bLightIsInWorld = Light->GetOwner() && !Light->GetOwner()->HasAnyFlags(RF_ClassDefaultObject) && World->ContainsActor(Light->GetOwner());
			if (bLightIsInWorld)
			{
				if (Light->HasStaticLighting() || Light->HasStaticShadowing())
				{
					// Add the light to the system's list of lights in the world.
					Lights.Add(Light);
				}
			}
		}
	}

	void RegisterMaterial(const UMaterialInterface* InMaterial, int InReferencerIndex)
	{
		TArray<UTexture*> UsedTextures;

		InMaterial->GetUsedTextures(UsedTextures, GetCurrentMaterialQualityLevelChecked());
		for (int32 TextureIndex = 0; TextureIndex < UsedTextures.Num(); TextureIndex++)
		{
			UTexture* CurrentUsedTexture = UsedTextures[TextureIndex];

			if (IsTextureValidForStats(CurrentUsedTexture))
			{
				RegisterTexture(CurrentUsedTexture, InReferencerIndex);
			}
		}

		Materials.Add(InMaterial);
	}

	void RegisterTexture(const UTexture* InTexture, int InReferencerIndex)
	{
		const UTexture2D* Texture2D = Cast<const UTexture2D>(InTexture);
		if (Texture2D)
		{
			Stats.Set(FDataprepStats::StatNameTextureSize, 
					  FMath::Max(Stats.Get(FDataprepStats::StatNameTextureSize), FMath::Max(Texture2D->GetSizeX(), Texture2D->GetSizeY())));
		}

		Textures.Add(InTexture);
	}

	void RegisterComponent(UPrimitiveComponent* InComponent, int InReferencerIndex)
	{
		// If the referenced asset is a primitive component get the materials used by the component
		TArray<UMaterialInterface*> UsedMaterials;
		InComponent->GetUsedMaterials(UsedMaterials);
		for (int32 MaterialIndex = 0; MaterialIndex < UsedMaterials.Num(); MaterialIndex++)
		{
			// For each material, find the textures used by that material and add it to the stat list
			const UMaterialInterface* CurrentMaterial = UsedMaterials[MaterialIndex];
			if (CurrentMaterial)
			{
				RegisterMaterial(CurrentMaterial, InReferencerIndex);
			}
		}

		// Transient objects are not part of level, but if part of transient package, we can allow depending on flag bAllowTransientWorld.
		if( InComponent->HasAnyFlags( RF_Transient ) )
		{
			return; //Result;
		}

		// Owned by a default object? Not part of a level either.
		if(InComponent->GetOuter() && InComponent->GetOuter()->IsTemplate())
		{
			return; // Result;
		}

		UStaticMeshComponent*			StaticMeshComponent				= Cast<UStaticMeshComponent>(InComponent);
		UInstancedStaticMeshComponent*	InstancedStaticMeshComponent	= Cast<UInstancedStaticMeshComponent>(InComponent);
		USkeletalMeshComponent*			SkeletalMeshComponent			= Cast<USkeletalMeshComponent>(InComponent);
		ULandscapeComponent*			LandscapeComponent				= Cast<ULandscapeComponent>(InComponent);

		// If we should skip the actor. Skip if the actor has no outer or if we are only showing selected actors and the actor isn't selected
		// Dont' care about components without a resource.
		if (InComponent->GetWorld() == World && IsValidChecked(InComponent))
		{
			if( StaticMeshComponent )
			{
				UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

				if( StaticMesh && StaticMesh->GetRenderData())
				{
					Meshes.Add(StaticMesh);

					FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();

					for (int32 SectionIndex = 0; SectionIndex < RenderData->LODResources[0].Sections.Num(); SectionIndex++ )
					{
						const FStaticMeshSection& StaticMeshSection = RenderData->LODResources[0].Sections[SectionIndex];
						Stats.AddCount(FDataprepStats::StatNameTriangles, StaticMeshSection.NumTriangles);
						Stats.AddCount(FDataprepStats::StatNameVertices, RenderData->LODResources[0].GetNumVertices());
					}

					if (StaticMesh->IsNaniteEnabled() && StaticMesh->HasValidNaniteData())
					{
						const Nanite::FResources& Resources = *StaticMesh->GetRenderData()->NaniteResourcesPtr.Get();
						if (Resources.RootData.Num() > 0)
						{
							Stats.AddCount(FDataprepStats::StatNameNaniteTriangles, Resources.NumInputTriangles);
							Stats.AddCount(FDataprepStats::StatNameNaniteVertices, Resources.NumInputVertices);
						}
					}
				}
			}
			else if( SkeletalMeshComponent )
			{
				USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
				if( SkeletalMesh )
				{
					SkeletalMeshes.Add(SkeletalMesh);

					FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering();
					if (SkelMeshRenderData->LODRenderData.Num())
					{
						const FSkeletalMeshLODRenderData& BaseLOD = SkelMeshRenderData->LODRenderData[0];
						for( int32 SectionIndex=0; SectionIndex<BaseLOD.RenderSections.Num(); SectionIndex++ )
						{
							const FSkelMeshRenderSection& Section = BaseLOD.RenderSections[SectionIndex];
							Stats.AddCount(FDataprepStats::StatNameTriangles, Section.NumVertices);
							Stats.AddCount(FDataprepStats::StatNameVertices, Section.NumVertices);
						}
					}
				}
			}
			else if (LandscapeComponent)
			{
				TSet<UTexture2D*> UniqueTextures;
				for (auto ItComponents = LandscapeComponent->GetLandscapeProxy()->LandscapeComponents.CreateConstIterator(); ItComponents; ++ItComponents)
				{
					const ULandscapeComponent* CurrentComponent = *ItComponents;
					Stats.AddCount(FDataprepStats::StatNameTriangles, FMath::Square(CurrentComponent->ComponentSizeQuads) * 2);
				}
			}
		}
	}
};

TSharedPtr<FDataprepStats> FDataprepStats::GenerateWorldStats(UWorld* InWorld)
{
	TSharedPtr<FDataprepStats> Result = MakeShared<FDataprepStats>();
	DataprepStatsGenerator Gen(InWorld);
	*Result = Gen.GenerateStats();
	return Result;
}
