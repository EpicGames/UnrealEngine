// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UWorld;

struct FDataprepStats
{
	/**	Stat names */
	static const FName StatNameTriangles;
	static const FName StatNameVertices;
	static const FName StatNameNaniteTriangles;
	static const FName StatNameNaniteVertices;
	static const FName StatNameTextures;
	static const FName StatNameTextureSize;
	static const FName StatNameMeshes;
	static const FName StatNameSkeletalMeshes;
	static const FName StatNameMaterials;
	static const FName StatNameLights;
	static const FName StatNameActors;
	static const FName StatNameActorComponents;

	static TSharedPtr<FDataprepStats> GenerateWorldStats(UWorld* InWorld);

	FDataprepStats()
	{
		StatsMap.Add(StatNameTriangles, 0);
		StatsMap.Add(StatNameVertices, 0);
		StatsMap.Add(StatNameNaniteTriangles, 0);
		StatsMap.Add(StatNameNaniteVertices, 0);
		StatsMap.Add(StatNameTextures, 0);
		StatsMap.Add(StatNameTextureSize, 0);
		StatsMap.Add(StatNameMeshes, 0);
		StatsMap.Add(StatNameSkeletalMeshes, 0);
		StatsMap.Add(StatNameMaterials, 0);
		StatsMap.Add(StatNameLights, 0);
		StatsMap.Add(StatNameActors, 0);
		StatsMap.Add(StatNameActorComponents, 0);
	}

	void Set(const FName InStatName, int32 InValue)
	{
		check(StatsMap.Contains(InStatName));
		StatsMap[InStatName] = InValue;
	}

	int32 Get(const FName InStatName) const
	{
		if (!StatsMap.Contains(InStatName))
		{
			return 0;
		}
		return StatsMap[InStatName];
	}

	const TMap<FName, int32>& GetMap() const
	{
		return StatsMap;
	}

	void AddCount(const FName InStatName, int32 InValue)
	{
		check(StatsMap.Contains(InStatName));
		StatsMap[InStatName] += InValue;
	}

private:
	TMap<FName, int32> StatsMap;
};
