// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewSystem/DataprepPreviewAssetColumn.h"

#include "PreviewSystem/DataprepPreviewSystem.h"
#include "Widgets/SDataprepPreviewRow.h"


#define LOCTEXT_NAMESPACE "DataprepPreviewAssetColumn"

const FName FDataprepPreviewAssetColumn::ColumnID = FName("PreviewSystem");

FDataprepPreviewAssetColumn::FDataprepPreviewAssetColumn(const TSharedRef<FDataprepPreviewSystem>& InPreviewSystem)
	: AssetPreviewWidget::IAssetPreviewColumn()
	, PreviewSystem( InPreviewSystem )
{
}

SHeaderRow::FColumn::FArguments FDataprepPreviewAssetColumn::ConstructHeaderRowColumn(const TSharedRef<AssetPreviewWidget::SAssetsPreviewWidget>& PreviewWidget)
{
	PreviewWidgetWeakPtr = PreviewWidget;
	PreviewSystem->GetOnPreviewIsDoneProcessing().AddSP( this, &FDataprepPreviewAssetColumn::OnPreviewSystemIsDoneProcessing );

	return SHeaderRow::Column( GetColumnID() )
		.DefaultLabel( LOCTEXT("Preview_HeaderText", "Preview") )
		.DefaultTooltip( LOCTEXT("Preview_HeaderTooltip", "Show the result of the current preview.") )
		.FillWidth( 5.0f );
}

const TSharedRef<SWidget> FDataprepPreviewAssetColumn::ConstructRowWidget(const AssetPreviewWidget::IAssetTreeItemPtr& TreeItem, const STableRow<AssetPreviewWidget::IAssetTreeItemPtr>& Row, const TSharedRef<AssetPreviewWidget::SAssetsPreviewWidget>& PreviewWidget)
{
	if ( !TreeItem->IsFolder() )
	{
		if ( UObject* Object = static_cast<AssetPreviewWidget::FAssetTreeAssetItem&>( *TreeItem.Get() ).AssetPtr.Get() )
		{
			return SNew( SDataprepPreviewRow, PreviewSystem->GetPreviewDataForObject(Object) )
				.HighlightText( PreviewWidget, &AssetPreviewWidget::SAssetsPreviewWidget::OnGetHighlightText );
		}
	}

	return SNullWidget::NullWidget;
}

void FDataprepPreviewAssetColumn::PopulateSearchStrings(const AssetPreviewWidget::IAssetTreeItemPtr& Item, TArray< FString >& OutSearchStrings, const AssetPreviewWidget::SAssetsPreviewWidget& AssetPreview) const
{
	if ( !Item->IsFolder() )
	{
		if ( FDataprepPreviewProcessingResult* PreviewData = PreviewSystem->GetPreviewDataForObject( static_cast<AssetPreviewWidget::FAssetTreeAssetItem&>( *Item.Get() ).AssetPtr.Get() ).Get() )
		{
			PreviewData->PopulateSearchStringFromFetchedData( OutSearchStrings );
		}
	}
}

void FDataprepPreviewAssetColumn::SortItems(TArray<AssetPreviewWidget::IAssetTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	using IItemPtr = AssetPreviewWidget::IAssetTreeItemPtr;
	OutItems.Sort([this, SortMode](const IItemPtr& First, const IItemPtr& Second)
	{
		if ( First->IsFolder() || Second->IsFolder() )
		{
			return First->Name < Second->Name;
		}

		FDataprepPreviewProcessingResult* FirstPreviewData = PreviewSystem->GetPreviewDataForObject( static_cast<AssetPreviewWidget::FAssetTreeAssetItem&>( *First.Get() ).AssetPtr.Get() ).Get();
		FDataprepPreviewProcessingResult* SecondPreviewData = PreviewSystem->GetPreviewDataForObject( static_cast<AssetPreviewWidget::FAssetTreeAssetItem&>( *Second.Get() ).AssetPtr.Get() ).Get();

		if ( FirstPreviewData && SecondPreviewData )
		{
			if ( FirstPreviewData->Status == SecondPreviewData->Status && FirstPreviewData->CurrentProcessingIndex == SecondPreviewData->CurrentProcessingIndex )
			{
				EDataprepPreviewResultComparison Comparaison = FirstPreviewData->CompareFetchedDataTo( *SecondPreviewData );
				if ( Comparaison != EDataprepPreviewResultComparison::Equal )
				{
					if ( SortMode == EColumnSortMode::Descending )
					{
						return Comparaison == EDataprepPreviewResultComparison::BiggerThan;
					}
					return Comparaison == EDataprepPreviewResultComparison::SmallerThan;
				}
			}
			else if ( FirstPreviewData->Status == EDataprepPreviewStatus::Pass )
			{
				return true;
			}
			else if ( SecondPreviewData->Status == EDataprepPreviewStatus::Pass)
			{
				return false;
			}
			else
			{
				// Filter by at which index they fail
				if ( FirstPreviewData->CurrentProcessingIndex != SecondPreviewData->CurrentProcessingIndex )
				{
					return FirstPreviewData->CurrentProcessingIndex > SecondPreviewData->CurrentProcessingIndex;
				}
			}
		}

		// If all else fail filter by name (always Ascending)
		return First->Name < Second->Name;
	});
	
}


void FDataprepPreviewAssetColumn::OnPreviewSystemIsDoneProcessing()
{
	if (TSharedPtr<AssetPreviewWidget::SAssetsPreviewWidget> PreviewWidget = PreviewWidgetWeakPtr.Pin())
	{
		if ( PreviewWidget->GetColumnSortMode( ColumnID ) != EColumnSortMode::None )
		{
			PreviewWidget->RequestSort();
		}
	}
}

#undef LOCTEXT_NAMESPACE
