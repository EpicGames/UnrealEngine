// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SAssetsPreviewWidget.h"


class FDataprepPreviewSystem;

class FDataprepPreviewAssetColumn : public AssetPreviewWidget::IAssetPreviewColumn
{
public:
	FDataprepPreviewAssetColumn(const TSharedRef<FDataprepPreviewSystem>& InPreviewSystem);
	virtual ~FDataprepPreviewAssetColumn() = default;


	virtual uint8 GetCulumnPositionPriorityIndex() const override
	{
		return 60;
	}

	virtual FName GetColumnID() const override
	{
		return ColumnID;
	}

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<AssetPreviewWidget::SAssetsPreviewWidget>& PreviewWidget) override;

	virtual const TSharedRef<SWidget> ConstructRowWidget(const AssetPreviewWidget::IAssetTreeItemPtr& TreeItem, const STableRow<AssetPreviewWidget::IAssetTreeItemPtr>& Row, const TSharedRef<AssetPreviewWidget::SAssetsPreviewWidget>& PreviewWidget) override;

	virtual void PopulateSearchStrings(const AssetPreviewWidget::IAssetTreeItemPtr& Item, TArray< FString >& OutSearchStrings, const AssetPreviewWidget::SAssetsPreviewWidget& AssetPreview) const override;

	virtual void SortItems(TArray<AssetPreviewWidget::IAssetTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;

	static const FName ColumnID;

private:
	// Request a sort refresh if this column was the one doing the sorting
	void OnPreviewSystemIsDoneProcessing();

	TWeakPtr<AssetPreviewWidget::SAssetsPreviewWidget> PreviewWidgetWeakPtr;
	TSharedPtr<FDataprepPreviewSystem> PreviewSystem;
};
