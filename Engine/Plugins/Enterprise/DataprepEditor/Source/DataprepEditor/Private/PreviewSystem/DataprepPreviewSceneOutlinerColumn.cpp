// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewSystem/DataprepPreviewSceneOutlinerColumn.h"

#include "PreviewSystem/DataprepPreviewSystem.h"
#include "Widgets/SDataprepPreviewRow.h"

#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "ISceneOutliner.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "DataprepPreviewOutlinerColumn"

namespace DataprepPreviewOutlinerColumnUtils
{
	
	UObject* GetObjectPtr(const ISceneOutlinerTreeItem& Item)
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			return ActorItem->Actor.Get();
		}
		else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
		{
			return ComponentItem->Component.Get();
		}
		return nullptr;
	}
}

const FName FDataprepPreviewOutlinerColumn::ColumnID = FName( TEXT("DataprepPreview") );

FDataprepPreviewOutlinerColumn::FDataprepPreviewOutlinerColumn(ISceneOutliner& SceneOutliner, const TSharedRef<FDataprepPreviewSystem>& PreviewData)
	: ISceneOutlinerColumn()
	, WeakSceneOutliner( StaticCastSharedRef<ISceneOutliner>( SceneOutliner.AsShared() ) )
	, CachedPreviewData( PreviewData )
{
}

FName FDataprepPreviewOutlinerColumn::GetColumnID()
{
	return ColumnID;
}

SHeaderRow::FColumn::FArguments FDataprepPreviewOutlinerColumn::ConstructHeaderRowColumn()
{
	CachedPreviewData->GetOnPreviewIsDoneProcessing().AddSP( this, &FDataprepPreviewOutlinerColumn::OnPreviewSystemIsDoneProcessing );

	return SHeaderRow::Column( GetColumnID() )
		.DefaultLabel( LOCTEXT("Preview_HeaderText", "Preview") )
		.DefaultTooltip( LOCTEXT("Preview_HeaderTooltip", "Show the result of the current preview.") )
		.FillWidth( 5.0f );
}

const TSharedRef<SWidget> FDataprepPreviewOutlinerColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if ( UObject* Object = DataprepPreviewOutlinerColumnUtils::GetObjectPtr(*TreeItem))
	{
		if ( TSharedPtr<ISceneOutliner> SceneOutliner = WeakSceneOutliner.Pin() )
		{ 
			return SNew( SDataprepPreviewRow, CachedPreviewData->GetPreviewDataForObject( Object ) )
				.HighlightText( SceneOutliner->GetFilterHighlightText() );
		}
	}
	
	return SNullWidget::NullWidget;
}

void FDataprepPreviewOutlinerColumn::PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings) const
{
	if (UObject* Object = DataprepPreviewOutlinerColumnUtils::GetObjectPtr(Item))
	{
		if ( TSharedPtr<FDataprepPreviewProcessingResult> PreviewResult = CachedPreviewData->GetPreviewDataForObject( Object ) )
		{
			PreviewResult->PopulateSearchStringFromFetchedData( OutSearchStrings );
		}
	}
}

void FDataprepPreviewOutlinerColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	OutItems.Sort([this, SortMode](const FSceneOutlinerTreeItemPtr& First, const FSceneOutlinerTreeItemPtr& Second)
	{
			UObject* FirstObject = DataprepPreviewOutlinerColumnUtils::GetObjectPtr(*First);
			UObject* SecondObject = DataprepPreviewOutlinerColumnUtils::GetObjectPtr(*Second);
			if ( FirstObject && SecondObject )
			{

				FDataprepPreviewProcessingResult* FirstPreviewData = CachedPreviewData->GetPreviewDataForObject( FirstObject ).Get();
				FDataprepPreviewProcessingResult* SecondPreviewData = CachedPreviewData->GetPreviewDataForObject( SecondObject ).Get();

				if ( FirstPreviewData && SecondPreviewData )
				{
					if ( FirstPreviewData->Status == SecondPreviewData->Status && FirstPreviewData->CurrentProcessingIndex == SecondPreviewData->CurrentProcessingIndex )
					{
						EDataprepPreviewResultComparison Comparaison = FirstPreviewData->CompareFetchedDataTo( *SecondPreviewData );
						if ( Comparaison != EDataprepPreviewResultComparison::Equal )
						{
							if ( SortMode == EColumnSortMode::Descending )
							{
								return Comparaison == EDataprepPreviewResultComparison::BiggerThan;
							}
							return Comparaison == EDataprepPreviewResultComparison::SmallerThan;
						}
					}
					else if ( FirstPreviewData->Status == EDataprepPreviewStatus::Pass )
					{
						return true;
					}
					else if ( SecondPreviewData->Status == EDataprepPreviewStatus::Pass )
					{
						return false;
					}
					else
					{
						// Filter by at which index they fail
						if ( FirstPreviewData->CurrentProcessingIndex != SecondPreviewData->CurrentProcessingIndex )
						{
							return FirstPreviewData->CurrentProcessingIndex > SecondPreviewData->CurrentProcessingIndex;
						}
					}
				}
			}

		// If all else fail filter by name (always Ascending)
		auto Outliner = WeakSceneOutliner.Pin();
		int32 SortPriorityFirst = Outliner->GetTypeSortPriority(*First);
		int32 SortPrioritySecond = Outliner->GetTypeSortPriority(*Second);
		if ( SortPriorityFirst != SortPrioritySecond )
		{
			return SortPriorityFirst < SortPrioritySecond;
		}

		return First->GetDisplayString() < Second->GetDisplayString();
	});
}

void FDataprepPreviewOutlinerColumn::OnPreviewSystemIsDoneProcessing()
{
	if ( TSharedPtr<ISceneOutliner> SceneOutliner = WeakSceneOutliner.Pin() )
	{
		if ( SceneOutliner->GetColumnSortMode( ColumnID ) != EColumnSortMode::None )
		{
			SceneOutliner->RequestSort();
		}
	}
}

#undef LOCTEXT_NAMESPACE
