// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerColumn.h"

class FDataprepPreviewSystem;
class ISceneOutliner;
class UObject;

class FDataprepPreviewOutlinerColumn : public ISceneOutlinerColumn
{
public:
	FDataprepPreviewOutlinerColumn(ISceneOutliner& SceneOutliner, const TSharedRef<FDataprepPreviewSystem>& PreviewData);
	virtual ~FDataprepPreviewOutlinerColumn() = default;


	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

	virtual void PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings) const override;

	virtual bool SupportsSorting() const override { return true; }

	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;

	const static FName ColumnID;

private:
	// Request a sort refresh if this column was the one doing the sorting
	void OnPreviewSystemIsDoneProcessing();

	TWeakPtr<ISceneOutliner> WeakSceneOutliner;

	TSharedPtr<FDataprepPreviewSystem> CachedPreviewData;
};
