// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewSystem/DataprepPreviewSystem.h"

#include "DataprepAsset.h"
#include "DataprepCoreUtils.h"
#include "SelectionSystem/DataprepFilter.h"

#include "Async/ParallelFor.h"
#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"

namespace DataprepPreviewUtils
{
	struct FPreviewVariantComparator
	{
		FPreviewVariantComparator(FDataprepPreviewProcessingResult& InCallingObject)
			: Caller( InCallingObject )
		{}

		EDataprepPreviewResultComparison operator()(const FEmptyVariantState&)
		{
			return EDataprepPreviewResultComparison::Equal;
		}

		EDataprepPreviewResultComparison operator()(int32 Value)
		{
			int32 CallerValue = Caller.FetchedData.Get<int32>();
			if ( CallerValue == Value )
			{
				return EDataprepPreviewResultComparison::Equal;
			}
			return CallerValue > Value ? EDataprepPreviewResultComparison::BiggerThan : EDataprepPreviewResultComparison::SmallerThan;
		}

		EDataprepPreviewResultComparison operator()(float Value)
		{
			float CallerValue = Caller.FetchedData.Get<float>();
			if ( CallerValue == Value )
			{
				return EDataprepPreviewResultComparison::Equal;
			}
			return CallerValue > Value ? EDataprepPreviewResultComparison::BiggerThan : EDataprepPreviewResultComparison::SmallerThan;
		}

		EDataprepPreviewResultComparison operator()(const FString& Value)
		{
			const FString& CallerValue = Caller.FetchedData.Get<FString>();
			int32 CompareResult = CallerValue.Compare( Value );
			if ( CompareResult == 0 )
			{
				return EDataprepPreviewResultComparison::Equal;
			}
			return CompareResult > 0 ? EDataprepPreviewResultComparison::BiggerThan : EDataprepPreviewResultComparison::SmallerThan;
		}

		template<class T>
		EDataprepPreviewResultComparison operator()(T) = delete;

		FDataprepPreviewProcessingResult& Caller;
	};

	struct FGetTextFromVariant
	{
		FText operator()(const FEmptyVariantState&)
		{
			return FText::GetEmpty();
		}

		FText operator()(int32 Value)
		{
			return FText::AsNumber(Value);
		}

		FText operator()(float Value)
		{
			FNumberFormattingOptions Options = FInternationalization::Get().GetCurrentLocale()->GetDecimalNumberFormattingRules().CultureDefaultFormattingOptions;
			Options.SetMaximumFractionalDigits(8);
			Options.SetRoundingMode(ERoundingMode::FromZero);
			return FText::AsNumber(Value, &Options);
		}

		FText operator()(const FString& Value)
		{
			return FText::FromString(Value);
		}

		// force explicit declaration of 
		template <typename T>
		FText operator()(T) = delete;
	};

	struct FAddSearchStringsFromVariant
	{
		FAddSearchStringsFromVariant(TArray<FString>& OutStrings)
			: RefToOutStrings( OutStrings )
		{
		}

		void operator()(const FEmptyVariantState&)
		{
		}

		void operator()(int32 Value)
		{
			RefToOutStrings.Add( FText::AsNumber(Value).ToString() );
		}

		void operator()(float Value)
		{
			FNumberFormattingOptions Options = FInternationalization::Get().GetCurrentLocale()->GetDecimalNumberFormattingRules().CultureDefaultFormattingOptions;
			Options.SetMaximumFractionalDigits(8);
			Options.SetRoundingMode(ERoundingMode::FromZero);
			RefToOutStrings.Add( FText::AsNumber(Value, &Options).ToString() );
		}

		void operator()(const FString& Value)
		{
			RefToOutStrings.Add( Value );
		}

		// force explicit declaration of 
		template <typename T>
		void operator()(T) = delete;

	private:
		TArray<FString>& RefToOutStrings;
	};
}

const int32 FDataprepPreviewSystem::IncrementalCount = 2000;

EDataprepPreviewResultComparison FDataprepPreviewProcessingResult::CompareFetchedDataTo(const FDataprepPreviewProcessingResult& Other)
{
	if ( FetchedData.GetIndex() != Other.FetchedData.GetIndex() )
	{
		return EDataprepPreviewResultComparison::Equal;
	}

	return Visit( DataprepPreviewUtils::FPreviewVariantComparator( *this ), Other.FetchedData );
}

FText FDataprepPreviewProcessingResult::GetFetchedDataAsText() const
{
	return Visit( DataprepPreviewUtils::FGetTextFromVariant(), FetchedData );
}

void FDataprepPreviewProcessingResult::PopulateSearchStringFromFetchedData(TArray<FString>& OutStrings) const
{
	Visit( DataprepPreviewUtils::FAddSearchStringsFromVariant( OutStrings ), FetchedData );
}


FDataprepPreviewSystem::~FDataprepPreviewSystem()
{
	StopTrackingObservedObjects();
}

void FDataprepPreviewSystem::UpdateDataToProcess(const TArrayView<UObject*>& Objects)
{
	PreviewResult.Empty( Objects.Num() );

	for ( UObject* Object : Objects )
	{
		PreviewResult.Add( Object, MakeShared<FDataprepPreviewProcessingResult>() );
	}

	RestartProcessing();
};

void FDataprepPreviewSystem::SetObservedObjects(const TArrayView<UDataprepParameterizableObject*>& StepObjects)
{
	StopTrackingObservedObjects();

	ObservedSteps.Empty( StepObjects.Num() );
	// Reserve the double. For the filters we need to track their fetcher also.
	ObservedOnPostEdit.Empty( StepObjects.Num() * 2 );

	ObservedActionAsset = nullptr;
	for ( UDataprepParameterizableObject* Object : StepObjects )
	{
		if ( Object )
		{
			UDataprepActionAsset* ActionAssetOfStep = FDataprepCoreUtils::GetDataprepActionAssetOf( Object );
			if ( !ObservedActionAsset )
			{
				ObservedActionAsset = ActionAssetOfStep;
			}
			
			if ( !ActionAssetOfStep || ActionAssetOfStep != ObservedActionAsset )
			{
				ObservedActionAsset = nullptr;
				ObservedSteps.Empty();
				ObservedOnPostEdit.Empty();
				RestartProcessing();
				return;
			}
		}
	}

	if ( ObservedActionAsset )
	{
		OnActionStepOrderChangedHandle = ObservedActionAsset->GetOnStepsOrderChanged().AddLambda( [this]()
			{
				EnsureValidityOfTheObservedObjects();
			});
	}

	for ( UDataprepParameterizableObject* Object : StepObjects )
	{
		if ( Object )
		{
			ObservedSteps.Add( Object );
			
			FDelegateHandle Handle = Object->GetOnPostEdit().AddSP( this, &FDataprepPreviewSystem::OnObservedObjectPostEdit );
			ObservedOnPostEdit.Add( Object, Handle );

			if ( UDataprepFilter* Filter  = Cast<UDataprepFilter>( Object ) )
			{
				if ( UDataprepFetcher* Fetcher = Filter->GetFetcher() )
				{
					Handle = Fetcher->GetOnPostEdit().AddSP( this, &FDataprepPreviewSystem::OnObservedObjectPostEdit);
					ObservedOnPostEdit.Add( Fetcher, Handle );
				}
			}
			else if( UDataprepFilterNoFetcher* FilterNF = Cast<UDataprepFilterNoFetcher>( Object ) )
			{
				Handle = FilterNF->GetOnPostEdit().AddSP( this, &FDataprepPreviewSystem::OnObservedObjectPostEdit);
				ObservedOnPostEdit.Add( FilterNF, Handle );
			}
		}
	}

	// Make sure the step are inserted in the right order
	EnsureValidityOfTheObservedObjects();
}

void FDataprepPreviewSystem::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects( ObservedSteps );
	Collector.AddReferencedObjects( ObservedOnPostEdit );
	Collector.AddReferencedObjects( PreviewResult );
	Collector.AddReferencedObject( ObservedActionAsset );
}

TSharedPtr<FDataprepPreviewProcessingResult> FDataprepPreviewSystem::GetPreviewDataForObject(UObject* Object) const
{
	if ( const TSharedRef<FDataprepPreviewProcessingResult>* PreviewData = PreviewResult.Find( Object ) )
	{
		return *PreviewData;
	}

	return {};
}

void FDataprepPreviewSystem::IncrementalProcess()
{
	const int32 ItemsCount = PreviewResult.Num();
	int32 BudgetLeft = IncrementalCount;
	while ( BudgetLeft > 0 && CurrentProgress.CurrentFilterIndex < ObservedSteps.Num() )
	{
		int32 ItemToProcessCount = ItemsCount - CurrentProgress.CurrentObjectProcessed;

		int32 ItemsProcessedCount = FMath::Min(ItemToProcessCount, BudgetLeft);

		bool bDoneWithCurrentObject = false;

		// We skip the current object if we are not able to process it
		bool bDidSomeProcessing = false;

		if ( CurrentProgress.Iterator.IsValid() )
		{
			if ( UDataprepFilter* Filter = Cast<UDataprepFilter>( ObservedSteps[CurrentProgress.CurrentFilterIndex] ) )
			{
				bDidSomeProcessing = true;
				PrepareFilterBuffers( IncrementalCount );
				ItemsProcessedCount = FillObjectsBuffer( ItemsProcessedCount );

				TArrayView<UObject*> Objects( ObjectsBuffer.GetData(), ItemsProcessedCount );
				TArrayView<FDataprepSelectionInfo> FilterResults( FilterResultsBuffer.GetData(), ItemsProcessedCount );
				Filter->FilterAndGatherInfo( Objects, FilterResults );
				PopulateResultFromFilter( ItemsProcessedCount );
			}
			if ( UDataprepFilterNoFetcher* FilterNF = Cast<UDataprepFilterNoFetcher>( ObservedSteps[CurrentProgress.CurrentFilterIndex] ) )
			{
				bDidSomeProcessing = true;
				PrepareFilterBuffers( IncrementalCount );
				ItemsProcessedCount = FillObjectsBuffer( ItemsProcessedCount );

				TArrayView<UObject*> Objects( ObjectsBuffer.GetData(), ItemsProcessedCount );
				TArrayView<FDataprepSelectionInfo> FilterResults( FilterResultsBuffer.GetData(), ItemsProcessedCount );
				FilterNF->FilterAndGatherInfo( Objects, FilterResults );
				PopulateResultFromFilter( ItemsProcessedCount );
			}

			if ( CurrentProgress.CurrentObjectProcessed >= ItemsCount )
			{
				bDoneWithCurrentObject = true;
			}
		}

		bDoneWithCurrentObject |= !bDidSomeProcessing;

		if ( bDidSomeProcessing )
		{
			BudgetLeft -= ItemsProcessedCount;
		}

		if ( bDoneWithCurrentObject )
		{
			CurrentProgress.CurrentFilterIndex++;
			CurrentProgress.CurrentObjectProcessed = 0;
			CurrentProgress.Iterator = MakeUnique<FResultIterator>( PreviewResult );
		}
	}
}

void FDataprepPreviewSystem::RestartProcessing()
{
	const bool bResetProcessStatus = CurrentProgress.CurrentFilterIndex != 0 || CurrentProgress.CurrentObjectProcessed != 0;

	CurrentProgress.CurrentFilterIndex = 0;
	CurrentProgress.CurrentObjectProcessed = 0;

	if ( PreviewResult.Num() > 0 )
	{
		CurrentProgress.Iterator = MakeUnique<FResultIterator>( PreviewResult );

		if ( bResetProcessStatus )
		{
			FResultIterator ResultIterator( PreviewResult );
			while ( ResultIterator )
			{
				FDataprepPreviewProcessingResult& Result = ResultIterator.Value().Get();
				Result.Status = EDataprepPreviewStatus::BeingProcessed;
				Result.FetchedData.Set<FEmptyVariantState>( FEmptyVariantState() );
				Result.CurrentProcessingIndex = 0;
				++ResultIterator;
			}
		}

		if ( ObservedSteps.Num() > 0 )
		{
			bIsProcessing = true;
		}
	}
}

void FDataprepPreviewSystem::ClearProcessing()
{
	SetObservedObjects( MakeArrayView<UDataprepParameterizableObject*>( nullptr, 0) );
}

bool FDataprepPreviewSystem::HasAnObjectObserved(const TArrayView<UDataprepParameterizableObject*>& StepObjects) const 
{
	for ( UDataprepParameterizableObject* StepObject : StepObjects )
	{
		if ( ObservedOnPostEdit.Contains( StepObject ) )
		{
			return true;
		}
	}

	return false;
}

bool FDataprepPreviewSystem::IsObservingStepObject(const UDataprepParameterizableObject* StepObject) const
{
	return ObservedOnPostEdit.Contains( StepObject );
}

void FDataprepPreviewSystem::EnsureValidityOfTheObservedObjects()
{
	if ( ObservedActionAsset )
	{
		if ( UDataprepAsset* Owner = Cast<UDataprepAsset>( ObservedActionAsset->GetOuter() ) )
		{
			bool bActionWasFoundInParent = false;
			for ( int32 Index = 0; Index < Owner->GetActionCount(); ++Index )
			{
				if ( ObservedActionAsset == Owner->GetAction( Index ) )
				{
					bActionWasFoundInParent = true;
					break;
				}
			}

			if ( !bActionWasFoundInParent )
			{
				// The action wasn't found in the dataprep asset. Cancel the preview.
				SetObservedObjects( MakeArrayView<UDataprepParameterizableObject*>( nullptr, 0 ) );
				return;
			}
		}
		else
		{
			// If the action is not own by a dataprep asset bail out.
			SetObservedObjects( MakeArrayView<UDataprepParameterizableObject*>( nullptr, 0 ) );
			return;
		}

		TArray<UDataprepParameterizableObject*> ObjectOrdering;
		ObjectOrdering.Reserve( ObservedSteps.Num() );

		TSet<UDataprepParameterizableObject*> ObservedStepsSet;
		ObservedStepsSet.Append(ObjectPtrDecay(ObservedSteps));

		bool bShouldUpdateStepsOrder = false;

		int32 StepCount = ObservedActionAsset->GetStepsCount();
		for ( int32 Index = 0; Index < StepCount; ++Index )
		{
			if ( UDataprepActionStep* Step = ObservedActionAsset->GetStep( Index ).Get() )
			{
				UDataprepParameterizableObject* StepObject = Step->GetStepObject();
				if ( ObservedStepsSet.Contains( StepObject ) )
				{
					if ( ObservedSteps[ObjectOrdering.Num()] != StepObject )
					{
						bShouldUpdateStepsOrder = true;
					}

					ObjectOrdering.Add( StepObject );
				}
			}
		}

		if ( ObjectOrdering.Num() != ObservedSteps.Num() )
		{
			// We lost a step. Cancel the preview.
			SetObservedObjects( MakeArrayView<UDataprepParameterizableObject*>( nullptr, 0 ) );
			return;
		}

		if ( bShouldUpdateStepsOrder )
		{
			for ( int32 Index = 0; Index < ObjectOrdering.Num(); ++Index )
			{
				ObservedSteps[Index] = ObjectOrdering[Index];
			}
		}
		
		// Just in case a observed object was edited
		RestartProcessing();
	}
}

int32 FDataprepPreviewSystem::FillObjectsBuffer(int32 MaximunNumberOfObject)
{
	MaximunNumberOfObject = FMath::Min( ObjectsBuffer.Num(), MaximunNumberOfObject );

	int32 Index = 0;
	// Fill the object buffer
	while ( Index < MaximunNumberOfObject && bool( *CurrentProgress.Iterator.Get() ) )
	{
		if ( CurrentProgress.Iterator->Value()->Status == EDataprepPreviewStatus::BeingProcessed )
		{
			ObjectsBuffer[Index] = CurrentProgress.Iterator->Key();
			Index++;
		}

		CurrentProgress.CurrentObjectProcessed++;
		CurrentProgress.Iterator->operator++();
	}

	return Index;
}

void FDataprepPreviewSystem::PrepareFilterBuffers(int32 DesiredSize)
{
	if ( ObjectsBuffer.Num() != DesiredSize )
	{
		ObjectsBuffer.Empty( DesiredSize );
		ObjectsBuffer.AddZeroed( DesiredSize );
	}

	if ( FilterResultsBuffer.Num() != DesiredSize )
	{
		FilterResultsBuffer.Empty( DesiredSize );
		FilterResultsBuffer.AddDefaulted( DesiredSize );
	}
}

void FDataprepPreviewSystem::PopulateResultFromFilter(int32 NumberOfValidObjects)
{
	check( ObjectsBuffer.Num() == FilterResultsBuffer.Num() );

	NumberOfValidObjects = FMath::Min( ObjectsBuffer.Num(), NumberOfValidObjects );
	
	const bool bIsLastFilter =  CurrentProgress.CurrentFilterIndex == ObservedSteps.Num() - 1;

	ParallelFor( NumberOfValidObjects, [this, bIsLastFilter](int32 Index)
		{
			UObject* Object = ObjectsBuffer[Index];
			if ( TSharedRef<FDataprepPreviewProcessingResult>* ResultPtr = PreviewResult.Find( Object ) )
			{
				FDataprepPreviewProcessingResult& Result = ResultPtr->Get();
				Result.CurrentProcessingIndex = CurrentProgress.CurrentFilterIndex;
				FDataprepSelectionInfo& FilterResult = FilterResultsBuffer[Index];
				if ( FilterResult.bHasPassFilter )
				{
					if ( bIsLastFilter )
					{
						Result.Status = EDataprepPreviewStatus::Pass;
						if ( FilterResult.bWasDataFetchedAndCached )
						{
							Result.FetchedData = MoveTemp( FilterResult.FetchedData );
						}
					}
				}
				else
				{
					Result.Status = EDataprepPreviewStatus::Failed;
					if ( FilterResult.bWasDataFetchedAndCached && bIsLastFilter )
					{
						Result.FetchedData = MoveTemp( FilterResult.FetchedData );
					}
				}

				FilterResult.bHasPassFilter = false;

			}
		});
}

void FDataprepPreviewSystem::StopTrackingObservedObjects()
{
	if ( ObservedActionAsset )
	{
		ObservedActionAsset->GetOnStepsOrderChanged().Remove( OnActionStepOrderChangedHandle );
	}

	for ( auto& Pair : ObservedOnPostEdit )
	{
		if ( Pair.Key )
		{
			Pair.Key->GetOnPostEdit().Remove( Pair.Value );
		}
	}
}

void FDataprepPreviewSystem::OnObservedObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	RestartProcessing();
}


void FDataprepPreviewSystem::Tick(float DeltaTime)
{
	IncrementalProcess();

	if ( CurrentProgress.CurrentFilterIndex >= ObservedSteps.Num() )
	{
		bIsProcessing = false;
		OnPreviewIsDoneProcessing.Broadcast();
	}
}

TStatId FDataprepPreviewSystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDataprepPreviewSystem, STATGROUP_Tickables);
}
