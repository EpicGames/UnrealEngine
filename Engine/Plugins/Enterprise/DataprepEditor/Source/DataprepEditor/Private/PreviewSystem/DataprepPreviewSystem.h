// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariantMeta.h"
#include "SelectionSystem/DataprepSelectionSystemStructs.h"

#include "TickableEditorObject.h"
#include "UObject/GCObject.h"

class UDataprepParameterizableObject;

class FReferenceCollector;
class UDataprepActionAsset;

struct TStatId;

enum class EDataprepPreviewStatus : uint8
{
	BeingProcessed,
	NotSupported,
	Failed,
	Pass
	// There is assumption on the order of these so that the expression Pass > Failed return true
};

enum class EDataprepPreviewResultComparison : uint8
{
	SmallerThan,
	Equal,
	BiggerThan
};

struct FDataprepPreviewProcessingResult
{
	FDataprepPreviewProcessingResult()
		: Status( EDataprepPreviewStatus::BeingProcessed )
		, CurrentProcessingIndex( 0 )
		, FetchedData()
	{}

	EDataprepPreviewResultComparison CompareFetchedDataTo(const FDataprepPreviewProcessingResult& Other);

	FText GetFetchedDataAsText() const;

	void PopulateSearchStringFromFetchedData(TArray<FString>& OutStrings) const;

	EDataprepPreviewStatus Status;
	uint32 CurrentProcessingIndex;
	FFilterVariantData FetchedData;
};

/**
 * A class managing the responsibility of processing the data for the preview of the result of a filter
 */
class FDataprepPreviewSystem final : public FGCObject, public FTickableEditorObject, public TSharedFromThis<FDataprepPreviewSystem>
{
public:
	FDataprepPreviewSystem() = default;
	virtual ~FDataprepPreviewSystem();

	FDataprepPreviewSystem(const FDataprepPreviewSystem&) = delete;
	FDataprepPreviewSystem(FDataprepPreviewSystem&&) = delete;
	FDataprepPreviewSystem& operator=(const FDataprepPreviewSystem&) = delete;
	FDataprepPreviewSystem& operator=(FDataprepPreviewSystem&&) = delete;

	// Update the data to process
	void UpdateDataToProcess(const TArrayView<UObject*>& Objects);

	void SetObservedObjects(const TArrayView<UDataprepParameterizableObject*>& Objects);

	bool HasObservedObjects() const { return ObservedSteps.Num() > 0; }

	// Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDataprepPreviewSystem");
	}
	// End FGCObject interface

	TSharedPtr<FDataprepPreviewProcessingResult> GetPreviewDataForObject(UObject* Object) const;

	// Reset the processing from the start
	void RestartProcessing();

	// Cancel any ongoing processing
	void ClearProcessing();

	bool HasAnObjectObserved(const TArrayView<UDataprepParameterizableObject*>& StepObjects) const;

	// Is the preview system observing the object
	bool IsObservingStepObject(const UDataprepParameterizableObject* StepObject) const;

	// Return the current number of the step observed
	int32 GetObservedStepsCount() const { return ObservedSteps.Num(); }

	/**
	 * Check if the observed object are valid and also check if the observed object are in the right order
	 * If an observed object is invalid clear the preview
	 */
	void EnsureValidityOfTheObservedObjects();

	// Broadcast when the preview system is done processing the data to preview
	DECLARE_EVENT(FDataprepPreviewSystem, FOnPreviewIsDoneProcessing)
	FOnPreviewIsDoneProcessing& GetOnPreviewIsDoneProcessing() { return OnPreviewIsDoneProcessing; }

private:

	void IncrementalProcess();

	/**
	 * Fill the object buffer from the current iterator and update the current progress
	 * @param MaximunNumberOfObject Useful if we don't want to fill all the buffer
	 * @return the number of object inserted into the buffer
	 */
	int32 FillObjectsBuffer(int32 MaximunNumberOfObject);

	/**
	 * Ensure that buffer are of the proper size
	 */
	void PrepareFilterBuffers(int32 DesiredSize);

	/**
	 * Migrate the result of the filter from the buffer to the preview result map
	 * @param NumberOfValidObjects Allow to stop before the end of the buffer it wasn't fully use
	 */
	void PopulateResultFromFilter(int32 NumberOfValidObjects);

	void StopTrackingObservedObjects();

	void OnObservedObjectPostEdit(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent);

public:

	// Begin FTickableObjectBase interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return bIsProcessing; }
	virtual TStatId GetStatId() const override;
	// End FTickableObjectBase interface

private:
	// The action from which the step object are previewed
	TObjectPtr<UDataprepActionAsset> ObservedActionAsset;
	FDelegateHandle OnActionStepOrderChangedHandle;

	// The filters previewed
	TArray<TObjectPtr<UDataprepParameterizableObject>> ObservedSteps;

	// The handle for the PostEditEvents of the observed objects
	TMap<TObjectPtr<UDataprepParameterizableObject>,FDelegateHandle> ObservedOnPostEdit;

	// The inprogress or done result
	TMap<TObjectPtr<UObject>, TSharedRef<FDataprepPreviewProcessingResult>> PreviewResult;

	using FResultIterator = decltype(PreviewResult)::TIterator;

	// The current progress of processing for the preview
	struct
	{
		int32 CurrentFilterIndex;
		int32 CurrentObjectProcessed;
		TUniquePtr<FResultIterator> Iterator;
	} CurrentProgress;

	bool bIsProcessing = false;

	// Buffers of object to process
	TArray<UObject*> ObjectsBuffer;

	// Buffer for the filters
	TArray<FDataprepSelectionInfo> FilterResultsBuffer;

	static const int32 IncrementalCount;

	FOnPreviewIsDoneProcessing OnPreviewIsDoneProcessing;
};
