// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepAllMenuActionCollector.h"

#include "SchemaActions/DataprepFilterMenuActionCollector.h"
#include "SchemaActions/DataprepOperationMenuActionCollector.h"
#include "SchemaActions/DataprepSelectionTransformMenuActionCollector.h"
#include "SchemaActions/DataprepSchemaAction.h"

namespace FDataprepAllMenuActionCollectorUtils
{
	void AddRootCategoryToActions(TArray<TSharedPtr<FDataprepSchemaAction>> Actions, const FText& Category)
	{
		for ( TSharedPtr<FDataprepSchemaAction> Action : Actions )
		{
			if ( Action )
			{
				Action->CosmeticUpdateCategory( FText::FromString( Category.ToString() + TEXT("|") + Action->GetCategory().ToString() ) );
			}
		}
	}
}

TArray<TSharedPtr<FDataprepSchemaAction>> FDataprepAllMenuActionCollector::CollectActions()
{
	FDataprepFilterMenuActionCollector FilterCollector;
	FilterCollector.GroupingPriority = 1;
	FDataprepSelectionTransformMenuActionCollector SelectionTransformCollector;
	TArray< TSharedPtr< FDataprepSchemaAction > > Actions;
	Actions.Append( FilterCollector.CollectActions() );
	Actions.Append( SelectionTransformCollector.CollectActions() );

	FDataprepOperationMenuActionCollector OperationCollector;
	TArray< TSharedPtr< FDataprepSchemaAction > > OperationActions = OperationCollector.CollectActions();
	Actions.Append( MoveTemp( OperationActions ) );

	return Actions;
}

bool FDataprepAllMenuActionCollector::ShouldAutoExpand()
{
	return false;
}
