// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/IDataprepMenuActionCollector.h"


struct FDataprepSchemaAction;

/**
 * Help collecting all the dataprep specific menu action (operations and filters)
 */
class FDataprepAllMenuActionCollector : public IDataprepMenuActionCollector
{
public:
	virtual TArray<TSharedPtr<FDataprepSchemaAction>> CollectActions() override;
	virtual bool ShouldAutoExpand() override;
};
