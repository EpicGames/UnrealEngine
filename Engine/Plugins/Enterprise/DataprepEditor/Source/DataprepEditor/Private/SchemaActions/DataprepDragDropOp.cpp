// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepDragDropOp.h"

#include "DataprepCoreUtils.h"
#include "DataprepGraph/DataprepGraph.h"
#include "DataprepGraph/DataprepGraphActionNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"
#include "Widgets/DataprepGraph/SDataprepGraphActionStepNode.h"
#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"

#include "EdGraphSchema_K2.h"
#include "ScopedTransaction.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DataprepDragAndDrop"

FDataprepDragDropOp::FDataprepDragDropOp()
	: FGraphEditorDragDropAction()
	, HoveredDataprepActionContext()
{
	bDropTargetValid = false;
	bDropTargetItself = false;
}

TSharedRef<FDataprepDragDropOp> FDataprepDragDropOp::New(TSharedRef<FDataprepSchemaAction> InAction)
{
	TSharedRef< FDataprepDragDropOp > DragDrop = MakeShared< FDataprepDragDropOp >();
	DragDrop->DataprepGraphOperation.BindSP( InAction, &FDataprepSchemaAction::ExecuteAction );
	DragDrop->Construct();
	return DragDrop;
}

TSharedRef<FDataprepDragDropOp> FDataprepDragDropOp::New(FDataprepGraphOperation&& DataprepGraphOperation)
{
	TSharedRef< FDataprepDragDropOp > DragDrop = MakeShared< FDataprepDragDropOp >();
	DragDrop->DataprepGraphOperation = MoveTemp( DataprepGraphOperation );
	DragDrop->Construct();
	return DragDrop;
}

TSharedRef<FDataprepDragDropOp> FDataprepDragDropOp::New(const TSharedRef<SDataprepGraphTrackNode>& InTrackNode, const TSharedRef<SDataprepGraphActionStepNode>& InDraggedNode)
{
	TSharedRef<FDataprepDragDropOp> Operation = MakeShareable(new FDataprepDragDropOp);

	Operation->TrackNodePtr = InTrackNode;
	Operation->DraggedNodeWidgets.Add(InDraggedNode);
	if(UDataprepGraphActionStepNode* ActionStepNode = Cast<UDataprepGraphActionStepNode>(InDraggedNode->GetNodeObj()))
	{
		Operation->DraggedSteps.Emplace(ActionStepNode->GetDataprepActionAsset(), ActionStepNode->GetStepIndex(), ActionStepNode->GetDataprepActionStep() );
	}

	// adjust the decorator away from the current mouse location a small amount based on cursor size
	Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();

	Operation->Construct();

	return Operation;
}

TSharedRef<FDataprepDragDropOp> FDataprepDragDropOp::New(UDataprepActionStep* InActionStep)
{
	TSharedRef<FDataprepDragDropOp> Operation = MakeShareable(new FDataprepDragDropOp);

	if(InActionStep != nullptr)
	{
		Operation->DraggedSteps.Emplace(nullptr, INDEX_NONE, InActionStep );

		// adjust the decorator away from the current mouse location a small amount based on cursor size
		Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();

		Operation->Construct();
	}

	return Operation;
}

void FDataprepDragDropOp::Construct()
{
	FGraphEditorDragDropAction::Construct();
	CursorDecoratorWindow->SetOpacity(0.9f);
}

void FDataprepDragDropOp::HoverTargetChanged()
{
	FText DrapDropText;

	bDropTargetItself = false;

	if(DraggedNodeWidgets.Num() > 0)
	{
		HoverTargetChangedWithNodes();
		return;
	}

	if ( HoveredDataprepActionContext )
	{
		bDropTargetValid = true;
		DrapDropText = LOCTEXT("TargetIsDataprepActionContext", "Add a Step to Dataprep Action");
	}
	else if(Cast<UDataprepGraphActionStepNode>(GetHoveredNode()) != nullptr)
	{
		bDropTargetValid = true;
		DrapDropText = LOCTEXT("CopyDataprepActionStepNode", "Add/Insert step to location");
	}
	else if(Cast<UDataprepGraphActionNode>(GetHoveredNode()) != nullptr)
	{
		bDropTargetValid = true;
		DrapDropText = LOCTEXT("CopyDataprepActionAssetNode", "Add/Insert step to location");
	}
	else if(Cast<UDataprepGraphRecipeNode>(GetHoveredNode()) != nullptr)
	{
		bDropTargetValid = true;
		DrapDropText = LOCTEXT("InsertDataprepActionAssetNode", "Insert action to location");
	}
	else if ( UEdGraph* EdGraph = GetHoveredGraph() )
	{
		if ( const UEdGraphSchema_K2* GraphSchema_k2 = Cast<UEdGraphSchema_K2>( EdGraph->GetSchema() ) )
		{
			bDropTargetValid = true;
			DrapDropText = LOCTEXT("TargetIsBlueprintGraph", "Add a Dataprep Action");
		}
		else
		{
			bDropTargetValid = false;
			DrapDropText = LOCTEXT("TargetGraphIsInvalid", "Can only be drop on a blueprint graph");
		}
	}
	else
	{
		bDropTargetValid = false;
		DrapDropText = FText::FromString( TEXT("Can't drop here") );
	}

	const FSlateBrush* Symbol = bDropTargetValid ? FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.OK") ) : FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.Error") );
	SetSimpleFeedbackMessage( Symbol, FLinearColor::White, DrapDropText );
}

void FDataprepDragDropOp::OnDragged(const FDragDropEvent& DragDropEvent)
{
	FVector2D TargetPosition = DragDropEvent.GetScreenSpacePosition();

	// Reposition the info window to the dragged position
	CursorDecoratorWindow->MoveWindowTo(TargetPosition + DecoratorAdjust);
	// Request the active panel to scroll if required

	if(SDataprepGraphTrackNode* TrackNode = TrackNodePtr.Pin().Get())
	{
		TrackNode->RequestViewportPan(TargetPosition);
	}

	Super::OnDragged(DragDropEvent);
}

EVisibility FDataprepDragDropOp::GetIconVisible() const
{
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	const bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	return bDropTargetValid || bCopyRequested ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FDataprepDragDropOp::GetErrorIconVisible() const
{
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	const bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	return bDropTargetValid || bCopyRequested ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply FDataprepDragDropOp::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if(UDataprepGraphActionStepNode* ActionStepNode = Cast<UDataprepGraphActionStepNode>(GetHoveredNode()))
	{
		return DoDropOnActionStep(ActionStepNode);
	}
	else if(UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(GetHoveredNode()))
	{
		return DoDropOnActionAsset(ActionNode);
	}

	return FReply::Unhandled();
}

UDataprepGraphActionStepNode* FDataprepDragDropOp::GetDropTargetNode() const
{
	return Cast<UDataprepGraphActionStepNode>(GetHoveredNode());
}

void FDataprepDragDropOp::SetHoveredDataprepActionContext(TOptional<FDataprepSchemaActionContext> Context)
{
	if ( HoveredDataprepActionContext != Context )
	{
		HoveredDataprepActionContext = Context;
		HoverTargetChanged();
	}
}

FReply FDataprepDragDropOp::DroppedOnDataprepActionContext(const FDataprepSchemaActionContext& Context)
{
	if ( DataprepPreDropConfirmation.IsBound() )
	{
		TFunction<void ()> OnConfirmation( [Operation = StaticCastSharedRef<FDataprepDragDropOp>(AsShared()), Context = FDataprepSchemaActionContext(Context)] ()
		{
			Operation->DoDropOnDataprepActionContext( Context );
		} );

		DataprepPreDropConfirmation.Execute( Context, OnConfirmation );
	}
	else
	{
		DoDropOnDataprepActionContext( Context );
	}

	return FReply::Handled();
}

void FDataprepDragDropOp::HoverTargetChangedWithNodes()
{
	bDropTargetItself = DraggedNodeWidgets[0]->GetNodeObj() == GetHoveredNode();
	bDropTargetValid = GetHoveredNode() && !bDropTargetItself;

	const FSlateBrush* Icon = FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.OK") );

	TAttribute<FText> MessageText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FDataprepDragDropOp::GetMessageText));

	TSharedRef<SVerticalBox> FeedbackBox = SNew(SVerticalBox);

	FeedbackBox->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SImage)
				.Visibility(EVisibility::Visible)
				.Image( TAttribute<const FSlateBrush*>::Create(TAttribute<const FSlateBrush*>::FGetter::CreateSP(this, &FDataprepDragDropOp::GetIcon)) )
				.ColorAndOpacity( FLinearColor::White )
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f)
		.MaxWidth(500)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.WrapTextAt( 480 )
			.Text( MessageText )
		]
	];

	for (int32 i=0; i < DraggedNodeWidgets.Num(); i++)
	{
		FeedbackBox->AddSlot()
		.AutoHeight()
		[
			DraggedNodeWidgets[i]->GetStepTitleWidget().ToSharedRef()
		];
	}

	SetFeedbackMessage(FeedbackBox);
}

FText FDataprepDragDropOp::GetMessageText()
{
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	const bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	if(bDropTargetValid || bCopyRequested)
	{
		if(UEdGraphNode* CurrentHoveredNode = GetHoveredNode())
		{
			if(CurrentHoveredNode->IsA<UDataprepGraphActionStepNode>() || CurrentHoveredNode->IsA<UDataprepGraphActionNode>())
			{
				LastMessageText = bCopyRequested ? LOCTEXT("DataprepActionStepNode_Copy", "Copy step to location") : LOCTEXT("DataprepActionStepNode_Move", "Move step to location");
			}
			else if(CurrentHoveredNode->IsA<UDataprepGraphRecipeNode>())
			{
				LastMessageText = LOCTEXT("DataprepActionAssetNode_Insert", "Insert action to location");
			}
		}
		else
		{
			LastMessageText = LOCTEXT("DataprepActionStepNode_NotImplemented", "Operation not allowed");
		}
	}
	else if(GetHoveredNode() != nullptr)
	{
		LastMessageText = LOCTEXT("DataprepActionStepNode_Move", "Move step to location");
	}
	else
	{
		LastMessageText = LOCTEXT("DataprepActionStepNode_NotImplemented", "Operation not allowed");
	}

	return LastMessageText;
}

const FSlateBrush* FDataprepDragDropOp::GetIcon() const
{
	static const FSlateBrush* IconOK = FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.OK") );
	static const FSlateBrush* IconError = FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.Error") );

	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	const bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	return bDropTargetValid || (GetHoveredNode() != nullptr && bCopyRequested) 
		? IconOK 
		: (bDropTargetItself ? nullptr : IconError);
}

void FDataprepDragDropOp::SetPreDropConfirmation(FDataprepPreDropConfirmation && Confirmation)
{
	DataprepPreDropConfirmation = MoveTemp( Confirmation );
}

bool FDataprepDragDropOp::DoDropOnDataprepActionContext(const FDataprepSchemaActionContext& Context)
{
	if ( DataprepGraphOperation.IsBound() )
	{
		FScopedTransaction Transaction( LOCTEXT("AddStep", "Add a Step to a Dataprep Action") );
		bool bDidModification = DataprepGraphOperation.Execute( Context );
		if ( !bDidModification )
		{
			Transaction.Cancel();
		}
		return bDidModification;
	}
	return false;
}

FReply FDataprepDragDropOp::DoDropOnActionStep(UDataprepGraphActionStepNode* TargetActionStepNode)
{
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	if(!bDropTargetValid && !bCopyRequested)
	{
		return FReply::Unhandled().EndDragDrop();
	}

	UDataprepActionAsset* TargetActionAsset = TargetActionStepNode->GetDataprepActionAsset();
	ensure(TargetActionAsset);

	if(DraggedSteps.Num() > 0)
	{
		for(FDraggedStepEntry& DraggedStepEntry : DraggedSteps)
		{
			if(DraggedStepEntry.Get<0>().IsValid() || DraggedStepEntry.Get<2>().IsValid())
			{
				FScopedTransaction Transaction( LOCTEXT("DropOnActionStep", "Copy/Move Dataprep Action Step") );
					bool bTransactionSuccessful = true;

					// External drag and drop
					if(!DraggedStepEntry.Get<0>().IsValid())
					{
						bTransactionSuccessful &= TargetActionAsset->InsertStep( DraggedStepEntry.Get<2>().Get(), TargetActionStepNode->GetStepIndex() );
					}
					// Drag and drop within an action asset or between two action assets
					else
					{
						UDataprepActionAsset* SourceActionAsset = DraggedStepEntry.Get<0>().Get();
						check(SourceActionAsset);

						int32 StepIndex = DraggedStepEntry.Get<1>();
						check(StepIndex != INDEX_NONE);

						// Hold onto the action step in case of a move
						TStrongObjectPtr<UDataprepActionStep> SourceActionStepPtr = TStrongObjectPtr<UDataprepActionStep>( SourceActionAsset->GetStep(StepIndex).Get() );
						check(SourceActionStepPtr.IsValid());

						// source action asset differs from target action asset
						if( TargetActionAsset != SourceActionAsset)
						{
							if(!bCopyRequested)
							{
								int32 ActionIndex = INDEX_NONE;
								bTransactionSuccessful &= FDataprepCoreUtils::RemoveStep(SourceActionAsset, StepIndex, ActionIndex, false);
							}

							bTransactionSuccessful &= TargetActionAsset->InsertStep( SourceActionStepPtr.Get(), TargetActionStepNode->GetStepIndex() );
						}
						else if(bCopyRequested)
						{
							bTransactionSuccessful &= TargetActionAsset->InsertStep( SourceActionStepPtr.Get(), TargetActionStepNode->GetStepIndex() );
						}
						else
						{
							bTransactionSuccessful &= TargetActionAsset->MoveStep( StepIndex, TargetActionStepNode->GetStepIndex() );
						}
					}

				if(!bTransactionSuccessful)
				{
					Transaction.Cancel();
				}
			}
		}

		DraggedNodeWidgets.Reset();
		DraggedSteps.Reset();
	}
	else
	{
		DropStepFromPanel(TargetActionAsset, TargetActionStepNode->GetStepIndex());
	}

	return FReply::Handled().EndDragDrop();
}

void FDataprepDragDropOp::DropStepFromPanel(UDataprepActionAsset* TargetActionAsset, int32 InsertIndex)
{
	FScopedTransaction Transaction( LOCTEXT("AddStepToAction", "Add/Insert Dataprep Action Step") );

	FDataprepSchemaActionContext Context;
	Context.DataprepActionPtr = TargetActionAsset;
	Context.StepIndex = InsertIndex;

	if ( !DataprepGraphOperation.Execute( Context ) )
	{
		Transaction.Cancel();
	}
}

FReply FDataprepDragDropOp::DoDropOnTrack(UDataprepAsset* TargetDataprepAsset, int32 InsertIndex)
{
	if(Cast<UDataprepGraphRecipeNode>(GetHoveredNode()) == nullptr)
	{
		return FReply::Handled().EndDragDrop();
	}

	int32 ActionInsertIndex = TrackNodePtr.Pin()->GetActionIndex(InsertIndex);

	if (DraggedNodeWidgets.Num() > 0)
	{
		TSharedPtr<SDataprepGraphActionNode> ParentNodePtr = DraggedNodeWidgets[0]->GetParentNode().Pin();
		if (ParentNodePtr.IsValid() && ParentNodePtr->GetExecutionOrder() < InsertIndex)
		{
			ActionInsertIndex = TrackNodePtr.Pin()->GetActionIndex(InsertIndex + 1);
		}
	}

	if(DraggedSteps.Num() > 0)
	{
		FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
		bool bCopySteps = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

		TArray<const UDataprepActionStep*> Steps;
		Steps.Reserve(DraggedSteps.Num());

		TArray<TStrongObjectPtr<UDataprepActionStep>> StepsStrongPtr;
		StepsStrongPtr.Reserve(DraggedSteps.Num());

		TMap<UDataprepActionAsset*, TArray<int32>> ActionAssetMap;

		for(FDraggedStepEntry& DraggedStepEntry : DraggedSteps)
		{
			if(DraggedStepEntry.Get<2>().IsValid())
			{
				Steps.Add(DraggedStepEntry.Get<2>().Get());

				// If copy is not active, hold onto the steps and build list of steps to remove
				if(!bCopySteps)
				{
					StepsStrongPtr.Emplace(DraggedStepEntry.Get<2>().Get());
					ActionAssetMap.FindOrAdd(DraggedStepEntry.Get<0>().Get()).Add(DraggedStepEntry.Get<1>());
				}
			}
		}

		if(Steps.Num() > 0)
		{
			FScopedTransaction Transaction( LOCTEXT("DropOnTrack", "Add Dataprep Action") );
			bool bTransactionSuccessful = true;

			if(!bCopySteps)
			{
				int32 ActionIndex;
				for(TPair<UDataprepActionAsset*, TArray<int32>>& Entry : ActionAssetMap)
				{
					if(Entry.Key)
					{
						bTransactionSuccessful &= FDataprepCoreUtils::RemoveSteps(Entry.Key, Entry.Value, ActionIndex, false);
					}
				}
			}

			if(ActionInsertIndex >= 0 && ActionInsertIndex < TargetDataprepAsset->GetActionCount())
			{
				TargetDataprepAsset->InsertAction(Steps, ActionInsertIndex);
			}
			else
			{
				TargetDataprepAsset->AddAction(Steps);
			}
		}

		DraggedNodeWidgets.Reset();
		DraggedSteps.Reset();
	}
	else
	{
		FScopedTransaction Transaction( LOCTEXT("AddNodeFromStep", "Add Dataprep Action Node from Step") );

		bool bTransactionSuccessful = false;

		UDataprepActionAsset* Action = nullptr;

		if(ActionInsertIndex >= 0 && ActionInsertIndex < TargetDataprepAsset->GetActionCount())
		{
			bTransactionSuccessful = TargetDataprepAsset->InsertAction(ActionInsertIndex);
			Action = TargetDataprepAsset->GetAction(ActionInsertIndex);
		}
		else
		{
			ActionInsertIndex = TargetDataprepAsset->AddAction();
			Action = TargetDataprepAsset->GetAction(ActionInsertIndex);

			bTransactionSuccessful = ActionInsertIndex != INDEX_NONE;
		}

		if ( !bTransactionSuccessful )
		{
			Transaction.Cancel();
			return FReply::Unhandled();
		}

		FDataprepSchemaActionContext Context;
		Context.DataprepActionPtr = Action;

		bTransactionSuccessful = DataprepGraphOperation.Execute( Context );

		if ( !bTransactionSuccessful )
		{
			TargetDataprepAsset->RemoveAction(ActionInsertIndex);
			Transaction.Cancel();
		}
	}

	return FReply::Handled();
}

FReply FDataprepDragDropOp::DoDropOnActionAsset(UDataprepGraphActionNode* TargetActionAssetNode)
{
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	bool bCopyRequested = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	UDataprepActionAsset* TargetActionAsset = TargetActionAssetNode->GetDataprepActionAsset();

	if(DraggedSteps.Num() > 0)
	{
		for(FDraggedStepEntry& DraggedStepEntry : DraggedSteps)
		{
			if(DraggedStepEntry.Get<0>().IsValid() || DraggedStepEntry.Get<2>().IsValid())
			{
				FScopedTransaction Transaction( LOCTEXT("DropOnActionStep", "Copy/Move Dataprep Action Step") );
				bool bTransactionSuccessful = true;

				// External drag and drop
				if(!DraggedStepEntry.Get<0>().IsValid())
				{
					bTransactionSuccessful &= TargetActionAsset->AddStep( DraggedStepEntry.Get<2>().Get() ) != INDEX_NONE;
				}
				// Drag and drop within an action asset or between two action assets
				else
				{
					UDataprepActionAsset* SourceActionAsset = DraggedStepEntry.Get<0>().Get();
					check(SourceActionAsset);

					int32 StepIndex = DraggedStepEntry.Get<1>();
					check(StepIndex != INDEX_NONE);

					// Hold onto the action step in case of a move
					TStrongObjectPtr<UDataprepActionStep> SourceActionStepPtr = TStrongObjectPtr<UDataprepActionStep>( SourceActionAsset->GetStep(StepIndex).Get() );
					check(SourceActionStepPtr.IsValid());

					// source action asset differs from target action asset
					if( TargetActionAsset != SourceActionAsset)
					{
						if(!bCopyRequested)
						{
							int32 ActionIndex = INDEX_NONE;
							bTransactionSuccessful &= FDataprepCoreUtils::RemoveStep(SourceActionAsset, StepIndex, ActionIndex, false);
						}

						bTransactionSuccessful &= TargetActionAsset->AddStep( SourceActionStepPtr.Get() ) != INDEX_NONE;
					}
					else if(bCopyRequested)
					{
						bTransactionSuccessful &= TargetActionAsset->AddStep( SourceActionStepPtr.Get() ) != INDEX_NONE;
					}
					else
					{
						bTransactionSuccessful &= TargetActionAsset->MoveStep( StepIndex, TargetActionAsset->GetStepsCount() - 1 );
					}
				}

				if(!bTransactionSuccessful)
				{
					Transaction.Cancel();
				}
			}
		}
	}
	else
	{
		DropStepFromPanel(TargetActionAsset);
	}

	DraggedNodeWidgets.Reset();
	DraggedSteps.Reset();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
