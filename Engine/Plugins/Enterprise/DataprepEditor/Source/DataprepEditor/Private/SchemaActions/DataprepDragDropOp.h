// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/DataprepSchemaAction.h"

#include "GraphEditorDragDropAction.h"

class FReply;
class UDataprepActionStep;

struct FDataprepSchemaActionContext;
class SDataprepGraphActionStepNode;
class SDataprepGraphTrackNode;
class UDataprepActionAsset;
class UDataprepAsset;
class UDataprepGraphActionNode;
class UDataprepGraphActionStepNode;

// Return true if there was a modification that require a transaction
DECLARE_DELEGATE_RetVal_OneParam( bool, FDataprepGraphOperation, const FDataprepSchemaActionContext& /** Context */ )

DECLARE_DELEGATE_TwoParams( FDataprepPreDropConfirmation, const FDataprepSchemaActionContext& /** Context */, TFunction<void ()> /** CallBack To confirm drag and drop */)

/**
 * The Dataprep drag and drop is a specialized  drag and drop that can interact with the dataprep action nodes.
 * When dropped on a dataprep action node it will do a callback on the Dataprep Graph Operation.
 * If dropped on a compatible graph, the dataprep drag and drop operation will create a new dataprep action node and execute the callback that new node
 */
class FDataprepDragDropOp : public FGraphEditorDragDropAction
{
public:
	FDataprepDragDropOp();

	DRAG_DROP_OPERATOR_TYPE(FDataprepDragDropOp, FGraphEditorDragDropAction)

	static TSharedRef<FDataprepDragDropOp> New(TSharedRef<FDataprepSchemaAction> InAction);
	static TSharedRef<FDataprepDragDropOp> New(FDataprepGraphOperation&& GraphOperation);
	static TSharedRef<FDataprepDragDropOp> New(const TSharedRef<SDataprepGraphTrackNode>& InTrackNode, const TSharedRef<SDataprepGraphActionStepNode>& InDraggedNode);
	static TSharedRef<FDataprepDragDropOp> New(UDataprepActionStep* InActionStep);

	void SetHoveredDataprepActionContext(TOptional<FDataprepSchemaActionContext> Context);

	virtual FReply DroppedOnDataprepActionContext(const FDataprepSchemaActionContext& Context);

	// FGraphEditorDragDropAction Interface
	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual void OnDragged (const class FDragDropEvent& DragDropEvent ) override;
	virtual EVisibility GetIconVisible() const;
	virtual EVisibility GetErrorIconVisible() const;
	virtual void Construct() override;
	// End of FGraphEditorDragDropAction Interface

	/**
	 * Executes drop on track. A new action will inserted at the specified index.
	 * @param InsertIndex Index at which to insert the new action. The action will be inserted
	 *		  at the end if the index is not a valid one
	 * @return FReply::handled.
	 */
	FReply DoDropOnTrack(UDataprepAsset* TargetDataprepAsset, int32 InsertIndex);

	// Allow to add an extra step to the drag and drop before doing the dropping
	void SetPreDropConfirmation(FDataprepPreDropConfirmation && Confirmation);

	// Returns action step node targeted for the drop
	UDataprepGraphActionStepNode* GetDropTargetNode() const;

	void SetTrackNode(const TSharedPtr<SDataprepGraphTrackNode>& InTrackNodePtr)
	{
		TrackNodePtr = InTrackNodePtr;
	}

	bool IsValidDrop() { return bDropTargetValid; }

protected:
	typedef FGraphEditorDragDropAction Super;

	bool DoDropOnDataprepActionContext(const FDataprepSchemaActionContext& Context);

	/** Executes drop on existing action step */
	FReply DoDropOnActionStep(UDataprepGraphActionStepNode* TargetActionStepNode);

	/** Executes drop on existing action step */
	FReply DoDropOnActionAsset(UDataprepGraphActionNode* TargetActionNode);

	virtual void HoverTargetChangedWithNodes();

	TOptional<FDataprepSchemaActionContext> HoveredDataprepActionContext;

	FDataprepPreDropConfirmation DataprepPreDropConfirmation;
	FDataprepGraphOperation DataprepGraphOperation;

private:
	FText GetMessageText();

	const FSlateBrush* GetIcon() const;

	/**
	 * Drop a step from the Operation panel to an action. The step will be added or inserted.
	 * @param TargetActionAsset Action on which the insertion will be performed
	 * @param InsertIndex Index at which the insertion must occur in the existing list of steps.
	 *                    An index of -1 will trigger an addition.
	 */
	void DropStepFromPanel(UDataprepActionAsset* TargetActionAsset, int32 InsertIndex = INDEX_NONE);

private:
	typedef TTuple<TWeakObjectPtr<UDataprepActionAsset>,int32,TWeakObjectPtr<UDataprepActionStep>> FDraggedStepEntry;

	/** Graph panel associated with the Dataprep graph editor */
	TWeakPtr<SDataprepGraphTrackNode> TrackNodePtr;

	/** Array of action steps being dragged */
	TArray<TSharedRef<SDataprepGraphActionStepNode>> DraggedNodeWidgets;

	/** Array of action steps being dragged */
	TArray<FDraggedStepEntry> DraggedSteps;

	/** Offset information for the decorator widget */
	FVector2D	DecoratorAdjust;

	/** Cache last displayed text message */
	FText LastMessageText;

	bool bDropTargetItself;
};
