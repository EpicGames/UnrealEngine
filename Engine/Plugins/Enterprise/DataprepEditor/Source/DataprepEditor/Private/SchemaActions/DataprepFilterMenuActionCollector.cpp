// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepFilterMenuActionCollector.h"

// Dataprep Includes
#include "DataprepEditorLogCategory.h"
#include "DataprepMenuActionCollectorUtils.h"
#include "SchemaActions/DataprepSchemaAction.h"
#include "SelectionSystem/DataprepFilter.h"

// Engine includes
#include "DataprepActionAsset.h"

const  FText FDataprepFilterMenuActionCollector::FilterCategory = NSLOCTEXT("DataprepSelectors", "Selectors Category", "Select by");

TArray<TSharedPtr<FDataprepSchemaAction>> FDataprepFilterMenuActionCollector::CollectActions()
{
	const double  Start = FPlatformTime::Seconds();

	TArray< TSharedPtr< FDataprepSchemaAction > > Actions;

	TArray<UClass*> FilterClasses = DataprepMenuActionCollectorUtils::GetNativeChildClasses( *UDataprepFilter::StaticClass() );

	for ( UClass* FilterClass : FilterClasses )
	{
		if ( FilterClass )
		{
			UDataprepFilter* Filter = FilterClass->GetDefaultObject< UDataprepFilter >();
			if ( Filter )
			{
				UClass* FetcherClass = Filter->GetAcceptedFetcherClass().Get();
				if ( FetcherClass )
				{
					Actions.Append( DataprepMenuActionCollectorUtils::GatherMenuActionForDataprepClass( *FetcherClass
						, DataprepMenuActionCollectorUtils::FOnCreateMenuAction::CreateLambda( [this, Filter] (UClass& Class) -> TSharedPtr< FDataprepSchemaAction >
						{
							return CreateMenuActionFromClass( Class, *Filter );
						})));
				}
			}
		}
	}

	TArray<UClass*> FilterNoFetcherClasses = DataprepMenuActionCollectorUtils::GetNativeChildClasses( *UDataprepFilterNoFetcher::StaticClass() );

	for ( UClass* FilterClass : FilterNoFetcherClasses )
	{
		if ( FilterClass && !FilterClass->HasMetaData( TEXT( "Hidden" ) ) )
		{
			UDataprepFilterNoFetcher* Filter = FilterClass->GetDefaultObject< UDataprepFilterNoFetcher >();
			if ( Filter )
			{
				Actions.Append(DataprepMenuActionCollectorUtils::GatherMenuActionForDataprepClass( *FilterClass
					, DataprepMenuActionCollectorUtils::FOnCreateMenuAction::CreateLambda([this, Filter](UClass& Class) -> TSharedPtr< FDataprepSchemaAction >
					{
						return CreateMenuActionFromClass( *Filter );
					}), true ));
			}
		}
	}

	UE_LOGF( LogDataprepEditor, Log, "The discovery of the filters/fetchers and the creation of the menu actions took %f seconds.", ( FPlatformTime::Seconds() - Start ) );

	return Actions;
}

bool FDataprepFilterMenuActionCollector::ShouldAutoExpand()
{
	return false;
}

TSharedPtr<FDataprepSchemaAction> FDataprepFilterMenuActionCollector::CreateMenuActionFromClass(UClass& Class, UDataprepFilter& Filter)
{
	check( Class.IsChildOf<UDataprepFetcher>() );
	
	const UDataprepFetcher* Fetcher = static_cast< UDataprepFetcher* >( Class.GetDefaultObject() );
	if ( Fetcher )
	{
		FDataprepSchemaAction::FOnExecuteAction OnExcuteMenuAction;
		OnExcuteMenuAction.BindLambda( [FetcherClass = Fetcher->GetClass()] (const FDataprepSchemaActionContext& InContext)
		{
			UDataprepActionAsset* Action = InContext.DataprepActionPtr.Get();
			if ( Action )
			{
				int32 NewFilterIndex = Action->AddStep( FetcherClass );
				if ( InContext.StepIndex != INDEX_NONE && InContext.StepIndex != NewFilterIndex )
				{
					Action->MoveStep( NewFilterIndex, InContext.StepIndex );
				}
			}
		});

		return MakeShared< FDataprepSchemaAction >( Filter.GetFilterCategoryText()
			, Fetcher->GetDisplayFetcherName(), Fetcher->GetTooltipText()
			, GroupingPriority, Fetcher->GetAdditionalKeyword(), OnExcuteMenuAction, DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Filter
			);
	}
	
	return {};
}


TSharedPtr<FDataprepSchemaAction> FDataprepFilterMenuActionCollector::CreateMenuActionFromClass(UDataprepFilterNoFetcher& Filter)
{
	FDataprepSchemaAction::FOnExecuteAction OnExcuteMenuAction;
	OnExcuteMenuAction.BindLambda( [FilterClass = Filter.GetClass()](const FDataprepSchemaActionContext& InContext )
	{
		UDataprepActionAsset* Action = InContext.DataprepActionPtr.Get();
		if ( Action )
		{
			int32 NewFilterIndex = Action->AddStep( FilterClass );
			if ( InContext.StepIndex != INDEX_NONE && InContext.StepIndex != NewFilterIndex )
			{
				Action->MoveStep(NewFilterIndex, InContext.StepIndex);
			}
		}
	});

	return MakeShared< FDataprepSchemaAction >(Filter.GetFilterCategoryText()
		, Filter.GetDisplayFilterName(), Filter.GetTooltipText()
		, GroupingPriority, Filter.GetAdditionalKeyword(), OnExcuteMenuAction, DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Filter);
}
