// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/IDataprepMenuActionCollector.h"


class UDataprepFilter;
class UDataprepFilterNoFetcher;

struct FDataprepSchemaAction;

/**
 * Help collecting all the dataprep filter menu action
 */
class FDataprepFilterMenuActionCollector : public IDataprepMenuActionCollector
{
public:
	virtual TArray<TSharedPtr<FDataprepSchemaAction>> CollectActions() override;
	virtual bool ShouldAutoExpand() override;

	int32 GroupingPriority = 0;

	static const FText FilterCategory;

private:
	TSharedPtr<FDataprepSchemaAction> CreateMenuActionFromClass(UClass& Class, UDataprepFilter& Filter);
	TSharedPtr<FDataprepSchemaAction> CreateMenuActionFromClass(UDataprepFilterNoFetcher& Filter);
};
