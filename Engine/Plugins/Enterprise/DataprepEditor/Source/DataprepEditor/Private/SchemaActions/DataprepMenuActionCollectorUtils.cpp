// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepMenuActionCollectorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "SchemaActions/DataprepSchemaAction.h"

TArray<TSharedPtr<FDataprepSchemaAction>> DataprepMenuActionCollectorUtils::GatherMenuActionForDataprepClass(UClass& Class, FOnCreateMenuAction OnValidClassFound, bool bIncludeBaseClass)
{
	TArray< TSharedPtr< FDataprepSchemaAction > > Actions;

	// Get the native Classes
	TArray< UClass* > NativeClasses = GetNativeChildClasses( Class );
	
	if ( bIncludeBaseClass 
		 && OnValidClassFound.IsBound()
		 && !Class.HasAnyClassFlags( CLASS_CompiledFromBlueprint | NonDesiredClassFlags ) 
		 && Class.HasAllClassFlags( CLASS_Native ) )
	{
		NativeClasses.Add( &Class );
	}
	
	Actions.Reserve( NativeClasses.Num() );

	for ( UClass* ChildClass : NativeClasses )
	{
		if ( OnValidClassFound.IsBound() )
		{
			TSharedPtr< FDataprepSchemaAction > DataprepMenuAction = OnValidClassFound.Execute( *ChildClass );
			if ( DataprepMenuAction )
			{
				Actions.Emplace( MoveTemp(DataprepMenuAction) );
			}
		}
	}

	// Just a heads up, this implementation might be a bit naive. At some point a cache should be added.
	// Get the classes created by blueprints
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>( "AssetRegistry" );

	TArray< FTopLevelAssetPath > BasesClass;
	BasesClass.Add( Class.GetClassPathName() );
	TSet< FTopLevelAssetPath > Excludeds;
	TSet< FTopLevelAssetPath > ChildClassNames;
	AssetRegistryModule.Get().GetDerivedClassNames( BasesClass, Excludeds, ChildClassNames );

	TArray< FAssetData > AssetsData;
	AssetRegistryModule.Get().GetAssetsByClass( UBlueprint::StaticClass()->GetClassPathName(), AssetsData, true );
	Actions.Reserve( AssetsData.Num() );

	for ( FAssetData& AssetData : AssetsData )
	{
		FAssetDataTagMapSharedView::FFindTagResult GeneratedClassPathPtr = AssetData.TagsAndValues.FindTag( TEXT("GeneratedClass") );
		if ( GeneratedClassPathPtr.IsSet() )
		{
			const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath( GeneratedClassPathPtr.GetValue() ));
			if ( ChildClassNames.Contains(ClassObjectPath) )
			{
				UClass* ChildClass = StaticLoadClass( &Class, nullptr, *ClassObjectPath.ToString() );
				if ( ChildClass && !ChildClass->HasAnyClassFlags( NonDesiredClassFlags ) )
				{
					if ( OnValidClassFound.IsBound() )
					{
						TSharedPtr< FDataprepSchemaAction > DataprepMenuAction = OnValidClassFound.Execute( *ChildClass );
						if ( DataprepMenuAction )
						{
							DataprepMenuAction->GeneratedClassObjectPath = ClassObjectPath.ToString();
							Actions.Emplace( MoveTemp( DataprepMenuAction ) );
						}
					}
				}
			}
		}
	}

	return Actions;
}

TArray<UClass*> DataprepMenuActionCollectorUtils::GetNativeChildClasses(UClass& Class)
{
	// Get the native Classes
	TArray< UClass* > PotentialClasses;
	GetDerivedClasses( &Class, PotentialClasses, true );;

	TArray< UClass* > ValidClasses;
	ValidClasses.Reserve( PotentialClasses.Num() );

	for ( UClass* ChildClass : PotentialClasses )
	{
		if ( ChildClass && !ChildClass->HasAnyClassFlags( CLASS_CompiledFromBlueprint | NonDesiredClassFlags ) && ChildClass->HasAllClassFlags( CLASS_Native ) )
		{
			ValidClasses.Add( ChildClass );
		}
	}

	return ValidClasses;
}

