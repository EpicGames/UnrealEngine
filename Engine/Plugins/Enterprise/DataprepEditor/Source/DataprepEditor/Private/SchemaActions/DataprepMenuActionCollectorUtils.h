// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UObject/ObjectMacros.h"

struct FDataprepSchemaAction;

namespace DataprepMenuActionCollectorUtils
{
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<FDataprepSchemaAction>, FOnCreateMenuAction,	UClass&);

	/**
	 * Utils to gather the menu action from a base class 
	 * @param Class The base class from which we want to create the actions
	 * @param OnValidClassFound Callback to generate the menu action from the class
	 * @param bIncludeBaseClass Should we take into account the base class (Class)
	 */
	TArray<TSharedPtr<FDataprepSchemaAction>> GatherMenuActionForDataprepClass(UClass& Class, FOnCreateMenuAction OnValidClassFound, bool bIncludeBaseClass = false);

	TArray<UClass*> GetNativeChildClasses(UClass&);

	constexpr EClassFlags NonDesiredClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract; 

	enum EDataprepMenuActionCategory
	{
		Filter = 1,
		SelectionTransform,
		Operation
	};
}
