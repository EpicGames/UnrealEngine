// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepOperationMenuActionCollector.h"

// Dataprep Includes
#include "DataprepActionAsset.h"
#include "DataprepEditorLogCategory.h"
#include "DataprepMenuActionCollectorUtils.h"
#include "DataprepOperation.h"
#include "SchemaActions/DataprepSchemaAction.h"

// Engine includes

const FText FDataprepOperationMenuActionCollector::OperationCategory( NSLOCTEXT("DataprepOperators", "Operations Category", "Operations") );

TArray<TSharedPtr<FDataprepSchemaAction>> FDataprepOperationMenuActionCollector::CollectActions()
{
	const double  Start = FPlatformTime::Seconds();

	TArray< TSharedPtr< FDataprepSchemaAction > > Actions = DataprepMenuActionCollectorUtils::GatherMenuActionForDataprepClass( *UDataprepOperation::StaticClass()
		, DataprepMenuActionCollectorUtils::FOnCreateMenuAction::CreateRaw( this, &FDataprepOperationMenuActionCollector::CreateMenuActionFromClass ) 
		);

	UE_LOGF( LogDataprepEditor, Log, "The discovery of the operations and the creation of the menu actions took %f seconds.", ( FPlatformTime::Seconds() - Start ) );

	return Actions;
}

bool FDataprepOperationMenuActionCollector::ShouldAutoExpand()
{
	return false;
}

TSharedPtr<FDataprepSchemaAction> FDataprepOperationMenuActionCollector::CreateMenuActionFromClass(UClass& Class)
{
	check( Class.IsChildOf<UDataprepOperation>() );

	const UDataprepOperation* Operation = static_cast<UDataprepOperation*>( Class.GetDefaultObject() );
	if ( Operation )
	{
		FDataprepSchemaAction::FOnExecuteAction OnExcuteMenuAction;
		OnExcuteMenuAction.BindLambda( [Class = Operation->GetClass()] (const FDataprepSchemaActionContext& InContext)
		{
			UDataprepActionAsset* Action = InContext.DataprepActionPtr.Get();
			if ( Action )
			{
				int32 NewOperationIndex = Action->AddStep( Class );
				if ( InContext.StepIndex != INDEX_NONE && InContext.StepIndex != NewOperationIndex )
				{
					Action->MoveStep( NewOperationIndex, InContext.StepIndex);
				}
			}
		});

		return MakeShared< FDataprepSchemaAction >( Operation->GetCategory()
			, Operation->GetDisplayOperationName(), Operation->GetTooltip()
			, 0, Operation->GetAdditionalKeyword(), OnExcuteMenuAction, DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Operation
			);
	}

	return {};
}
