// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/IDataprepMenuActionCollector.h"


struct FDataprepSchemaAction;

/**
 * Help collecting all the dataprep operation menu action
 */
class FDataprepOperationMenuActionCollector : public IDataprepMenuActionCollector
{
public:
	virtual TArray<TSharedPtr<FDataprepSchemaAction>> CollectActions() override;
	virtual bool ShouldAutoExpand() override;

	static const FText OperationCategory;
private:
	TSharedPtr<FDataprepSchemaAction> CreateMenuActionFromClass(UClass& Class);
};
