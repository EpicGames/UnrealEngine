// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepSchemaAction.h"

FDataprepSchemaAction::FDataprepSchemaAction(FText InActionCategory, FText InMenuDescription, FText InToolTip, const int32 InGrouping, FText InKeywords, const FOnExecuteAction& InAction, int32 InSectionID)
	: FEdGraphSchemaAction( FText(), MoveTemp(InMenuDescription), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID )
	, ActionCategory( InActionCategory )
	, Action( InAction )
{}

bool FDataprepSchemaAction::ExecuteAction(const FDataprepSchemaActionContext& Context)
{
	return Action.ExecuteIfBound( Context );
}
