// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

#include "DataprepSchemaAction.generated.h"

/**
 * This is the context on which a dataprep menu action will operate.
 */
struct FDataprepSchemaActionContext
{

	FDataprepSchemaActionContext()
		: DataprepActionPtr()
		, DataprepActionStepPtr()
		, StepIndex( INDEX_NONE )
	{}

	// The action targeted
	TWeakObjectPtr<class UDataprepActionAsset> DataprepActionPtr;

	// The step targeted
	TWeakObjectPtr<class UDataprepActionStep> DataprepActionStepPtr;

	// Index of the step targeted (only valid if there is a action step otherwise should be INDEX_NONE)
	int32 StepIndex;
};

bool inline operator==(const FDataprepSchemaActionContext& Left, const FDataprepSchemaActionContext& Right)
{
	return Left.DataprepActionPtr == Right.DataprepActionPtr
		&& Left.DataprepActionStepPtr == Right.DataprepActionStepPtr
		&& Left.StepIndex == Right.StepIndex;
}

bool inline operator!=(const FDataprepSchemaActionContext& Left, const FDataprepSchemaActionContext& Right)
{
	return !(Left == Right);
}


/**
 * The DataprepSchemaAction is the data structure used to populate the Dataprep action menu and the Dataprep palette
 */
USTRUCT()
struct FDataprepSchemaAction : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	/** Simple type info. */
	static FName StaticGetTypeId() { static FName Type("FDataprepSchemaAction"); return Type; }
	virtual FName GetTypeId() const { return StaticGetTypeId(); }

	DECLARE_DELEGATE_OneParam(FOnExecuteAction, const FDataprepSchemaActionContext& /** Context */);

	FDataprepSchemaAction() {}

	/** 
	 * If the class associated with this action is blueprint generated,
	 * this is its path
	 */
	FString GeneratedClassObjectPath;

	FText ActionCategory;

	/**
	 * Create a DataprepSchemaAction
	 * @param InActionCategory The category in which the action will be displayed
	 * @param InMenuDescription The text displayed in the menu for this action
	 * @param InToolTip The tooltip that should be displayed for this action
	 * @param InGrouping This is a priority number that allow to override the alphabetical order. (higher value == higher in the list)
	 * @param InKeywords Allow to add some extra text for the search to match on
	 * @param InAction The function that will be executed if this action is selected
	 * @param InSectionID The section where this action belongs
	 */
	FDataprepSchemaAction(FText InActionCategory, FText InMenuDescription, FText InToolTip, const int32 InGrouping, FText InKeywords, const FOnExecuteAction& InAction, int32 InSectionID = 0);

	bool ExecuteAction(const FDataprepSchemaActionContext& Context);

private:
	FOnExecuteAction Action;
};

