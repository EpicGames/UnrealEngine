// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"

namespace DataprepSchemaActionUtils
{
	template<class GraphNodeClass>
	GraphNodeClass* SpawnEdGraphNode(UEdGraph& ParentGraph, const FVector2D& Location)
	{
		static_assert(TIsDerivedFrom< GraphNodeClass, UEdGraphNode >::IsDerived, "The node class must derive from UEdGraphNode");

		GraphNodeClass* NewNode = NewObject< GraphNodeClass >(&ParentGraph, GraphNodeClass::StaticClass(), NAME_None, RF_Transactional);
		check(NewNode != nullptr);
		NewNode->CreateNewGuid();

		NewNode->NodePosX = Location.X;
		NewNode->NodePosY = Location.Y;

		NewNode->AllocateDefaultPins();
		NewNode->PostPlacedNewNode();

		ParentGraph.Modify();
		ParentGraph.AddNode(NewNode, /*bFromUI =*/true, /*bSelectNewNode =*/true);

		return NewNode;
	}
}
