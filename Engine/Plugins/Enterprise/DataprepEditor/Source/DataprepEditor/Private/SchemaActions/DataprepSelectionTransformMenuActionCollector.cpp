// Copyright Epic Games, Inc. All Rights Reserved.

#include "SchemaActions/DataprepSelectionTransformMenuActionCollector.h"

// Dataprep Includes
#include "DataprepActionAsset.h"
#include "DataprepEditorLogCategory.h"
#include "DataprepMenuActionCollectorUtils.h"
#include "SelectionSystem/DataprepSelectionTransform.h"
#include "SchemaActions/DataprepSchemaAction.h"

// Engine includes

TArray<TSharedPtr<FDataprepSchemaAction>> FDataprepSelectionTransformMenuActionCollector::CollectActions()
{
	const double  Start = FPlatformTime::Seconds();

	TArray< TSharedPtr< FDataprepSchemaAction > > Actions = DataprepMenuActionCollectorUtils::GatherMenuActionForDataprepClass(*UDataprepSelectionTransform::StaticClass()
																															   , DataprepMenuActionCollectorUtils::FOnCreateMenuAction::CreateRaw(this, &FDataprepSelectionTransformMenuActionCollector::CreateMenuActionFromClass)
	);

	UE_LOGF(LogDataprepEditor, Log, "The discovery of the selection transforms and the creation of the menu actions took %f seconds.", (FPlatformTime::Seconds() - Start));

	return Actions;
}

bool FDataprepSelectionTransformMenuActionCollector::ShouldAutoExpand()
{
	return false;
}

TSharedPtr<FDataprepSchemaAction> FDataprepSelectionTransformMenuActionCollector::CreateMenuActionFromClass(UClass& Class)
{
	check(Class.IsChildOf<UDataprepSelectionTransform>());

	const UDataprepSelectionTransform* SelectionTransform = static_cast<UDataprepSelectionTransform*>(Class.GetDefaultObject());
	if (SelectionTransform)
	{
		FDataprepSchemaAction::FOnExecuteAction OnExcuteMenuAction;
		OnExcuteMenuAction.BindLambda([Class = SelectionTransform->GetClass()](const FDataprepSchemaActionContext& InContext)
		{
			UDataprepActionAsset* Action = InContext.DataprepActionPtr.Get();
			if (Action)
			{
				int32 NewOperationIndex = Action->AddStep(Class);
				if (InContext.StepIndex != INDEX_NONE && InContext.StepIndex != NewOperationIndex)
				{
					Action->MoveStep(NewOperationIndex, InContext.StepIndex);
				}
			}
		});

		return MakeShared< FDataprepSchemaAction >(SelectionTransform->GetCategory()
												   , SelectionTransform->GetDisplayTransformName(), SelectionTransform->GetTooltip()
												   , 0, SelectionTransform->GetAdditionalKeyword(), OnExcuteMenuAction, DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::SelectionTransform
												   );
	}

	return {};
}
