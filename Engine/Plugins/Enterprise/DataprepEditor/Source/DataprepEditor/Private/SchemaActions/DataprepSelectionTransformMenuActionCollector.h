// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/IDataprepMenuActionCollector.h"


struct FDataprepSchemaAction;

/**
 * Help collecting all the dataprep selection transform menu action
 */
class FDataprepSelectionTransformMenuActionCollector : public IDataprepMenuActionCollector
{
public:
	virtual TArray<TSharedPtr<FDataprepSchemaAction>> CollectActions() override;
	virtual bool ShouldAutoExpand() override;

	int32 GroupingPriority = 0;

private:
	TSharedPtr<FDataprepSchemaAction> CreateMenuActionFromClass(UClass& Class);
};
