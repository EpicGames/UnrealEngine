// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FDataprepSchemaAction;
struct FGraphActionListBuilderBase;

/**
 * The DataprepActionCollector is an interface for the SDataprepActionMenu
 */
class IDataprepMenuActionCollector
{
public:

	virtual ~IDataprepMenuActionCollector() {}

	/** The function that will be called to collect all the actions */
	virtual TArray<TSharedPtr<FDataprepSchemaAction>> CollectActions() = 0;

	/** Should the categories be auto expended */
	virtual bool ShouldAutoExpand() = 0;
};
