// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepAssetView.h"

#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepWidgets.h"

#include "DataprepContentConsumer.h"

#include "DetailsViewArgs.h"
#include "Engine/SCS_Node.h"
#include "Framework/Views/TableViewMetadata.h"
#include "K2Node_AddComponent.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"

#include "DetailColumnSizeData.h"
#include "Modules/ModuleManager.h"
#include "ISCSEditorUICustomization.h"
#include "SSubobjectInstanceEditor.h"
#include "SubobjectData.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SDataprepProducersWidget.h"

#define LOCTEXT_NAMESPACE "DataprepAssetView"

namespace DataprepEditorUtils
{
	FSlateFontInfo GetGlyphFont()
	{
		return FAppStyle::Get().GetFontStyle( "FontAwesome.11" );
	}
}

class FDataprepSCSEditorUICustomization : public ISCSEditorUICustomization
{
public:
	void SetHideComponentsTree(bool bInHide) { bHideComponentsTree = bInHide; }

	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override { return true; }

	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override { return true; }

	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const override { return bHideComponentsTree; }

private:
	bool bHideComponentsTree = false;
};

// Inspired from SKismetInspector::Construct
void SGraphNodeDetailsWidget::Construct(const FArguments& InArgs)
{
	SelectedActor = nullptr;

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;

	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);

	// Enable selection (copy/paste) for metadata property only

	PropertyView->GetIsCustomRowReadOnlyDelegate().BindLambda([this](const FName InRowName, const FName InParentName) -> bool
		{
			return !bCanEditProperties;
		});

	PropertyView->GetIsPropertyReadOnlyDelegate().BindLambda([this](const FPropertyAndParent& PNP) -> bool
		{
			return !bCanEditProperties && (PNP.Property.GetName() != TEXT("MetaData"));
		});

	DetailsSplitter = SNew(SSplitter)
		.MinimumSlotHeight(80.0f)
		.Orientation(Orient_Vertical)
		.Style(FAppStyle::Get(), "SplitterDark")
		.PhysicalSplitterHandleSize(2.0f)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
				.Visibility_Lambda([this]() -> EVisibility { return SelectedActor ? EVisibility::Visible : EVisibility::Collapsed; })
				[
					SAssignNew(SubobjectEditor, SSubobjectInstanceEditor)
						.ObjectContext_Lambda([this]() -> UObject* { return SelectedActor; })
						.OnSelectionUpdated(this, &SGraphNodeDetailsWidget::OnSCSEditorTreeViewSelectionChanged)
						.AllowEditing(false)
				]
		]
		+ SSplitter::Slot()
		[
			PropertyView.ToSharedRef()
		];

	SubobjectEditorUICustomization = MakeShared<FDataprepSCSEditorUICustomization>();
	SubobjectEditor->SetUICustomization(SubobjectEditorUICustomization);

	// Create the border that all of the content will get stuffed into
	ChildSlot
	[
		SNew(SVerticalBox)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("BlueprintInspector")))
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(ContextualEditingBorderWidget, SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		]
	];
}

void SGraphNodeDetailsWidget::ShowDetailsObjects(const TArray<UObject*>& Objects)
{
	bRefreshOnTick = true;
	RefreshPropertyObjects.Empty();

	// Flush content of IDetailView widget. It will be rebuilt on next tick
	PropertyView->SetObjects(TArray<UObject*>());

	RefreshPropertyObjects.Append(Objects);
}


void SGraphNodeDetailsWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRefreshOnTick)
	{
		UpdateFromObjects(RefreshPropertyObjects);
		RefreshPropertyObjects.Empty();
		bRefreshOnTick = false;
	}
}

struct FKismetSelectionInfo
{
public:
	TArray<UActorComponent*> EditableComponentTemplates;
	TArray<UObject*> ObjectsForPropertyEditing;
};

void SGraphNodeDetailsWidget::AddPropertiesRecursive(FProperty* Property)
{
	if (Property != NULL)
	{
		// Add this property
		SelectedObjectProperties.Add(Property);

		// If this is a struct or an array of structs, recursively add the child properties
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (StructProperty != NULL &&
			StructProperty->Struct != NULL)
		{
			for (TFieldIterator<FProperty> StructPropIt(StructProperty->Struct); StructPropIt; ++StructPropIt)
			{
				FProperty* InsideStructProperty = *StructPropIt;
				AddPropertiesRecursive(InsideStructProperty);
			}
		}
		else if (ArrayProperty && ArrayProperty->Inner->IsA<FStructProperty>())
		{
			AddPropertiesRecursive(ArrayProperty->Inner);
		}
	}
}

void SGraphNodeDetailsWidget::OnSCSEditorTreeViewSelectionChanged(const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
{
	if (SelectedNodes.Num() > 0)
	{
		if (SelectedActor)
		{
			TArray<UObject*> DetailsObjects;

			for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
			{
				if (SelectedNode.IsValid())
				{
					if (SelectedNode->IsRootActorNode())
					{
						// Root actor takes precedence
						DetailsObjects.Empty();
						DetailsObjects.Add(SelectedActor);
						break;
					}
					else if (SelectedNode->IsComponentNode())
					{
						FSubobjectData* Data = SelectedNode->GetDataSource();
						if (const UActorComponent* ComponentInstance = Data->FindComponentInstanceInActor(SelectedActor))
						{
							DetailsObjects.Add(const_cast<UActorComponent*>(ComponentInstance));
						}
					}
				}
			}

			PropertyView->SetObjects(DetailsObjects);
		}
	}
}

// Inspired from SKismetInspector::UpdateFromObjects
void SGraphNodeDetailsWidget::UpdateFromObjects(const TArray<UObject*>& PropertyObjects, bool bSelfUpdate)
{
	TSharedRef< SVerticalBox > ContextualEditingWidget = SNew(SVerticalBox);

	// DATAPREP_TODO: TO be revisited based on tpm's feedback. For the time being, simple view of properties
	SelectedObjects.Empty();
	FKismetSelectionInfo SelectionInfo;

	if (!bSelfUpdate)
	{
		SelectedActor = nullptr;
	}

	for (auto ObjectIt = PropertyObjects.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (UObject* Object = *ObjectIt)
		{
			if (!Object->IsValidLowLevel())
			{
				ensureMsgf(false, TEXT("Object in KismetInspector is invalid, see TTP 281915"));
				continue;
			}

			SelectedObjects.Add(Object);

			if (USCS_Node* SCSNode = Cast<USCS_Node>(Object))
			{
				// Edit the component template
				UActorComponent* NodeComponent = SCSNode->ComponentTemplate;
				if (NodeComponent != NULL)
				{
					SelectionInfo.ObjectsForPropertyEditing.Add(NodeComponent);
					SelectionInfo.EditableComponentTemplates.Add(NodeComponent);
				}
			}
			else if (UK2Node* K2Node = Cast<UK2Node>(Object))
			{
				// Edit the component template if it exists
				if (UK2Node_AddComponent* ComponentNode = Cast<UK2Node_AddComponent>(K2Node))
				{
					if (UActorComponent* Template = ComponentNode->GetTemplateFromNode())
					{
						SelectionInfo.ObjectsForPropertyEditing.Add(Template);
						SelectionInfo.EditableComponentTemplates.Add(Template);
					}
				}

				// See if we should edit properties of the node
				if (K2Node->ShouldShowNodeProperties())
				{
					SelectionInfo.ObjectsForPropertyEditing.Add(Object);
				}
			}
			else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
			{
				AActor* Owner = ActorComponent->GetOwner();
				if (Owner != NULL && Owner->HasAnyFlags(RF_ClassDefaultObject))
				{
					// We're editing a component that's owned by a CDO, so set the CDO to the property editor (so that propagation works) and then filter to just the component property that we want to edit
					SelectionInfo.ObjectsForPropertyEditing.AddUnique(Owner);
					SelectionInfo.EditableComponentTemplates.Add(ActorComponent);
				}
				else
				{
					// We're editing a component that exists outside of a CDO, so just edit the component instance directly
					SelectionInfo.ObjectsForPropertyEditing.AddUnique(ActorComponent);
				}
			}
			else
			{
				// Editing any UObject*
				SelectionInfo.ObjectsForPropertyEditing.AddUnique(Object);
			}
		}
	}

	if (!bSelfUpdate && SelectedObjects.Num() == 1)
	{
		SelectedActor = Cast<AActor>(SelectedObjects[0]);
	}

	// By default, no property filtering
	SelectedObjectProperties.Empty();

	// Add to the property filter list for any editable component templates
	if (SelectionInfo.EditableComponentTemplates.Num())
	{
		for (auto CompIt = SelectionInfo.EditableComponentTemplates.CreateIterator(); CompIt; ++CompIt)
		{
			UActorComponent* EditableComponentTemplate = *CompIt;
			check(EditableComponentTemplate != NULL);

			// Add all properties belonging to the component template class
			for (TFieldIterator<FProperty> PropIt(EditableComponentTemplate->GetClass()); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				check(Property != NULL);

				AddPropertiesRecursive(Property);
			}

			// Attempt to locate a matching property for the current component template
			for (auto ObjIt = SelectionInfo.ObjectsForPropertyEditing.CreateIterator(); ObjIt; ++ObjIt)
			{
				UObject* Object = *ObjIt;
				check(Object != NULL);

				if (Object != EditableComponentTemplate)
				{
					FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(Object->GetClass(), EditableComponentTemplate->GetFName());
					if (ObjectProperty != nullptr)
					{
						SelectedObjectProperties.Add(ObjectProperty);
					}
					else if (UActorComponent* Archetype = Cast<UActorComponent>(EditableComponentTemplate->GetArchetype()))
					{
						if (AActor* Owner = Archetype->GetOwner())
						{
							if (UClass* OwnerClass = Owner->GetClass())
							{
								AActor* OwnerCDO = CastChecked<AActor>(OwnerClass->GetDefaultObject());
								for (TFieldIterator<FObjectProperty> ObjPropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); ObjPropIt; ++ObjPropIt)
								{
									ObjectProperty = *ObjPropIt;
									check(ObjectProperty != nullptr);

									// If the property value matches the current archetype, add it as a selected property for filtering
									if (Archetype->GetClass()->IsChildOf(ObjectProperty->PropertyClass)
										&& Archetype == ObjectProperty->GetObjectPropertyValue_InContainer(OwnerCDO))
									{
										ObjectProperty = FindFProperty<FObjectProperty>(Object->GetClass(), ObjectProperty->GetFName());
										if (ObjectProperty != nullptr)
										{
											SelectedObjectProperties.Add(ObjectProperty);
											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	SubobjectEditorUICustomization->SetHideComponentsTree(SelectedActor == nullptr);

	PropertyView->SetObjects(SelectionInfo.ObjectsForPropertyEditing);

	if (SelectionInfo.ObjectsForPropertyEditing.Num() > 0)
	{
		ContextualEditingWidget->AddSlot()
		.FillHeight(0.9f)
		.VAlign(VAlign_Top)
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					DetailsSplitter.ToSharedRef()
				]
			]
		];
	}

	ContextualEditingBorderWidget->SetContent(ContextualEditingWidget);

	// Don't update component tree if this update comes from itself (avoids infinite recusrsion)
	if (!bSelfUpdate)
	{
		SubobjectEditor->UpdateTree();
	}
}

TSharedRef<ITableRow> SDataprepAssetView::OnGenerateRowForCategoryTree( TSharedRef<EDataprepCategory> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTable )
{
	TSharedPtr<ITableRow> Row;

	TSharedPtr<SWidget> ConsumerWidget;
	
	if (DataprepAssetInterfacePtr->GetConsumer())
	{
		ConsumerWidget = SNew( SDataprepDetailsView ).Object( DataprepAssetInterfacePtr->GetConsumer() );
	}
	else
	{
		ConsumerWidget = SNew( STextBlock )
			.Font( FAppStyle::GetFontStyle("BoldFont") )
			.Text( LOCTEXT( "NoConsumer", "No consumer found" ) )
			.Margin( 5.0f )
			.ColorAndOpacity( FLinearColor(1, 0, 0, 1) );
	}

	switch( InTreeNode.Get() )
	{
		case EDataprepCategory::Producers:
		{
			ProducersWidget = SNew( SDataprepProducersWidget, DataprepAssetInterfacePtr->GetProducers() )
				.ColumnSizeData( ColumnSizeData )
				.DataprepImportProducersDelegate( DataprepImportProducersDelegate )
				.DataprepImportProducersEnabledDelegate( DataprepImportProducersEnabledDelegate );

			TSharedPtr< SWidget > ProducerWrapper = SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 5.0f, 0.0f, 0.0f, 0.0f )
				[
					ProducersWidget.ToSharedRef()
				];

			TSharedPtr< SHorizontalBox > AddNewProducerWrapper = SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 8.0f, 0.0f, 0.0f, 0.0f )
				.HAlign( EHorizontalAlignment::HAlign_Right )
				[
					ProducersWidget->GetAddNewMenu().ToSharedRef()
				];

			Row = SNew( SDataprepCategoryWidget, ProducerWrapper.ToSharedRef(), InOwnerTable )
				.ColumnSizeData( ColumnSizeData )
				.Title( LOCTEXT("DataprepProducersWidget_Producers_label", "Inputs") )
				.TitleDetail( AddNewProducerWrapper.ToSharedRef() );
			
			break;
		}
		case EDataprepCategory::Consumers:
		{
			TSharedPtr< SVerticalBox > ConsumerContainer = SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				.AutoHeight()
				[
					ConsumerWidget.ToSharedRef()
				];

			TSharedPtr< SHorizontalBox > ConsumerSelectorWrapper = SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign( EHorizontalAlignment::HAlign_Right )
				[
					ConsumerSelector.ToSharedRef()
				];

			Row = SNew( SDataprepCategoryWidget, ConsumerContainer.ToSharedRef(), InOwnerTable )
				.ColumnSizeData( ColumnSizeData )
				.Title( LOCTEXT("DataprepAssetView_Consumer_label", "Output") )
				.TitleDetail( ConsumerSelectorWrapper.ToSharedRef() );

			break;
		}
		case EDataprepCategory::Parameterization:
		{
			TSharedPtr<SDataprepDetailsView> ParameterizationDetailsView;

			TSharedPtr< DataprepWidgetUtils::SConstrainedBox> ParametrizationContainer = SNew( DataprepWidgetUtils::SConstrainedBox )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.Padding( 8.0f, 5.0f, 0.0f, 0.0f )
					.AutoHeight()
					[

						SAssignNew( ParameterizationDetailsView, SDataprepDetailsView )
						.Object( DataprepAssetInterfacePtr->GetParameterizationObject() )
						.ColumnSizeData( ColumnSizeData )
						.Spacing( 10.0f )
						.ColumnPadding( true )
					]
				];

			if( DataprepAssetInterfacePtr->IsA<UDataprepAsset>() )
			{
				TWeakObjectPtr<UDataprepAsset> DataprepAsset = static_cast<UDataprepAsset*>( DataprepAssetInterfacePtr.Get() );
				OnParameterizationWasEdited = DataprepAsset->OnParameterizedObjectsStatusChanged.AddLambda( [ParameterizationDetailsView, DataprepAsset](const TSet<UObject*>* Objects )
				{
					if( Objects && Objects->Contains(DataprepAsset->GetParameterizationObject()) )
					{
						ParameterizationDetailsView->ForceRefresh();
					}
				});
			}

			Row = SNew( SDataprepCategoryWidget, ParametrizationContainer.ToSharedRef(), InOwnerTable )
				.Title( LOCTEXT("DataprepAssetView_Consumer_Parameterization", "Parameterization") )
				.ColumnSizeData( ColumnSizeData );
			
			break;
		}
	}

	return Row.ToSharedRef();
}

void SDataprepAssetView::Construct( const FArguments& InArgs, UDataprepAssetInterface* InDataprepAssetPtr )
{
	DataprepImportProducersDelegate = InArgs._DataprepImportProducersDelegate;
	DataprepImportProducersEnabledDelegate = InArgs._DataprepImportProducersEnabledDelegate;

	check( InDataprepAssetPtr );

	DataprepAssetInterfacePtr = InDataprepAssetPtr;

	DataprepAssetInterfacePtr->GetOnChanged().AddRaw( this, &SDataprepAssetView::OnDataprepAssetChanged );

	bIsChecked = true;

	ColumnSizeData = MakeShared<FDetailColumnSizeData>();

	UDataprepAssetProducers* AssetProducers = DataprepAssetInterfacePtr->GetProducers();
	check( AssetProducers );

	for(int32 Index = 0; Index < AssetProducers->GetProducersCount(); ++Index)
	{
		bIsChecked &= AssetProducers->IsProducerEnabled( Index ) && !AssetProducers->IsProducerSuperseded( Index );
	}

	for( TObjectIterator< UClass > It ; It ; ++It )
	{
		UClass* CurrentClass = (*It);

		if ( !CurrentClass->HasAnyClassFlags( CLASS_Abstract ) )
		{
			if( CurrentClass->IsChildOf( UDataprepContentConsumer::StaticClass() ) )
			{
				if( UDataprepContentConsumer* Consumer = Cast< UDataprepContentConsumer >( CurrentClass->GetDefaultObject() ) )
				{
					TSharedPtr< FString >& ConsumerDescriptionLabel = ConsumerDescriptionList.Emplace_GetRef( new FString( Consumer->GetLabel().ToString() ) );
					ConsumerDescriptionMap.Add( ConsumerDescriptionLabel, CurrentClass );

					if (DataprepAssetInterfacePtr->GetConsumer() != nullptr && DataprepAssetInterfacePtr->GetConsumer()->GetClass() == CurrentClass )
					{
						SelectedConsumerDescription = ConsumerDescriptionLabel;
					}
				}
			}
		}
	}

	// Display a combo-box if there are more than one type of consumers
	if( ConsumerDescriptionMap.Num() > 1 )
	{
		if ( !SelectedConsumerDescription.IsValid() )
		{
			SelectedConsumerDescription = MakeShared<FString>( FString() );
		}

		ConsumerSelector = SNew( STextComboBox )
		.OptionsSource( &ConsumerDescriptionList )
		.OnSelectionChanged( this, &SDataprepAssetView::OnNewConsumerSelected )
		.InitiallySelectedItem( SelectedConsumerDescription );
	}
	else
	{
		ConsumerSelector = SNullWidget::NullWidget;
	}

	TSharedRef<SDataprepDetailsView> DetailView = SNew(SDataprepDetailsView)
		.ColumnSizeData( ColumnSizeData )
		.Object( DataprepAssetInterfacePtr->GetParameterizationObject() );

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar);

	TSharedRef<SWidget> ParentWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> ParentSpacer = SNullWidget::NullWidget;
	if(UDataprepAssetInstance* DataprepInstance = Cast<UDataprepAssetInstance>(InDataprepAssetPtr))
	{
		ParentWidget = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size( FVector2D( 200, 10 ) )		
				]
			]
			+ SVerticalBox::Slot()
			//.Padding( 5.0f )
			.AutoHeight()
			//.MaxHeight( 400.f )
			[
				DataprepWidgetUtils::CreateParameterRow( SNew(SDataprepInstanceParentWidget)
					.ColumnSizeData(ColumnSizeData)
					.DataprepInstance(DataprepInstance) )
			]
		];

		ParentSpacer = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size( FVector2D( 200, 10 ) )		
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 0, 10, 0, 0 )
				.HAlign( EHorizontalAlignment::HAlign_Center )
				[
					SNew(SColorBlock)
					.Color( FLinearColor( 0.9f, 0.9f, 0.9f ) )
					.Size( FVector2D( 3000, 1 ) )
				]
			]
		];
	}
	
	Categories.Add( MakeShared<EDataprepCategory>( EDataprepCategory::Producers ) );
	Categories.Add( MakeShared<EDataprepCategory>( EDataprepCategory::Consumers ) );
	Categories.Add( MakeShared<EDataprepCategory>( EDataprepCategory::Parameterization ) );

	TSharedRef< SDataprepCategoryTree > CategoryTree = SNew( SDataprepCategoryTree )
		.TreeItemsSource( &Categories )
		.OnGetChildren( this, &SDataprepAssetView::OnGetChildrenForCategoryTree )
		.OnGenerateRow( this, &SDataprepAssetView::OnGenerateRowForCategoryTree )
		.SelectionMode( ESelectionMode::None )
		.HandleDirectionalNavigation( false );

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[

				SNew(SScrollBox)
				.ExternalScrollbar(ScrollBar)
				+ SScrollBox::Slot()
				[
					SNew(SVerticalBox)
					// Begin - Section for Dataprep parent
					+ SVerticalBox::Slot()
					.Padding( 5.0f )
					.AutoHeight()
					.MaxHeight( 400.f )
					[
						ParentWidget
					]
					+ SVerticalBox::Slot()
					[
						ParentSpacer
					]
					// End - Section for Dataprep parent
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						CategoryTree
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SBox )
				.WidthOverride( FOptionalSize( 16 ) )
				[
					ScrollBar
				]
			]
		]
	];
}

SDataprepAssetView::~SDataprepAssetView()
{
	if( UDataprepAssetInterface* DataprepAssetInterface = DataprepAssetInterfacePtr.Get() )
	{
		DataprepAssetInterface->GetOnChanged().RemoveAll( this );

		if ( OnParameterizationWasEdited.IsValid() )
		{
			if ( DataprepAssetInterfacePtr->IsA<UDataprepAsset>() )
			{
				TWeakObjectPtr<UDataprepAsset> DataprepAsset = static_cast<UDataprepAsset*>( DataprepAssetInterfacePtr.Get() );
				DataprepAsset->OnParameterizedObjectsStatusChanged.Remove( OnParameterizationWasEdited );
			}
		}
	}
}

void SDataprepAssetView::OnNewConsumerSelected( TSharedPtr<FString> NewConsumerDescription, ESelectInfo::Type SelectInfo)
{
	if ( !NewConsumerDescription.IsValid() || SelectedConsumerDescription == NewConsumerDescription )
	{
		return;
	}

	if( UDataprepAssetInterface* DataprepAsset = DataprepAssetInterfacePtr.Get() )
	{
		UClass** NewConsumerClassPtr = ConsumerDescriptionMap.Find(NewConsumerDescription);
		check(NewConsumerClassPtr);

		if( !DataprepAsset->SetConsumer( *NewConsumerClassPtr ) )
		{
			((STextComboBox*)ConsumerSelector.Get())->SetSelectedItem(SelectedConsumerDescription);
		}
		// Update SelectedConsumerDescription only, the widget displaying the consumer is updated thru notifications 
		else
		{
			SelectedConsumerDescription = NewConsumerDescription;
		}
	}
}

void SDataprepAssetView::OnDataprepAssetChanged(FDataprepAssetChangeType ChangeType)
{
	if( UDataprepAssetInterface* DataprepAsset = DataprepAssetInterfacePtr.Get() )
	{
		if( ChangeType == FDataprepAssetChangeType::ProducerModified ||
			ChangeType == FDataprepAssetChangeType::ProducerAdded ||
			ChangeType == FDataprepAssetChangeType::ProducerRemoved )
		{
			// Brute force : Regenerate the whole tree view
			ProducersWidget->Refresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
