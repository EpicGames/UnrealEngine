// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SDataprepProducersWidget.h"

#include "UObject/WeakFieldPtr.h"

class SBorder;
class SSplitter;
class STableViewBase;
class STextBlock;
class UDataprepAssetInterface;
enum class EDataprepCategory;

class IDetailsView;
class SDataprepAssetView;
class SDataprepConsumerWidget;
class SSubobjectEditor;
class FSubobjectEditorTreeNode;

class SDataprepAssetView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepAssetView) {}
	SLATE_ARGUMENT(FDataprepImportProducers, DataprepImportProducersDelegate) 
	SLATE_ARGUMENT(FDataprepImportProducersEnabled, DataprepImportProducersEnabledDelegate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepAssetInterface* InDataprepAssetPtr);

	~SDataprepAssetView();

private:
	void OnNewConsumerSelected( TSharedPtr<FString> NewConsumer, ESelectInfo::Type SelectInfo);

	/** Handles changes in the Dataprep asset */
	void OnDataprepAssetChanged(FDataprepAssetChangeType ChangeType);

	TSharedRef<ITableRow> OnGenerateRowForCategoryTree( TSharedRef<EDataprepCategory> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTable );
	void OnGetChildrenForCategoryTree( TSharedRef<EDataprepCategory> InTreeNode, TArray< TSharedRef<EDataprepCategory> >& OutChildren ) {}

	TArray<TSharedRef< EDataprepCategory >> Categories;

private:
	TWeakObjectPtr<UDataprepAssetInterface> DataprepAssetInterfacePtr;
	TSharedPtr<SDataprepProducersWidget> ProducersWidget;
	TSharedPtr< STextBlock > CheckBox;
	TArray< TSharedPtr< FString > > ConsumerDescriptionList;
	TMap< TSharedPtr< FString >, UClass* > ConsumerDescriptionMap;
	TSharedPtr< FString > SelectedConsumerDescription;
	TSharedPtr< SWidget > ConsumerSelector;
	bool bIsChecked;

	FDataprepImportProducers DataprepImportProducersDelegate;
	FDataprepImportProducersEnabled DataprepImportProducersEnabledDelegate;

	/** Container used by all splitters in the details view, so that they move in sync */
	TSharedPtr< FDetailColumnSizeData > ColumnSizeData;

	FDelegateHandle OnParameterizationWasEdited;
};

// Inspired from SKismetInspector class
class SGraphNodeDetailsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeDetailsWidget) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	void ShowDetailsObjects(const TArray<UObject*>& Objects);

	void SetCanEditProperties(bool bInCanEditProperties) { bCanEditProperties = bInCanEditProperties; };
	bool GetCanEditProperties() const { return bCanEditProperties; };
	const TArray< TWeakObjectPtr<UObject> >& GetObjectsShowInDetails() const { return SelectedObjects; };

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

private:
	/** Update the inspector window to show information on the supplied objects */
	void UpdateFromObjects(const TArray<UObject*>& PropertyObjects, bool bSelfUpdate = false);

	/** Add this property and all its child properties to SelectedObjectProperties */
	void AddPropertiesRecursive(FProperty* Property);

	void OnSCSEditorTreeViewSelectionChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode>>& SelectedNodes);

private:
	/** Property viewing widget */
	TSharedPtr<IDetailsView> PropertyView;

	/** Border widget that wraps a dynamic context-sensitive widget for editing objects that the property window is displaying */
	TSharedPtr<SBorder> ContextualEditingBorderWidget;

	/** Selected objects for this detail view */
	TArray< TWeakObjectPtr<UObject> > SelectedObjects;

	/** Set of object properties that should be visible */
	TSet<TWeakFieldPtr<FProperty> > SelectedObjectProperties;

	/** The splitter that divides object properties and components tree */
	TSharedPtr<SSplitter> DetailsSplitter;

	/** Component tree */
	TSharedPtr<SSubobjectEditor> SubobjectEditor;

	/** Customize how the component tree looks like */
	TSharedPtr<class FDataprepSCSEditorUICustomization> SubobjectEditorUICustomization;

	/** The first actor in the currently selected objects */
	AActor* SelectedActor;

	/** When TRUE, the SGraphNodeDetailsWidget needs to refresh the details view on Tick */
	bool bRefreshOnTick;

	bool bCanEditProperties;

	/** Holds the property objects that need to be displayed by the inspector starting on the next tick */
	TArray<UObject*> RefreshPropertyObjects;
};
