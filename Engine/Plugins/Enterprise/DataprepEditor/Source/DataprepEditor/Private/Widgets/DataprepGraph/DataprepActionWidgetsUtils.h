// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

namespace DataprepActionWidgetsUtils
{
	/**
	 * Fill an array with the reflexions data of a Enum for a SComboBox or others.
	 * The Tuple can described as the following (Displayed text, Tooltip and a mapping for the UEnum)
	 */
	template<class EnumType>
	void GenerateListEntriesFromEnum(TArray<TSharedPtr<TTuple<FText, FText, int32>>>& Entries)
	{
		// Generate the string matching options by reflection
		UEnum* Enum = StaticEnum< EnumType >();
		// This is the number of entry in the enum, - 1, because the last item in an enum is the _MAX item
		const int32 NumEnum = Enum->NumEnums() - 1;
		Entries.Empty( NumEnum );
		for ( int32 Index = 0; Index < NumEnum; Index++ )
		{
			// Ignore hidden enums
			const bool bShouldBeHidden = Enum->HasMetaData( TEXT("Hidden"), Index ) || Enum->HasMetaData( TEXT("Spacer"), Index );
			if ( !bShouldBeHidden )
			{
				FText DisplayText = Enum->GetDisplayNameTextByIndex( Index );
				FText TooltipText = Enum->GetToolTipTextByIndex( Index );
				Entries.Add( MakeShared< TTuple< FText, FText, int32 > >( DisplayText, TooltipText, Index ) );
			}
		}
	}
}
