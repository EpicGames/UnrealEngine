// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"



#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

class FDragDropActionNodeImpl : public FDragDropActionNode
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragDropActionNodeImpl, FDragDropActionNode)

	virtual void Construct() override
	{
		// Hide cursor when starting the drag. The track node will restore it
		MouseCursor = EMouseCursor::None;
		bCreateNewWindow = false;
		FDragDropActionNode::Construct();
	}

	// FDragDropOperation interface
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;
	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override;
	virtual FCursorReply OnCursorQuery() override { return FDragDropOperation::OnCursorQuery(); }
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override { return FDragDropOperation::GetDefaultDecorator(); }
	virtual FVector2D GetDecoratorPosition() const override { return FDragDropOperation::GetDecoratorPosition(); }
	virtual void SetDecoratorVisibility(bool bVisible) override { FDragDropOperation::SetDecoratorVisibility(bVisible); }
	virtual bool IsExternalOperation() const override { return FDragDropOperation::IsExternalOperation(); }
	virtual bool IsWindowlessOperation() const override { return FDragDropOperation::IsWindowlessOperation(); }
	// End of FDragDropOperation interface

	TSharedPtr<SDataprepGraphTrackNode> TrackNodePtr;
	TSharedPtr<SDataprepGraphBaseActionNode> ActionNodePtr;
};

TSharedRef<FDragDropActionNode> FDragDropActionNode::New(const TSharedRef<SDataprepGraphTrackNode>& InTrackNodePtr, const TSharedRef<SDataprepGraphBaseActionNode>& InDraggedNode)
{
	TSharedPtr<FDragDropActionNodeImpl> OperationImpl = MakeShared<FDragDropActionNodeImpl>();

	OperationImpl->TrackNodePtr = InTrackNodePtr;
	OperationImpl->ActionNodePtr = InDraggedNode;

	OperationImpl->Construct();

	InTrackNodePtr->OnStartNodeDrag(InDraggedNode);

	TSharedRef<FDragDropActionNode> Operation = MakeShareable(new FDragDropActionNode);
	Operation->Impl = StaticCastSharedPtr<FDragDropActionNode>(OperationImpl);

	return Operation;
}

void FDragDropActionNodeImpl::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	TrackNodePtr->OnEndNodeDrag(bDropWasHandled);

	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
}

void FDragDropActionNodeImpl::OnDragged(const FDragDropEvent& DragDropEvent)
{
	const bool bNodeDragged = TrackNodePtr->OnNodeDragged( ActionNodePtr, DragDropEvent.GetScreenSpacePosition(), DragDropEvent.GetCursorDelta() );

	MouseCursor = bNodeDragged ? EMouseCursor::None : EMouseCursor::SlashedCircle;

	FDragDropOperation::OnDragged(DragDropEvent);
}

#undef LOCTEXT_NAMESPACE
