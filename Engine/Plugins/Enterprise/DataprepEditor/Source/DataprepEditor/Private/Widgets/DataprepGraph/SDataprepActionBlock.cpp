// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepActionBlock.h"

#include "DataprepActionAsset.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorStyle.h"
#include "SchemaActions/DataprepSchemaAction.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDataprepActionBlock"

void SDataprepActionBlock::Construct(const FArguments& InArgs, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext)
{
	DataprepActionContext = InDataprepActionContext;

	const FLinearColor OutlineColor = GetOutlineColor().GetSpecifiedColor();

	ChildSlot
	[
		SNew( SVerticalBox )

		//The title of the block
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f)
		[
			SNew( SHorizontalBox )

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( FMargin( 5.f, 5.f, 5.f, 2.f ) )
			[
				GetTitleWidget()
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f, 2.f)
		[
			SNew( SSeparator )
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
			.Thickness(1.f)
			.Orientation(EOrientation::Orient_Horizontal)
			.ColorAndOpacity(FDataprepEditorStyle::GetColor("Dataprep.TextSeparator.Color"))
		]

		//The content of the block
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f)
		[
			SNew( SHorizontalBox )

			+ SHorizontalBox::Slot()
			.Padding( FMargin( 5.f, 0.f, 5.f, 0.f ) )
			[
				GetContentWidget()
			]
		]
	];
}

TSharedRef<SWidget> SDataprepActionBlock::GetBlockTitleWidget()
{
	const float DefaultPadding = FDataprepEditorStyle::GetFloat( "DataprepAction.Padding" );

	return SNew( SVerticalBox )

	//The title of the block
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding( FMargin( DefaultPadding, DefaultPadding, DefaultPadding, 0.f ) )
	[
		SNew( SConstraintCanvas )

		// The background of the title
		+ SConstraintCanvas::Slot()
		.Anchors( FAnchors( 0.f, 0.f, 1.f, 1.f ) )
		.Offset( FMargin() )
		[
			GetTitleBackgroundWidget()
		]

		// The title of the block
		+ SConstraintCanvas::Slot()
		.Anchors( FAnchors( 0.5f, 0.5f, 0.5f, 0.5f ) )
		.Offset( FMargin() )
		.AutoSize( true )
		[
			GetTitleWidget()
		]
	];
}

FSlateColor SDataprepActionBlock::GetOutlineColor() const
{
	return FDataprepEditorStyle::GetColor( "DataprepAction.OutlineColor" );
}

FText SDataprepActionBlock::GetBlockTitle() const
{
	return FText::FromString( TEXT("Default Action Block Title") );
}

TSharedRef<SWidget> SDataprepActionBlock::GetTitleWidget()
{
	const float DefaultPadding = FDataprepEditorStyle::GetFloat( "DataprepAction.Padding" );

	return SNew( STextBlock )
		.Text( GetBlockTitle() )
		.TextStyle( &FDataprepEditorStyle::GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.TitleTextBlockStyle" ) )
		.ColorAndOpacity( FLinearColor( 1.f, 1.f, 1.f ) )
		.Margin( FMargin( DefaultPadding ) )
		.Justification( ETextJustify::Center );
}

TSharedRef<SWidget> SDataprepActionBlock::GetTitleBackgroundWidget()
{
	return SNew( SColorBlock ).Color( GetOutlineColor().GetSpecifiedColor() );
}

TSharedRef<SWidget> SDataprepActionBlock::GetContentWidget()
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDataprepActionBlock::GetContentBackgroundWidget()
{
	return SNew( SColorBlock )
		.Color( FDataprepEditorStyle::GetColor("DataprepActionBlock.ContentBackgroundColor.Old") );
}

void SDataprepActionBlock::PopulateMenuBuilder(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection( FName( TEXT("NodeSection") ), LOCTEXT("NodeSection", "Common") );
	{
		FUIAction DeleteAction;
		DeleteAction.ExecuteAction.BindSP( this, &SDataprepActionBlock::DeleteStep );

		TSharedPtr<FUICommandInfo> DeleteCommand = FGenericCommands::Get().Delete;
		MenuBuilder.AddMenuEntry( DeleteCommand->GetLabel(),
			DeleteCommand->GetDescription(),
			DeleteCommand->GetIcon(),
			DeleteAction );
	}
	MenuBuilder.EndSection();
}

void SDataprepActionBlock::DeleteStep()
{
	if ( FDataprepSchemaActionContext* ActionContext = DataprepActionContext.Get() )
	{
		if ( UDataprepActionAsset* ActionAsset = ActionContext->DataprepActionPtr.Get() )
		{
			FScopedTransaction Transaction( LOCTEXT("DeleteStepTransaction", "Remove step from action") );

			int32 ActionIndex = INDEX_NONE;
			if( !FDataprepCoreUtils::RemoveStep( ActionAsset, ActionContext->StepIndex, ActionIndex ) )
			{
				Transaction.Cancel();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
