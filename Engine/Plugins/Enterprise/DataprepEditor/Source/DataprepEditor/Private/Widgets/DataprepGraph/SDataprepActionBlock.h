// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FDataprepSchemaActionContext;
struct FTextBlockStyle;

/**
 * The SDataprepActionBlock is the base class that is used for the block that compose a dataprep action
 */
class SDataprepActionBlock : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDataprepActionBlock) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FDataprepSchemaActionContext>& DataprepActionContext);

	// SWidget Interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();
		return FVector2D( FMath::Max(280.f, ChildSize.X) , ChildSize.Y );
	}
	// End of SWidget Interface

	/**
	 * Return the outline color
	 */
	virtual FSlateColor GetOutlineColor() const;

public:

	/** Return the block title */
	TSharedRef<SWidget> GetBlockTitleWidget();

	/**
	 * Return the title widget
	 */
	virtual TSharedRef<SWidget> GetTitleWidget();

protected:
	/**
	 * Return the block title.
	 * This is used in the implementation of SDataprepActionBlock::GetTitleWidget
	 */
	virtual FText GetBlockTitle() const;

	/**
	 * Get title background widget
	 */
	TSharedRef<SWidget> GetTitleBackgroundWidget();

	/**
	 * Return the content widget
	 */
	virtual TSharedRef<SWidget> GetContentWidget();

	/**
	 * Get content background widget
	 */
	TSharedRef<SWidget> GetContentBackgroundWidget();

	/**
	 * Populate the right click menu by overriding this function.
	 * Call the parent implementation if you want the full menu.
	 */
	virtual void PopulateMenuBuilder(class FMenuBuilder& MenuBuilder);

private:

	/**
	 * Delete this step
	 */
	void DeleteStep();

	TSharedPtr<FDataprepSchemaActionContext> DataprepActionContext;
};
