// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepActionSteps.h"

#include "DataprepActionAsset.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorStyle.h"
#include "DataprepOperation.h"
#include "Layout/WidgetPath.h"
#include "SchemaActions/DataprepAllMenuActionCollector.h"
#include "SchemaActions/DataprepDragDropOp.h"
#include "SelectionSystem/DataprepFilter.h"
#include "SelectionSystem/DataprepSelectionTransform.h"
#include "Widgets/DataprepGraph/SDataprepFilter.h"
#include "Widgets/DataprepGraph/SDataprepSelectionTransform.h"
#include "Widgets/DataprepGraph/SDataprepOperation.h"
#include "Widgets/SDataprepActionMenu.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/SBoxPanel.h"


namespace DataprepActionStepsWidgetUtils
{
	TSharedRef<SWidget> CreateActionBlock(const TSharedRef<FDataprepSchemaActionContext>& InStepData)
	{
		if ( UDataprepActionStep* ActionStep = InStepData->DataprepActionStepPtr.Get())
		{
			UDataprepParameterizableObject* StepObject = ActionStep->GetStepObject();
			UClass* StepType = FDataprepCoreUtils::GetTypeOfActionStep( StepObject );
			if ( StepType == UDataprepOperation::StaticClass() )
			{
				UDataprepOperation* Operation = static_cast<UDataprepOperation*>( StepObject );
				return SNew( SDataprepOperation, Operation, InStepData );
			}
			else if ( StepType == UDataprepSelectionTransform::StaticClass() )
			{
				UDataprepSelectionTransform* SelectionTransform = static_cast<UDataprepSelectionTransform*>( StepObject );
				return SNew( SDataprepSelectionTransform, SelectionTransform, InStepData );
			}
			else if ( StepType == UDataprepFilter::StaticClass() )
			{
				UDataprepFilter* Filter = static_cast<UDataprepFilter*>( StepObject );
				return SNew( SDataprepFilter, *Filter, InStepData );
			}
			else if ( StepType == UDataprepFilterNoFetcher::StaticClass() )
			{
				UDataprepFilterNoFetcher* Filter = static_cast<UDataprepFilterNoFetcher*>( StepObject );
				return SNew( SDataprepFilterNoFetcher, *Filter, InStepData );
			}
		}
		return SNullWidget::NullWidget;
	}
}

void SDataprepActionStep::Construct(const FArguments& InArgs, const TSharedRef<FDataprepSchemaActionContext>& InStepData)
{
	const ISlateStyle* DataprepEditorStyle = FSlateStyleRegistry::FindSlateStyle( FDataprepEditorStyle::GetStyleSetName() );
	check( DataprepEditorStyle );
	const float DefaultPadding = DataprepEditorStyle->GetFloat( "DataprepActionStep.Padding" );

	StepData = InStepData;

	ChildSlot
	[
		SNew(SBox)
		.Padding( FMargin( 0.f/*DefaultPadding, DefaultPadding, DefaultPadding, 0.f*/ ) )
		.Content()
		[
			DataprepActionStepsWidgetUtils::CreateActionBlock( InStepData )
		]
	];
}

FReply SDataprepActionStep::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// block the SGraphPanel events unfortunately.
	return FReply::Unhandled();
}

FReply SDataprepActionStep::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check( StepData );

	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		TUniquePtr<IDataprepMenuActionCollector> ActionCollector = MakeUnique<FDataprepAllMenuActionCollector>();

		FDataprepSchemaActionContext ActionContext;
		ActionContext.DataprepActionPtr = StepData->DataprepActionPtr;
		ActionContext.DataprepActionStepPtr = StepData->DataprepActionStepPtr;
		ActionContext.StepIndex = StepData->StepIndex;

		TSharedRef<SDataprepActionMenu> ActionMenu = SNew( SDataprepActionMenu, MoveTemp( ActionCollector ) )
			.TransactionText( NSLOCTEXT("SDataprepActionStep", "AddingAStep", "Add a Step to Action") )
			.DataprepActionContext( MoveTemp( ActionContext ) );

		// Summon the context menu to add a step to the action
		TSharedPtr<IMenu> Menu = FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			ActionMenu,
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

		if (Menu.IsValid() && Menu->GetOwnedWindow().IsValid())
		{
			FSlateApplication::Get().SetKeyboardFocus( ActionMenu->GetFilterTextBox(), EFocusCause::WindowActivate );
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDataprepActionStep::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	check( StepData );
	if ( TSharedPtr<FDataprepDragDropOp> DataprepDragDropOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>() )
	{
		FDataprepSchemaActionContext ActionContext;
		ActionContext.DataprepActionPtr = StepData->DataprepActionPtr;
		ActionContext.DataprepActionStepPtr = StepData->DataprepActionStepPtr;
		ActionContext.StepIndex = StepData->StepIndex;
		DataprepDragDropOp->SetHoveredDataprepActionContext( ActionContext );
	}
}

void SDataprepActionStep::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDataprepDragDropOp> DataprepDragDropOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>())
	{
		DataprepDragDropOp->SetHoveredDataprepActionContext( TOptional<FDataprepSchemaActionContext>() );
	}
}

FReply SDataprepActionStep::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDataprepDragDropOp> DataprepDragDropOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>())
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SDataprepActionStep::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	check( StepData );
	if (TSharedPtr<FDataprepDragDropOp> DataprepDragDropOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>())
	{
		return DataprepDragDropOp->DroppedOnDataprepActionContext( *StepData.Get() );
	}
	return FReply::Unhandled();
}


void SDataprepActionSteps::Construct(const FArguments& InArgs, UDataprepActionAsset* InDataprepAction)
{
	SetCursor( EMouseCursor::Default );

	DataprepActionPtr = InDataprepAction;

	const ISlateStyle* DataprepEditorStyle = FSlateStyleRegistry::FindSlateStyle( FDataprepEditorStyle::GetStyleSetName() );
	check( DataprepEditorStyle );
	const float DefaultPadding = DataprepEditorStyle->GetFloat( "DataprepAction.Padding" );

	SAssignNew( StepsList, SVerticalBox );
	Refresh();

	ChildSlot
	[
		SNew(SBox)
		.Padding( FMargin( 3 * DefaultPadding ) )
		.Content()
		[
			SNew( SConstraintCanvas )
	
			// The outline. This is done by a background image
			+ SConstraintCanvas::Slot()
			.Anchors( FAnchors( 0.f, 0.f, 1.f, 1.f ) )
			.Offset( FMargin() )
			[
				SNew( SColorBlock )
				.Color( DataprepEditorStyle->GetColor( "DataprepAction.OutlineColor" ) )
			]

			// The background color
			+ SConstraintCanvas::Slot()
			.Anchors( FAnchors( 0.f, 0.f, 1.f, 1.f ) )
			.Offset( FMargin( DefaultPadding ) )
			[
				SNew( SColorBlock )
				.Color( DataprepEditorStyle->GetColor( "DataprepActionSteps.BackgroundColor" ) )
			]

			// The Steps
			+ SConstraintCanvas::Slot()
			.Anchors( FAnchors( 0.f, 0.f, 1.f, 1.f ) )
			.Offset( FMargin() )
			.AutoSize( true )
			[
				StepsList.ToSharedRef()
			]
		]
	];

	if ( InDataprepAction )
	{
		InDataprepAction->GetOnStepsOrderChanged().AddSP( this, &SDataprepActionSteps::OnStepsOrderChanged );
	}
}

void SDataprepActionSteps::OnStepsOrderChanged()
{
	Refresh();
}

void SDataprepActionSteps::Refresh()
{
	UDataprepActionAsset* DataprepAction = DataprepActionPtr.Get();
	if ( DataprepAction && StepsList )
	{
		const int32 NumberOfStep = DataprepAction->GetStepsCount();
		StepsList->ClearChildren();

		for ( int32 i = 0; i < NumberOfStep; i++ )
		{
			TSharedRef< FDataprepSchemaActionContext > StepData = MakeShared< FDataprepSchemaActionContext >();
			StepData->DataprepActionPtr = DataprepActionPtr;
			StepData->DataprepActionStepPtr =  DataprepAction->GetStep( i );
			StepData->StepIndex = i;

			StepsList->AddSlot()
				.AutoHeight()
				[
					SNew( SDataprepActionStep, StepData )
				];
		}

		//Add the last empty SDataprepactionStep as zone to add at the end of action
		TSharedRef< FDataprepSchemaActionContext > StepData = MakeShared< FDataprepSchemaActionContext >();
		StepData->DataprepActionPtr = DataprepActionPtr;
		StepData->DataprepActionStepPtr =  nullptr;
		StepData->StepIndex = INDEX_NONE;
		StepsList->AddSlot()
			.AutoHeight()
			[
				SNew( SDataprepActionStep, StepData )
			];
	}
}

