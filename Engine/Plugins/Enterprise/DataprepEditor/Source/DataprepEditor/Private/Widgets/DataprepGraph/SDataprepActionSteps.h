// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

class UDataprepActionAsset;
class UDataprepActionStep;

struct FDataprepSchemaActionContext;

/**
 * The widget that display and manage zone where a action step is displayed
 */
class SDataprepActionStep : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepActionStep) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedRef<FDataprepSchemaActionContext>& InStepData);

	// SWidget Interface
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget Interface


private:
	TSharedPtr<FDataprepSchemaActionContext> StepData;
};

/**
 * This is the widget that display the steps of dataprep action
 */
class SDataprepActionSteps : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepActionSteps) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepActionAsset* InDataprepAction);

private:

	void OnStepsOrderChanged();

	// Update the display of the steps
	void Refresh();

	TWeakObjectPtr<UDataprepActionAsset> DataprepActionPtr;

	TSharedPtr<class SVerticalBox> StepsList;

	TArray<TSharedPtr<FDataprepSchemaActionContext>> Steps;
};
