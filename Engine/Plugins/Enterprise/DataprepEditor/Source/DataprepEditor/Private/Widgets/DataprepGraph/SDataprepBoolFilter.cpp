// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepBoolFilter.h"



void SDataprepBoolFilter::Construct(const FArguments& InArgs, UDataprepBoolFilter& InFilter)
{
}
