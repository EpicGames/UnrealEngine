// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UDataprepBoolFilter;

class SDataprepBoolFilter : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDataprepBoolFilter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepBoolFilter& InFilter);
};
