// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepFilter.h"

#include "DataprepEditorStyle.h"
#include "SelectionSystem/DataprepBoolFilter.h"
#include "SelectionSystem/DataprepFloatFilter.h"
#include "SelectionSystem/DataprepIntegerFilter.h"
#include "SelectionSystem/DataprepStringFilter.h"
#include "SelectionSystem/DataprepStringsArrayFilter.h"
#include "SelectionSystem/DataprepObjectSelectionFilter.h"
#include "Widgets/DataprepGraph/SDataprepBoolFilter.h"
#include "Widgets/DataprepGraph/SDataprepFloatFilter.h"
#include "Widgets/DataprepGraph/SDataprepIntegerFilter.h"
#include "Widgets/DataprepGraph/SDataprepStringFilter.h"
#include "Widgets/DataprepGraph/SDataprepObjectSelectionFilter.h"
#include "Widgets/DataprepWidgets.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "SDataprepFilter"

void SDataprepFilter::Construct(const FArguments& InArgs, UDataprepFilter& InFilter, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext)
{
	Filter = &InFilter;

	TAttribute<FText> TooltipTextAttribute = MakeAttributeSP( this, &SDataprepFilter::GetTooltipText );
	SetToolTipText( TooltipTextAttribute );

	bIsPreviewed = InArgs._IsPreviewed;

	SDataprepActionBlock::Construct( SDataprepActionBlock::FArguments(), InDataprepActionContext );
}

void SDataprepFilter::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ( DetailsView.IsValid() && Filter )
	{
		if ( UDataprepFetcher* Fetcher = Filter->GetFetcher() )
		{
			DetailsView->SetObjectToDisplay( *Fetcher );
		}
	}
}

FSlateColor SDataprepFilter::GetOutlineColor() const
{
	if ( bIsPreviewed )
	{
		return FDataprepEditorStyle::GetColor( "Graph.ActionStepNode.PreviewColor" );
	}

	return FDataprepEditorStyle::GetColor( "DataprepActionStep.Filter.OutlineColor" );
}

FText SDataprepFilter::GetBlockTitle() const
{
	if ( Filter )
	{
		UDataprepFetcher* Fetcher = Filter->GetFetcher();
		if ( Fetcher )
		{
			if ( Filter->IsExcludingResult() )
			{
				return FText::Format( LOCTEXT("ExcludingFilterTitle", "Exclude by {0}"), { Fetcher->GetNodeDisplayFetcherName() } );
			}
			else
			{
				return FText::Format( LOCTEXT("SelectingFilterTitle", "Filter by {0}"), { Fetcher->GetNodeDisplayFetcherName() });
			}
		}
	}
	return LOCTEXT("DefaultFilterTitle", "Unknow Filter Type");
}

TSharedRef<SWidget> SDataprepFilter::GetTitleWidget()
{
	const ISlateStyle* DataprepEditorStyle = FSlateStyleRegistry::FindSlateStyle( FDataprepEditorStyle::GetStyleSetName() );
	check( DataprepEditorStyle );
	const float DefaultPadding = DataprepEditorStyle->GetFloat( "DataprepAction.Padding" );
	TSharedRef<SWidget> DefaultTitle = SNew( STextBlock )
		.Text( this, &SDataprepFilter::GetBlockTitle )
		.TextStyle( &DataprepEditorStyle->GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.TitleTextBlockStyle" ) )
		.ColorAndOpacity( FLinearColor( 1.f, 1.f, 1.f ) )
		.Margin( FMargin( DefaultPadding ) )
		.Justification( ETextJustify::Center );


	if ( bIsPreviewed )
	{
		return SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				DefaultTitle
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 2.0f, 4.0f, 0.0f, 0.0f ) )
			[
				SNew( STextBlock )
				.TextStyle(  &DataprepEditorStyle->GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.PreviewTextBlockStyle" ) )
				.Text( LOCTEXT("PreviewLabel", "Previewing") )
			];
	}

	return DefaultTitle;
}

TSharedRef<SWidget> SDataprepFilter::GetContentWidget()
{
	TSharedPtr< SWidget > FilterWidget = SNullWidget::NullWidget;

	if ( Filter )
	{
		UClass* Class = Filter->GetClass();
		// This down casting implementation is faster then using Cast<UDataprepStringFilter>( Filter ) 
		if ( Class ==  UDataprepStringFilter::StaticClass() )
		{
			SAssignNew( FilterWidget, SDataprepStringFilter< UDataprepStringFilter >, *StaticCastPtr< UDataprepStringFilter>( Filter ) );
		}
		else if (Class == UDataprepStringsArrayFilter::StaticClass())
		{
			SAssignNew(FilterWidget, SDataprepStringFilter< UDataprepStringsArrayFilter >, *StaticCastPtr<UDataprepStringsArrayFilter>(Filter));
		}
		else if ( Class == UDataprepBoolFilter::StaticClass() )
		{
			SAssignNew( FilterWidget, SDataprepBoolFilter, *StaticCastPtr< UDataprepBoolFilter >( Filter ) );
		}
		else if (Class == UDataprepFloatFilter::StaticClass())
		{
			SAssignNew( FilterWidget, SDataprepFloatFilter, *StaticCastPtr< UDataprepFloatFilter >( Filter ) );
		}
		else if (Class == UDataprepIntegerFilter::StaticClass())
		{
			SAssignNew(FilterWidget, SDataprepIntegerFilter, *StaticCastPtr<UDataprepIntegerFilter>(Filter));
		}
	}

	return SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FilterWidget.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew( DetailsView, SDataprepDetailsView )
			.Object( Filter ? Filter->GetFetcher() : nullptr )
		];
}

void SDataprepFilter::PopulateMenuBuilder(FMenuBuilder& MenuBuilder)
{
	SDataprepActionBlock::PopulateMenuBuilder( MenuBuilder );

	MenuBuilder.BeginSection( FName( TEXT("FilterSection") ), LOCTEXT("FilterSection", "Filter") );
	{
		FUIAction InverseFilterAction;
		InverseFilterAction.ExecuteAction.BindSP( this, &SDataprepFilter::InverseFilter );
		MenuBuilder.AddMenuEntry( LOCTEXT("InverseFilter", "Inverse Selection"), 
			LOCTEXT("InverseFilterTooltip", "Inverse the resulting selection"),
			FSlateIcon(),
			InverseFilterAction );
	}
	MenuBuilder.EndSection();
}

void SDataprepFilter::InverseFilter()
{
	if ( Filter )
	{
		FScopedTransaction Transaction( LOCTEXT("InverseFilterTransaction", "Inverse the filter") );
		Filter->SetIsExcludingResult( !Filter->IsExcludingResult() );
	}
}

FText SDataprepFilter::GetTooltipText() const
{
	FText TooltipText;
	if ( Filter )
	{
		UDataprepFetcher* Fetcher = Filter->GetFetcher();
		if ( Fetcher )
		{
			TooltipText = Fetcher->GetTooltipText();
		}
	}
	return TooltipText;
}

void SDataprepFilter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}


void SDataprepFilterNoFetcher::Construct(const FArguments& InArgs, UDataprepFilterNoFetcher& InFilter, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext)
{
	Filter = &InFilter;

	TAttribute<FText> TooltipTextAttribute = MakeAttributeSP( this, &SDataprepFilterNoFetcher::GetTooltipText );
	SetToolTipText( TooltipTextAttribute );

	bIsPreviewed = InArgs._IsPreviewed;

	SDataprepActionBlock::Construct( SDataprepActionBlock::FArguments(), InDataprepActionContext );
}

void SDataprepFilterNoFetcher::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ( DetailsView.IsValid() && Filter )
	{
		DetailsView->SetObjectToDisplay( *Filter );
	}
}

FSlateColor SDataprepFilterNoFetcher::GetOutlineColor() const
{
	if ( bIsPreviewed )
	{
		return FDataprepEditorStyle::GetColor( "Graph.ActionStepNode.PreviewColor" );
	}

	return FDataprepEditorStyle::GetColor( "DataprepActionStep.Filter.OutlineColor" );
}

FText SDataprepFilterNoFetcher::GetBlockTitle() const
{
	if ( Filter )
	{
		if ( Filter->IsExcludingResult() )
		{
			return FText::Format( LOCTEXT("ExcludingFilterTitle", "Exclude by {0}"), { Filter->GetNodeDisplayFilterName() } );
		}
		else
		{
			return FText::Format( LOCTEXT("SelectingFilterTitle", "Filter by {0}"), { Filter->GetNodeDisplayFilterName() });
		}
	}
	return LOCTEXT("DefaultFilterTitle", "Unknow Filter Type");
}

TSharedRef<SWidget> SDataprepFilterNoFetcher::GetTitleWidget()
{
	const ISlateStyle* DataprepEditorStyle = FSlateStyleRegistry::FindSlateStyle( FDataprepEditorStyle::GetStyleSetName() );
	check( DataprepEditorStyle );
	const float DefaultPadding = DataprepEditorStyle->GetFloat( "DataprepAction.Padding" );
	TSharedRef<SWidget> DefaultTitle = SNew( STextBlock )
		.Text( this, &SDataprepFilterNoFetcher::GetBlockTitle )
		.TextStyle( &DataprepEditorStyle->GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.TitleTextBlockStyle" ) )
		.ColorAndOpacity( FLinearColor( 1.f, 1.f, 1.f ) )
		.Margin( FMargin( DefaultPadding ) )
		.Justification( ETextJustify::Center );


	if ( bIsPreviewed )
	{
		return SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				DefaultTitle
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 2.0f, 4.0f, 0.0f, 0.0f ) )
			[
				SNew( STextBlock )
				.TextStyle(  &DataprepEditorStyle->GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.PreviewTextBlockStyle" ) )
				.Text( LOCTEXT("PreviewLabel", "Previewing") )
			];
	}

	return DefaultTitle;
}

TSharedRef<SWidget> SDataprepFilterNoFetcher::GetContentWidget()
{
	TSharedPtr< SWidget > FilterWidget = SNullWidget::NullWidget;

	if ( Filter )
	{
		UClass* Class = Filter->GetClass();
		if ( Class == UDataprepObjectSelectionFilter::StaticClass() )
		{
			SAssignNew(FilterWidget, SDataprepObjectSelectionFilter, *StaticCastPtr< UDataprepObjectSelectionFilter>( Filter ) );
		}
	}

	return SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FilterWidget.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew( DetailsView, SDataprepDetailsView )
			.Object( Filter )
		];
}

void SDataprepFilterNoFetcher::PopulateMenuBuilder(FMenuBuilder& MenuBuilder)
{
	SDataprepActionBlock::PopulateMenuBuilder( MenuBuilder );

	MenuBuilder.BeginSection( FName( TEXT("FilterSection") ), LOCTEXT("FilterSection", "Filter") );
	{
		FUIAction InverseFilterAction;
		InverseFilterAction.ExecuteAction.BindSP( this, &SDataprepFilterNoFetcher::InverseFilter );
		MenuBuilder.AddMenuEntry( LOCTEXT("InverseFilter", "Inverse Selection"), 
			LOCTEXT("InverseFilterTooltip", "Inverse the resulting selection"),
			FSlateIcon(),
			InverseFilterAction );
	}
	MenuBuilder.EndSection();
}

void SDataprepFilterNoFetcher::InverseFilter()
{
	if ( Filter )
	{
		FScopedTransaction Transaction( LOCTEXT("InverseFilterTransaction", "Inverse the filter") );
		Filter->SetIsExcludingResult( !Filter->IsExcludingResult() );
	}
}

FText SDataprepFilterNoFetcher::GetTooltipText() const
{
	FText TooltipText;
	if ( Filter )
	{
		TooltipText = Filter->GetTooltipText();
	}
	return TooltipText;
}

void SDataprepFilterNoFetcher::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}

#undef LOCTEXT_NAMESPACE
