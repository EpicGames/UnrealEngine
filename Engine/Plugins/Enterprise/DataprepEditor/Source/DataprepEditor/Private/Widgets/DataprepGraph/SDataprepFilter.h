// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DataprepGraph/SDataprepActionBlock.h"

#include "UObject/GCObject.h"

class UDataprepFilter;
class UDataprepFilterNoFetcher;

struct FDataprepSchemaActionContext;


class SDataprepFilter : public SDataprepActionBlock, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SDataprepFilter)
		: _IsPreviewed(false)
	{}
		SLATE_ARGUMENT( bool, IsPreviewed )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepFilter& InFilter, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;	

protected:

	// SDataprepActionBlock interface
	virtual FSlateColor GetOutlineColor() const override;
	virtual FText GetBlockTitle() const override;
	virtual TSharedRef<SWidget> GetTitleWidget() override;
	virtual TSharedRef<SWidget> GetContentWidget() override;
	virtual void PopulateMenuBuilder(class FMenuBuilder& MenuBuilder) override;
	//~ end of SDataprepActionBlock interface

private:

	void InverseFilter();

	FText GetTooltipText() const;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepFilter");
	}

	TSharedPtr<class SDataprepDetailsView> DetailsView;

	TObjectPtr<UDataprepFilter> Filter = nullptr;

	bool bIsPreviewed;
};

class SDataprepFilterNoFetcher : public SDataprepActionBlock, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SDataprepFilterNoFetcher)
		: _IsPreviewed(false)
	{}
		SLATE_ARGUMENT( bool, IsPreviewed )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepFilterNoFetcher& InFilter, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;	

protected:

	// SDataprepActionBlock interface
	virtual FSlateColor GetOutlineColor() const override;
	virtual FText GetBlockTitle() const override;
	virtual TSharedRef<SWidget> GetTitleWidget() override;
	virtual TSharedRef<SWidget> GetContentWidget() override;
	virtual void PopulateMenuBuilder(class FMenuBuilder& MenuBuilder) override;
	//~ end of SDataprepActionBlock interface

private:

	void InverseFilter();

	FText GetTooltipText() const;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepFilterNoFetcher");
	}

	TSharedPtr<class SDataprepDetailsView> DetailsView;

	TObjectPtr<UDataprepFilterNoFetcher> Filter = nullptr;

	bool bIsPreviewed;
};
