// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepFloatFilter.h"

#include "DataprepAsset.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorUtils.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SelectionSystem/DataprepFloatFilter.h"
#include "Widgets/DataprepGraph/DataprepActionWidgetsUtils.h"
#include "Widgets/DataprepWidgets.h"
#include "Widgets/Parameterization/SDataprepParameterizationLinkIcon.h"

#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "DataprepFloatFilter"

namespace SDataprepFloatFilterUtils
{
	void PostEditChainProperty(UDataprepFloatFilter* Filter, FName PropertyName)
	{
		if ( Filter )
		{
			FProperty* Property = Filter->GetClass()->FindPropertyByName( PropertyName );
			check( Property );

			FEditPropertyChain EditChain;
			EditChain.AddHead( Property );
			EditChain.SetActivePropertyNode( Property );
			FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
			FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
			Filter->PostEditChangeChainProperty( EditChangeChainEvent );
		}
	}
};

void SDataprepFloatFilter::Construct(const FArguments& InArgs, UDataprepFloatFilter& InFilter)
{
	Filter = &InFilter;
	OldEqualValue = Filter->GetEqualValue();
	OldTolerance = Filter->GetTolerance();

	DataprepActionWidgetsUtils::GenerateListEntriesFromEnum< EDataprepFloatMatchType >( FloatMatchingOptions );

	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( Filter ) )
	{
		UClass* FilterClass = InFilter.GetClass();

		{
			FName PropertyName = FName( TEXT("FloatMatchingCriteria") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			MatchingCriteriaParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		{
			FName PropertyName = FName( TEXT("EqualValue") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			EqualValueParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		{
			FName PropertyName = FName( TEXT("Tolerance") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			ToleranceParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		OnParameterizationStatusForObjectsChangedHandle = DataprepAsset->OnParameterizedObjectsStatusChanged.AddSP( this, &SDataprepFloatFilter::OnParameterizationStatusForObjectsChanged );
	}

	UpdateVisualDisplay();
}

SDataprepFloatFilter::~SDataprepFloatFilter()
{
	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( Filter ) )
	{
		DataprepAsset->OnParameterizedObjectsStatusChanged.Remove( OnParameterizationStatusForObjectsChangedHandle );
	}
}

void SDataprepFloatFilter::UpdateVisualDisplay()
{
	TSharedPtr<SHorizontalBox> MatchingCriteriaHorizontalBox;
	TSharedPtr<SHorizontalBox> EqualValueHorizontalBox;
	TSharedPtr<SHorizontalBox> ToleranceHorizontalBox;

	ChildSlot
	[
		SNew( SBox )
		.MinDesiredWidth( 400.f )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.Padding( 5.f )
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepFloatFilter::OnGetContextMenuForMatchingCriteria )
					[
						SAssignNew( MatchingCriteriaHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SAssignNew( FloatMatchingCriteriaWidget, SComboBox< TSharedPtr< FListEntry > > )
							.OptionsSource( &FloatMatchingOptions )
							.OnGenerateWidget( this, &SDataprepFloatFilter::OnGenerateWidgetForMatchingCriteria )
							.OnSelectionChanged( this, &SDataprepFloatFilter::OnSelectedCriteriaChanged )
							.OnComboBoxOpening( this, &SDataprepFloatFilter::OnCriteriaComboBoxOpenning )
							[
								SNew( STextBlock )
								.Text( this, &SDataprepFloatFilter::GetSelectedCriteriaText )
								.ToolTipText( this, &SDataprepFloatFilter::GetSelectedCriteriaTooltipText )
								.Justification( ETextJustify::Center )
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding( 5.f )
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepFloatFilter::OnGetContextMenuForEqualValue )
					[
						SAssignNew( EqualValueHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SSpinBox<float> )
							.Value( this, &SDataprepFloatFilter::GetEqualValue )
							.OnValueChanged( this, &SDataprepFloatFilter::OnEqualValueChanged )
							.OnValueCommitted( this, &SDataprepFloatFilter::OnEqualValueComitted )
							.Justification( ETextJustify::Center )
							.MinValue( TOptional< float >() )
							.MaxValue( TOptional< float >() )
							.ContextMenuExtender( this, &SDataprepFloatFilter::ExtendContextMenuForEqualValueBox )
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SDataprepContextMenuOverride )
				.OnContextMenuOpening( this, &SDataprepFloatFilter::OnGetContextMenuForTolerance )
				[
					SNew(SHorizontalBox)
					.Visibility( this, &SDataprepFloatFilter::GetToleranceRowVisibility )
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign( VAlign_Center )
					.Padding( 5.f )
					[
						SAssignNew( ToleranceHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( STextBlock )
							.Text( LOCTEXT("ToleranceText", "Tolerance") )
							.ToolTipText(this, &SDataprepFloatFilter::GetSelectedCriteriaTooltipText)
							.Justification( ETextJustify::Center )
						]
					]
					+ SHorizontalBox::Slot()
					.Padding( 5.f )
					[
						SNew( SSpinBox<float> )
						.Value( this, &SDataprepFloatFilter::GetTolerance )
						.OnValueChanged( this, &SDataprepFloatFilter::OnToleranceChanged )
						.OnValueCommitted( this, &SDataprepFloatFilter::OnToleranceComitted )
						.Justification( ETextJustify::Center )
						.MinValue( 0.f )
						.MaxValue( TOptional< float >() )
						.ContextMenuExtender( this, &SDataprepFloatFilter::ExtendContextMenuForToleranceBox )
					]
				]
			]
		]
	];

	if ( MatchingCriteriaParameterizationActionData && MatchingCriteriaParameterizationActionData->IsValid() )
	{
		if ( MatchingCriteriaParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, MatchingCriteriaParameterizationActionData->PropertyChain ) )
		{
			MatchingCriteriaHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, MatchingCriteriaParameterizationActionData->DataprepAsset, Filter, MatchingCriteriaParameterizationActionData->PropertyChain )
				];
		}
	}

	if ( EqualValueParameterizationActionData && EqualValueParameterizationActionData->IsValid() )
	{
		if ( EqualValueParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, EqualValueParameterizationActionData->PropertyChain ) )
		{
			EqualValueHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, EqualValueParameterizationActionData->DataprepAsset, Filter, EqualValueParameterizationActionData->PropertyChain )
				];
		}
	}

	if ( ToleranceParameterizationActionData && ToleranceParameterizationActionData->IsValid() )
	{
		if ( ToleranceParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, ToleranceParameterizationActionData->PropertyChain ) )
		{
			ToleranceHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, ToleranceParameterizationActionData->DataprepAsset, Filter, ToleranceParameterizationActionData->PropertyChain )
				];
		}
	}
}

TSharedRef<SWidget> SDataprepFloatFilter::OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const
{
	return SNew( STextBlock )
		.Text( ListEntry->Get<0>() )
		.ToolTipText( ListEntry->Get<1>() );
}

FText SDataprepFloatFilter::GetSelectedCriteriaText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepFloatMatchType >();
	return Enum->GetDisplayNameTextByValue( static_cast<uint8>( Filter->GetFloatMatchingCriteria() ) );
}

FText SDataprepFloatFilter::GetSelectedCriteriaTooltipText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepFloatMatchType >();
	return Enum->GetToolTipTextByIndex( Enum->GetIndexByValue( static_cast<uint8>( Filter->GetFloatMatchingCriteria() ) ) );
}

void SDataprepFloatFilter::OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType)
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepFloatMatchType >();
	EDataprepFloatMatchType FloatMatchType = static_cast< EDataprepFloatMatchType >( Enum->GetValueByIndex( ListEntry->Get<2>() ) );

	if ( FloatMatchType != Filter->GetFloatMatchingCriteria() )
	{
		FScopedTransaction Transaction( LOCTEXT("SelectionCriteriaChangedTransaction", "Changed the Float Selection Criteria") );
		Filter->SetFloatMatchingCriteria( FloatMatchType );

		SDataprepFloatFilterUtils::PostEditChainProperty( Filter, TEXT("FloatMatchingCriteria") );
	}
}

void SDataprepFloatFilter::OnCriteriaComboBoxOpenning()
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepFloatMatchType >();
	int32 EnumValueMapping = Enum->GetIndexByValue( static_cast<uint8>( Filter->GetFloatMatchingCriteria() ) );

	TSharedPtr<FListEntry> ItemToSelect;
	for ( const TSharedPtr<FListEntry>& Entry : FloatMatchingOptions )
	{
		if ( Entry->Get<2>() == EnumValueMapping )
		{
			ItemToSelect = Entry;
			break;
		}
	}

	check( FloatMatchingCriteriaWidget );
	FloatMatchingCriteriaWidget->SetSelectedItem( ItemToSelect );
}

TSharedPtr<SWidget> SDataprepFloatFilter::OnGetContextMenuForMatchingCriteria()
{
	return FDataprepEditorUtils::MakeContextMenu( MatchingCriteriaParameterizationActionData );
}

void SDataprepFloatFilter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}

float SDataprepFloatFilter::GetEqualValue() const
{
	check( Filter );
	return Filter->GetEqualValue();
}

void SDataprepFloatFilter::OnEqualValueChanged(float NewEqualValue)
{
	check( Filter );
	Filter->SetEqualValue( NewEqualValue );
}

void SDataprepFloatFilter::OnEqualValueComitted(float NewEqualValue, ETextCommit::Type CommitType)
{
	check( Filter );

	if ( OldEqualValue != NewEqualValue )
	{
		// Trick for the transaction 
		Filter->SetEqualValue( OldEqualValue );
		FScopedTransaction Transaction( LOCTEXT("EqualValueChangedTransaction","Change the Equal Value") );
		Filter->SetEqualValue( NewEqualValue );

		SDataprepFloatFilterUtils::PostEditChainProperty( Filter, TEXT("EqualValue") );
		OldEqualValue = NewEqualValue;
	}
}

TSharedPtr<SWidget> SDataprepFloatFilter::OnGetContextMenuForEqualValue()
{
	return FDataprepEditorUtils::MakeContextMenu( EqualValueParameterizationActionData );
}

void SDataprepFloatFilter::ExtendContextMenuForEqualValueBox(FMenuBuilder& MenuBuilder)
{
	if ( EqualValueParameterizationActionData && EqualValueParameterizationActionData->IsValid() && Filter )
	{
		FDataprepEditorUtils::PopulateMenuForParameterization( MenuBuilder, *EqualValueParameterizationActionData->DataprepAsset, *Filter, EqualValueParameterizationActionData->PropertyChain );
	}
}

EVisibility SDataprepFloatFilter::GetToleranceRowVisibility() const
{
	check( Filter );
	return Filter->GetFloatMatchingCriteria() == EDataprepFloatMatchType::IsNearlyEqual ? EVisibility::Visible : EVisibility::Collapsed;
}

float SDataprepFloatFilter::GetTolerance() const
{
	check( Filter );
	return Filter->GetTolerance();
}

void SDataprepFloatFilter::OnToleranceChanged(float NewTolerance)
{
	check( Filter );
	Filter->SetTolerance( NewTolerance );
}

void SDataprepFloatFilter::OnToleranceComitted(float NewTolerance, ETextCommit::Type CommitType)
{
	check( Filter );

	if ( OldTolerance != NewTolerance )
	{
		// Trick for the transaction 
		Filter->SetTolerance( OldTolerance );
		FScopedTransaction Transaction( LOCTEXT("ToleranceChangedTransaction", "Change the Tolerance") );
		Filter->SetTolerance( NewTolerance );
		SDataprepFloatFilterUtils::PostEditChainProperty( Filter, TEXT("Tolerance") );
		OldTolerance = NewTolerance;
	}
}

TSharedPtr<SWidget> SDataprepFloatFilter::OnGetContextMenuForTolerance()
{
	return FDataprepEditorUtils::MakeContextMenu( ToleranceParameterizationActionData );
}

void SDataprepFloatFilter::ExtendContextMenuForToleranceBox(FMenuBuilder& MenuBuilder)
{
	if ( ToleranceParameterizationActionData && ToleranceParameterizationActionData->IsValid() && Filter )
	{
		FDataprepEditorUtils::PopulateMenuForParameterization( MenuBuilder, *ToleranceParameterizationActionData->DataprepAsset, *Filter, ToleranceParameterizationActionData->PropertyChain );
	}
}

void SDataprepFloatFilter::OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects)
{
	if ( !Objects || Objects->Contains( Filter ) )
	{
		UpdateVisualDisplay();
	}
}

#undef LOCTEXT_NAMESPACE
