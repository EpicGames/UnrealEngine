// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FMenuBuilder;
class SComboButton;
class UDataprepFloatFilter;

template<class t>
class SComboBox;

struct FDataprepParametrizationActionData;

class SDataprepFloatFilter : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SDataprepFloatFilter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepFloatFilter& InFilter);

	virtual ~SDataprepFloatFilter();

private:

	// Refresh the ui
	void UpdateVisualDisplay();

	// The string matching option for the combo box (Displayed text, Tooltip, mapping for the UEnum)
	using FListEntry = TTuple<FText, FText, int32>;

	// Those function are for the float matching criteria display
	TSharedRef<SWidget> OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const;
	FText GetSelectedCriteriaText() const;
	FText GetSelectedCriteriaTooltipText() const;
	void OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType);
	void OnCriteriaComboBoxOpenning();
	TSharedPtr<SWidget> OnGetContextMenuForMatchingCriteria();

	// Those functions are for the float that will be compare against the fetched float
	float GetEqualValue() const;
	void OnEqualValueChanged(float NewEqualValue);
	void OnEqualValueComitted(float NewEqualValue, ETextCommit::Type CommitType);
	TSharedPtr<SWidget> OnGetContextMenuForEqualValue();
	void ExtendContextMenuForEqualValueBox(FMenuBuilder& MenuBuilder);

	// Those function are for the tolerance display
	EVisibility GetToleranceRowVisibility() const;
	float GetTolerance() const;
	void OnToleranceChanged(float NewTolerance);
	void OnToleranceComitted(float NewTolerance, ETextCommit::Type CommitType);
	TSharedPtr<SWidget> OnGetContextMenuForTolerance();
	void ExtendContextMenuForToleranceBox(FMenuBuilder& MenuBuilder);

	// Event for dataprep parameterization
	void OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects);

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepFloatFilter");
	}
	//~ End FGCObject interface

	float OldEqualValue;
	float OldTolerance;

	TObjectPtr<UDataprepFloatFilter> Filter;

	TArray<TSharedPtr<FListEntry>> FloatMatchingOptions;

	TSharedPtr<SComboBox<TSharedPtr<FListEntry>>> FloatMatchingCriteriaWidget;

	TSharedPtr<FDataprepParametrizationActionData> MatchingCriteriaParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> EqualValueParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> ToleranceParameterizationActionData;

	FDelegateHandle OnParameterizationStatusForObjectsChangedHandle;
};
