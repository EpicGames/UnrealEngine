// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"

// Dataprep includes
#include "DataprepEditorStyle.h"
#include "DataprepGraph/DataprepGraphActionNode.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Application/SlateApplication.h"
#include "SchemaActions/DataprepDragDropOp.h"
#include "Widgets/DataprepGraph/SDataprepGraphActionStepNode.h"
#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"

// Engine Includes
#include "DragAndDrop/AssetDragDropOp.h"
#include "EditorFontGlyphs.h"
#include "SGraphPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

float SDataprepGraphBaseActionNode::DefaultWidth = 350.f;
float SDataprepGraphBaseActionNode::DefaultHeight = 100.f;

class SDataprepGraphActionProxyNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphActionProxyNode) {}
		SLATE_ARGUMENT( TSharedPtr<SInlineEditableTextBlock>, InlineEditableText )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDataprepGraphBaseActionNode>& InParentNode)
	{
		ParentNodePtr = InParentNode;
		GraphNode = InParentNode->GetNodeObj();

		SetCursor(EMouseCursor::Default);
		UpdateGraphNode();

		InlineEditableText = InArgs._InlineEditableText;
	}

	// SWidget interface
	// End of SWidget interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override
	{
		this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SColorBlock)
				.Color( FLinearColor::Transparent )
				.Size( TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP( this, &SDataprepGraphActionProxyNode::GetSize ) ) )
			]
		];
	}

	virtual FSlateRect GetTitleRect() const override
	{
		const FVector2f NodePosition = GetPosition2f();
		const FVector2f NodeSize = InlineEditableText.IsValid() ? InlineEditableText->GetDesiredSize() : GetDesiredSize();

		return FSlateRect( NodePosition.X, NodePosition.Y + NodeSize.Y, NodePosition.X + NodeSize.X, NodePosition.Y );
	}

	/* Helper function to check if node can be renamed */
	virtual bool IsNameReadOnly () const override
	{
		return ParentNodePtr.IsValid() ? ParentNodePtr.Pin()->IsNameReadOnly() : true;
	}

	const FSlateBrush* GetShadowBrush(bool bSelected) const
	{
		return  FAppStyle::GetNoBrush();
	}

	virtual bool ShouldAllowCulling() const override { return false; }
	// End of SGraphNode interface

	FVector2D GetSize()
	{
		static FVector2D DefaultSize(SDataprepGraphBaseActionNode::DefaultWidth, SDataprepGraphBaseActionNode::DefaultHeight);

		FVector2D Size(DefaultSize);

		if(SDataprepGraphBaseActionNode* ParentNode = ParentNodePtr.Pin().Get())
		{
			Size = ParentNode->GetCachedGeometry().GetLocalSize();

			if(Size == FVector2D::ZeroVector)
			{
				Size = ParentNode->GetDesiredSize();
				if(Size == FVector2D::ZeroVector)
				{
					Size = DefaultSize;
				}
			}
		}

		return Size;
	}

	void SetPosition(const FVector2D& Position)
	{
		GraphNode->NodePosX = Position.X;
		GraphNode->NodePosY = Position.Y;
	}

private:
	/** Pointer to the SDataprepGraphTrackNode displayed in the graph editor  */
	TWeakPtr<SDataprepGraphBaseActionNode> ParentNodePtr;
};

/**
 * The SDataprepEmptyActionStepNode is a helper class that handles drag and drop event at
 * the bottom of the SDataprepGraphActionNode widget
 */
class SDataprepEmptyActionStepNode : public SVerticalBox
{
	enum EColorType : uint8
	{
		TextColor = 0,
		OuterColor,
		InnerColor,
		ColorMaxType
	};

public:
	SLATE_BEGIN_ARGS(SDataprepEmptyActionStepNode)
		: _Text(LOCTEXT("DataprepEmptyActionStepLabel", "+ Add Step"))
		{}

		/** The text displayed in the text block */
		SLATE_ATTRIBUTE( FText, Text )
		SLATE_ATTRIBUTE( int32, StepIndex )
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedPtr<SDataprepGraphActionNode>& InParent)
	{
		ParentPtr = InParent;
		StepIndex = InArgs._StepIndex.Get(InParent->GetDataprepAction()->GetStepsCount());

		const float InterStepSpacing = 5.f;
		const bool bBottomSlot = StepIndex == InParent->GetDataprepAction()->GetStepsCount();

		SVerticalBox::Construct(SVerticalBox::FArguments());

		TAttribute<FSlateColor> TextColorAndOpacity = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepEmptyActionStepNode::GetColor, EColorType::TextColor));
		TAttribute<FSlateColor> OuterColorAndOpacity = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepEmptyActionStepNode::GetColor, EColorType::OuterColor));
		TAttribute<FSlateColor> InnerColorAndOpacity = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepEmptyActionStepNode::GetColor, EColorType::InnerColor));

		TSharedPtr<SWidget> Separator = SNullWidget::NullWidget;

		if(StepIndex != InParent->GetDataprepAction()->GetStepsCount())
		{
			Separator = SNew( SSeparator )
				.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
				.Thickness(2.f)
				.Orientation(EOrientation::Orient_Horizontal)
				.ColorAndOpacity(this, &SDataprepEmptyActionStepNode::GetDragAndDropColor);
		}

		AddSlot()
		.AutoHeight()
		.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.Padding" ))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(20.f, 0.f)
			.AutoHeight()
			[
				Separator.ToSharedRef()
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.Padding(10.f)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SBox)
					.MinDesiredWidth(250.f)
				]

				+ SOverlay::Slot()
				.Padding(0.f)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SImage)
					.ColorAndOpacity(MoveTemp(OuterColorAndOpacity))
					.Image(FAppStyle::GetBrush( "Graph.StateNode.Body" ))
				]

				+ SOverlay::Slot()
				.Padding(1.f)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SImage)
					.ColorAndOpacity(MoveTemp(InnerColorAndOpacity))
					.Image(FAppStyle::GetBrush( "Graph.StateNode.Body" ))
				]

				+ SOverlay::Slot()
				.Padding(10.f, 10.f, 10.f, bBottomSlot ? 10.f : 5.f)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(TextBlock, STextBlock)
						.TextStyle( &FDataprepEditorStyle::GetWidgetStyle<FTextBlockStyle>( "DataprepActionBlock.TitleTextBlockStyle" ) )
						.ColorAndOpacity(MoveTemp(TextColorAndOpacity))
						.Justification(ETextJustify::Center)
					]
				]
			]
		];

		TextBlock->SetText(InArgs._Text);
	}

	// SWidget Interface
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FDataprepDragDropOp> DragActionStepNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
		if(DragActionStepNodeOp.IsValid() && ParentPtr.IsValid())
		{
			ParentTrackNodePtr.Pin()->OnDragLeave(DragDropEvent);

			if (UDataprepActionAsset* DataprepAction = ParentPtr.Pin()->GetDataprepAction())
			{
				if (DataprepAction->GetAppearance()->GroupId == INDEX_NONE)
				{
					DragActionStepNodeOp->SetHoveredNode(ParentPtr.Pin()->GetNodeObj());
					ParentPtr.Pin()->SetHoveredIndex( StepIndex );
				}
				else
				{
					DragActionStepNodeOp->SetHoveredNode( nullptr );
					ParentPtr.Pin()->SetHoveredIndex( INDEX_NONE );
				}
			}
		}

		SVerticalBox::OnDragEnter(MyGeometry, DragDropEvent);
	}

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FDataprepDragDropOp> DragActionStepNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
		if(DragActionStepNodeOp.IsValid() && ParentPtr.IsValid())
		{
			if (UDataprepActionAsset* DataprepAction = ParentPtr.Pin()->GetDataprepAction())
			{
				if (DataprepAction->GetAppearance()->GroupId == INDEX_NONE)
				{
					DragActionStepNodeOp->SetHoveredNode(ParentPtr.Pin()->GetNodeObj());
					ParentPtr.Pin()->SetHoveredIndex( StepIndex );
				}
				else
				{
					DragActionStepNodeOp->SetHoveredNode( nullptr );
					ParentPtr.Pin()->SetHoveredIndex( INDEX_NONE );
				}
			}

			return FReply::Handled();
		}

		return SVerticalBox::OnDragOver(MyGeometry, DragDropEvent);
	}

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FDataprepDragDropOp> DragActionStepNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
		if(DragActionStepNodeOp.IsValid())
		{
			DragActionStepNodeOp->SetHoveredNode(nullptr);
		}

		ParentPtr.Pin()->SetHoveredIndex( INDEX_NONE );

		SVerticalBox::OnDragLeave(DragDropEvent);
	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		// Reset dragged index as drag is completed
		ParentPtr.Pin()->SetDraggedIndex( INDEX_NONE );

		// Process OnDrop if done by FDataprepDragDropOp
		TSharedPtr<FDataprepDragDropOp> DragActionStepNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
		if (DragActionStepNodeOp.IsValid() && ParentPtr.IsValid())
		{
			if(StepIndex == ParentPtr.Pin()->GetDataprepAction()->GetStepsCount())
			{
				const FVector2f NodeAddPosition = ParentPtr.Pin()->NodeCoordToGraphCoord( MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() ) );
				return DragActionStepNodeOp->DroppedOnNode(DragDropEvent.GetScreenSpacePosition(), NodeAddPosition);
			}
			else
			{
				return FReply::Handled().EndDragDrop();
			}
		}

		return SVerticalBox::OnDrop(MyGeometry, DragDropEvent);
	}
	// End of SWidget Interface

	void SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode)
	{
		ParentTrackNodePtr = InParentTrackNode;
	}

	void SetText(FText Text)
	{
		if(TextBlock.IsValid())
		{
			TextBlock->SetText(Text);
		}
	}

private:
	FSlateColor GetColor(EColorType Type) const
	{
		static const FLinearColor HoveredColors[EColorType::ColorMaxType] = {
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Text.Hovered"),
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Outer.Hovered"),
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Background.Hovered"),
		};
		static const FLinearColor NormalColors[EColorType::ColorMaxType] = {
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Text.Normal"),
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Outer.Normal"),
			FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Background.Normal"),
		};

		return StepIndex == ParentPtr.Pin()->GetHoveredIndex() ? HoveredColors[Type] : NormalColors[Type];
	}

	FSlateColor GetDragAndDropColor() const
	{
		return ParentPtr.Pin()->GetInsertColor(StepIndex);
	}

private:
	TWeakPtr<SDataprepGraphActionNode> ParentPtr;
	TWeakPtr<SDataprepGraphTrackNode> ParentTrackNodePtr;
	TSharedPtr<STextBlock> TextBlock;
	int32 StepIndex;
};

void SDataprepGraphBaseActionNode::Initialize(TWeakPtr<FDataprepEditor> InDataprepEditor, int32 InExecutionOrder, UEdGraphNode* InNode)
{
	UserSize = FVector2D( 
		FMath::Max(static_cast<float>(InNode->NodeWidth), SDataprepGraphActionNode::DefaultWidth),
		FMath::Max(static_cast<float>(InNode->NodeHeight), SDataprepGraphActionNode::DefaultHeight));

	ExecutionOrder = InExecutionOrder;
	GraphNode = InNode;
	DataprepEditor = InDataprepEditor;

	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();

	ProxyNodePtr = SNew(SDataprepGraphActionProxyNode, SharedThis(this))
	.InlineEditableText(InlineEditableText);
}

FReply SDataprepGraphBaseActionNode::OnDragOver(const FGeometry & MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetOp.IsValid())
	{
		return FReply::Handled();
	}

	return SGraphNodeResizable::OnDragOver(MyGeometry, DragDropEvent);
}

FVector2D SDataprepGraphBaseActionNode::ComputeDesiredSize(float f) const 
{
	const FVector2D Size = SGraphNodeResizable::ComputeDesiredSize(f);
	return FVector2D(FMath::Max(Size.X, UserSize.X), Size.Y);
}

FVector2f SDataprepGraphBaseActionNode::GetNodeMinimumSize2f() const
{
	return FVector2f( SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultHeight );
}

FVector2f SDataprepGraphBaseActionNode::GetNodeMaximumSize2f() const
{
	return FVector2f( UserSize.X + 100.0f, UserSize.Y + 0.0f );
}

void SDataprepGraphActionNode::Construct(const FArguments& InArgs, UDataprepGraphActionNode* InActionNode)
{
	DataprepActionPtr = InActionNode->GetDataprepActionAsset();
	check(DataprepActionPtr.IsValid());

	DraggedIndex = INDEX_NONE;
	InsertIndex = INDEX_NONE;

	SDataprepGraphBaseActionNode::Initialize(InArgs._DataprepEditor, InActionNode->GetExecutionOrder(), InActionNode);

	DataprepActionPtr->GetOnStepsOrderChanged().AddSP(this, &SDataprepGraphActionNode::OnStepsChanged);
}

void SDataprepGraphActionNode::SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode)
{
	SDataprepGraphBaseActionNode::SetParentTrackNode(InParentTrackNode);

	if(ActionStepListWidgetPtr.IsValid())
	{
		FChildren* StepListChildren = ActionStepListWidgetPtr->GetChildren();

		// Update parent track on step widgets
		for(TSharedPtr<SDataprepGraphActionStepNode>& ActionStepGraphNode : ActionStepGraphNodes)
		{
			if(ActionStepGraphNode->GetStepTitleWidget().IsValid())
			{
				ActionStepGraphNode->SetParentTrackNode(InParentTrackNode);
			}
			else
			{
				TSharedRef<SDataprepEmptyActionStepNode> EmptyWidgetPtr = StaticCastSharedRef<SDataprepEmptyActionStepNode>(StepListChildren->GetChildAt(ActionStepGraphNode->GetStepIndex()));
				EmptyWidgetPtr->SetParentTrackNode(InParentTrackNode);
			}
		}

		// Update parent on empty bottom widget
		TSharedRef<SDataprepEmptyActionStepNode> EmptyWidgetPtr = StaticCastSharedRef<SDataprepEmptyActionStepNode>(StepListChildren->GetChildAt(StepListChildren->Num() - 1));
		EmptyWidgetPtr->SetParentTrackNode(InParentTrackNode);
	}
}

void SDataprepGraphBaseActionNode::UpdateProxyNode(const FVector2D& Position)
{
	if(ProxyNodePtr.IsValid())
	{
		ProxyNodePtr->SetPosition(Position);
	}
}

void SDataprepGraphActionNode::UpdateGraphNode()
{
	static bool bUseNew = true;
	if(!bUseNew)
	{
		SGraphNode::UpdateGraphNode();
		return;
	}

	// Reset SGraphNode members.
	InputPins.Empty();
	OutputPins.Empty();
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	if(!DataprepActionPtr.IsValid())
	{
		this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.ColorAndOpacity( FSlateColor( FLinearColor::Red ) )
			.Text( FText::FromString( TEXT("This node doesn't have a dataprep action!") ) )
		];

		return;
	}

	TAttribute<FText> NodeTitle = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(GraphNode, &UEdGraphNode::GetNodeTitle, ENodeTitleType::EditableTitle));
	SAssignNew(InlineEditableText, SInlineEditableTextBlock)
		.Style(FDataprepEditorStyle::Get(), "DataprepAction.TitleInlineEditableText")
		.Text(NodeTitle)
		.OnVerifyTextChanged(this, &SDataprepGraphActionNode::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &SDataprepGraphActionNode::OnNameTextCommited)
		.IsReadOnly(this, &SDataprepGraphActionNode::IsNameReadOnly)
		.IsSelected(this, &SDataprepGraphActionNode::IsSelectedExclusively);

	PopulateActionStepListWidget();

	TAttribute<FMargin> OuterPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataprepGraphBaseActionNode::GetOuterPadding));

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.WidthOverride(DefaultWidth)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.Padding(OuterPadding)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(FDataprepEditorStyle::GetColor( "DataprepAction.OutlineColor" ))
				]

				+ SOverlay::Slot()
				.Padding(FDataprepEditorStyle::GetMargin( "DataprepAction.Body.Padding" ))
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(FDataprepEditorStyle::GetColor( "DataprepAction.BackgroundColor" ))
				]

				+ SOverlay::Slot()
				.Padding( FDataprepEditorStyle::GetMargin( "DataprepAction.Steps.Padding" ) )
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.f, 0, 5.f, 0)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							InlineEditableText.ToSharedRef()
						]

						+ SHorizontalBox::Slot()
						[
							SNew(SBox)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.f, 5.f, 5.f, 5.f)
						[
							SAssignNew(ExpandActionButton, SButton)
							.ButtonStyle(FAppStyle::Get(), "FlatButton.Primary")
							.ButtonColorAndOpacity(FLinearColor::Transparent)
							.ForegroundColor(FLinearColor::White)
							.ContentPadding(FMargin(6, 2))
							.Content()
							[

								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
								.Text_Lambda([this](){ return DataprepActionPtr->GetAppearance()->bIsExpanded ? FEditorFontGlyphs::Caret_Down : FEditorFontGlyphs::Caret_Up; })
							]
							.OnClicked_Lambda([this]()
							{
								DataprepActionPtr->GetAppearance()->bIsExpanded = !DataprepActionPtr->GetAppearance()->bIsExpanded;
								return FReply::Handled();
							})
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(5.0f, 0.0f, 7.0f, 2.0f))
					[
						SNew( SSeparator )
						.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
						.Thickness(1.f)
						.Orientation(EOrientation::Orient_Horizontal)
						.ColorAndOpacity(FDataprepEditorStyle::GetColor("Dataprep.TextSeparatorActionNode.Color"))
						.Visibility_Lambda([this]() { return DataprepActionPtr->GetAppearance()->bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed; })
					]

					//The content of the action
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						.Visibility_Lambda([this]()
						{
							return DataprepActionPtr->GetAppearance()->bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
						})
						+ SHorizontalBox::Slot()
						[
							ActionStepListWidgetPtr.ToSharedRef()
						]
					]
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(OuterPadding)
				[
					SNew(SImage)
					.ColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f))
					.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.Node.Body"))
					.Visibility_Lambda([&]()
					{
						if (const UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(GraphNode))
						{
							if (const UDataprepActionAsset* ActionAsset = ActionNode->GetDataprepActionAsset())
							{
								if (!ActionAsset->bIsEnabled)
								{
									return EVisibility::Visible;
								}
							}
						}
						return EVisibility::Collapsed;
					})
				]
			]
		]
	];
}

TSharedRef<SWidget> SDataprepGraphBaseActionNode::CreateBackground(const TAttribute<FSlateColor>& ColorAndOpacity)
{
	return SNew(SOverlay)

		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(ColorAndOpacity)
			.Image(FDataprepEditorStyle::GetBrush( "DataprepEditor.Node.Body" ))
		];
}

TSharedRef<SWidget> SDataprepGraphActionNode::CreateNodeContentArea()
{
	if(DataprepActionPtr.IsValid())
	{
		PopulateActionStepListWidget();

		return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			ActionStepListWidgetPtr.ToSharedRef()
		];
	}

	return 	SNew(STextBlock)
	.ColorAndOpacity( FSlateColor( FLinearColor::Red ) )
	.Text( FText::FromString( TEXT("This node doesn't have a dataprep action!") ) );
}

const FSlateBrush* SDataprepGraphActionNode::GetShadowBrush(bool bSelected) const
{
	return FAppStyle::GetNoBrush();
}

FReply SDataprepGraphActionNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SGraphNodeResizable::OnMouseButtonDown( MyGeometry, MouseEvent );

	if ( !Reply.IsEventHandled() )
	{
		if( MouseEvent.GetEffectingButton() ==  EKeys::LeftMouseButton )
		{
			GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );

			UDataprepActionAppearance* Appearance = GetDataprepAction()->GetAppearance();

			if (Appearance->GroupId != INDEX_NONE)
			{
				// Disallow dragging of grouped actions/steps
				return FReply::Handled();
			}

			return FReply::Handled().DetectDrag( AsShared(), EKeys::LeftMouseButton );
		}

		// Take ownership of the mouse if right mouse button clicked to display contextual menu
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			if ( !GetOwnerPanel()->SelectionManager.SelectedNodes.Contains( GraphNode ) )
			{
				GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );
			}
			return FReply::Handled();
		}
	}

	return Reply;
}

FReply SDataprepGraphActionNode::OnMouseButtonUp(const FGeometry & MyGeometry, const FPointerEvent & MouseEvent)
{
	FReply Reply = SGraphNodeResizable::OnMouseButtonUp( MyGeometry, MouseEvent );

	if ( !Reply.IsEventHandled() )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			ensure(OwnerGraphPanelPtr.IsValid());

			const FVector2D Position = MouseEvent.GetScreenSpacePosition();
			OwnerGraphPanelPtr.Pin()->SummonContextMenu(Position, Position, GraphNode, nullptr, TArray<UEdGraphPin*>());

			// Release mouse capture
			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return Reply;
}

FCursorReply SDataprepGraphActionNode::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	FCursorReply CursorReply = SGraphNodeResizable::OnCursorQuery( MyGeometry, CursorEvent );

	if ( !CursorReply.IsEventHandled() )
	{
		if (ExpandActionButton.IsValid() && ExpandActionButton->IsHovered())
		{
			CursorReply = FCursorReply::Cursor( EMouseCursor::Default );
		}
		else
		{
			TOptional<EMouseCursor::Type> TheCursor = GetCursor();
			CursorReply = ( TheCursor.IsSet() )
				? FCursorReply::Cursor( TheCursor.GetValue() )
				: FCursorReply::Unhandled();
		}
	}

	return CursorReply;
}

void SDataprepGraphActionGroupNode::SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel)
{
	if(!OwnerGraphPanelPtr.IsValid())
	{
		SGraphNode::SetOwner(OwnerPanel);
		OwnerPanel->AttachGraphEvents(SharedThis(this));
		OwnerPanel->AddGraphNode(SharedThis(ProxyNodePtr.Get()));

		for(TSharedPtr<SDataprepGraphActionNode>& ActionGraphNode : ActionGraphNodes)
		{
			if (ActionGraphNode.IsValid())
			{
				ActionGraphNode->SetOwner(OwnerPanel);
				OwnerPanel->AttachGraphEvents(ActionGraphNode);
			}
		}
	}
	else
	{
		ensure(OwnerPanel == OwnerGraphPanelPtr);
	}
}

void SDataprepGraphActionNode::SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel)
{
	if(!OwnerGraphPanelPtr.IsValid())
	{
		SGraphNode::SetOwner(OwnerPanel);
		OwnerPanel->AttachGraphEvents(SharedThis(this));

		OwnerPanel->AddGraphNode(SharedThis(ProxyNodePtr.Get()));

		for(TSharedPtr<SDataprepGraphActionStepNode>& ActionStepGraphNode : ActionStepGraphNodes)
		{
			if (ActionStepGraphNode.IsValid())
			{
				ActionStepGraphNode->SetOwner(OwnerPanel);
				OwnerPanel->AttachGraphEvents(ActionStepGraphNode);
			}
		}
	}
	else
	{
		ensure(OwnerPanel == OwnerGraphPanelPtr);
	}
}

FReply SDataprepGraphActionNode::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(GraphNode))
	{
		if (UDataprepActionAsset* ActionAsset = ActionNode->GetDataprepActionAsset())
		{
			return FReply::Handled().BeginDragDrop(FDragDropActionNode::New(SharedThis(ParentTrackNodePtr.Pin().Get()), SharedThis(this)));
		}
	}

	return FReply::Unhandled();
}

FReply SDataprepGraphActionNode::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SetCursor(EMouseCursor::Default);

	DraggedIndex = INDEX_NONE;

	TSharedPtr<FDragDropActionNode> DragOperation = DragDropEvent.GetOperationAs<FDragDropActionNode>();
	if (DragOperation.IsValid())
	{
		return FReply::Handled().EndDragDrop();
	}

	return SGraphNode::OnDrop(MyGeometry, DragDropEvent);
}

void SDataprepGraphActionNode::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Track node is not notified of drag left, do it
	TSharedPtr<FDragDropActionNode> DragOperation = DragDropEvent.GetOperationAs<FDragDropActionNode>();
	if (ParentTrackNodePtr.IsValid() && DragOperation.IsValid())
	{
		ParentTrackNodePtr.Pin()->OnDragLeave(DragDropEvent);
	}

	DraggedIndex = INDEX_NONE;

	SGraphNode::OnDragEnter(MyGeometry, DragDropEvent);
}

FSlateColor SDataprepGraphActionNode::GetInsertColor(int32 Index)
{
	static const FSlateColor BackgroundColor = FDataprepEditorStyle::GetColor("DataprepActionStep.BackgroundColor");
	static const FSlateColor DragAndDrop = FDataprepEditorStyle::GetColor("DataprepActionStep.Separator.Color");


	return Index == InsertIndex ? DragAndDrop : FLinearColor::Transparent;
}

void SDataprepGraphActionNode::SetDraggedIndex(int32 Index)
{
	DraggedIndex = Index;
	InsertIndex = INDEX_NONE;
}

void SDataprepGraphActionNode::SetHoveredIndex(int32 Index)
{
	if(DraggedIndex == INDEX_NONE || Index == DataprepActionPtr->GetStepsCount())
	{
		InsertIndex = Index;
	}
	else
	{
		InsertIndex = Index > DraggedIndex ? Index + 1 : (Index < DraggedIndex ? Index : INDEX_NONE);
	}
}

FMargin SDataprepGraphBaseActionNode::GetOuterPadding() const
{
	static const FMargin Selected = FDataprepEditorStyle::GetMargin( "DataprepAction.Outter.Selected.Padding" );
	static const FMargin Regular = FDataprepEditorStyle::GetMargin( "DataprepAction.Outter.Regular.Padding" );

	TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
	const bool bIsSelected = OwnerPanel.IsValid() ? GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode) : false;

	return bIsSelected ? Selected : Regular;
}

void SDataprepGraphActionNode::PopulateActionStepListWidget()
{
	if(!ActionStepListWidgetPtr.IsValid())
	{
		ActionStepListWidgetPtr = SNew(SVerticalBox);
	}
	else
	{
		ActionStepListWidgetPtr->ClearChildren();
	}

	const float InterStepSpacing = 2.f;
	UDataprepActionAsset* DataprepAction = DataprepActionPtr.Get();

	UEdGraph* EdGraph = GraphNode->GetGraph();
	const int32 StepsCount = DataprepAction->GetStepsCount();
	const UClass* GraphActionStepNodeClass = UDataprepGraphActionStepNode::StaticClass();
	
	EdGraphStepNodes.Reset(StepsCount);

	TSharedPtr<SDataprepGraphTrackNode> TrackNodePtr = ParentTrackNodePtr.Pin();

	ActionStepGraphNodes.SetNum(StepsCount);

	TSharedPtr<SGraphPanel> GraphPanelPtr = GetOwnerPanel();

	for ( int32 Index = 0; Index < StepsCount; ++Index )
	{
		EdGraphStepNodes.Emplace(NewObject<UDataprepGraphActionStepNode>( EdGraph, GraphActionStepNodeClass, NAME_None, RF_Transactional ));
		UDataprepGraphActionStepNode* ActionStepNode = EdGraphStepNodes.Last().Get();

		ActionStepNode->CreateNewGuid();
		ActionStepNode->PostPlacedNewNode();

		ActionStepNode->NodePosX = GraphNode->NodePosX;
		ActionStepNode->NodePosY = GraphNode->NodePosY;

		ActionStepNode->Initialize(DataprepAction, Index);

		TSharedPtr<SDataprepGraphActionStepNode> ActionStepGraphNode = SNew(SDataprepGraphActionStepNode, ActionStepNode, SharedThis(this))
			.DataprepEditor(DataprepEditor);

		if(TrackNodePtr.IsValid())
		{
			ActionStepGraphNode->SetParentTrackNode(TrackNodePtr);
		}

		if(ActionStepGraphNode->GetStepTitleWidget().IsValid())
		{
			ActionStepListWidgetPtr->AddSlot()
			.AutoHeight()
			[
				ActionStepGraphNode.ToSharedRef()
			];
		}
		else
		{
			TSharedPtr<SDataprepEmptyActionStepNode> EmptySlot;

			ActionStepListWidgetPtr->AddSlot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 10.f)
			[
				SAssignNew(EmptySlot, SDataprepEmptyActionStepNode, SharedThis(this))
				.Text(LOCTEXT("DataprepUnknownActionStepLabel", "Unknown Step Class"))
				.StepIndex(Index)
			];

			if(TrackNodePtr.IsValid())
			{
				EmptySlot->SetParentTrackNode(TrackNodePtr);
			}
		}

		ActionStepGraphNodes[Index] = ActionStepGraphNode;
	}

	if(GraphPanelPtr.IsValid())
	{
		for ( TSharedPtr<SDataprepGraphActionStepNode>& ActionStepGraphNode : ActionStepGraphNodes )
		{
			ActionStepGraphNode->SetOwner(GraphPanelPtr.ToSharedRef());
		}
	}

	TSharedPtr<SDataprepEmptyActionStepNode> BottomSlot;
	ActionStepListWidgetPtr->AddSlot()
	.AutoHeight()
	.Padding(0.f, 0.f, 0.f, 10.f)
	[
		SAssignNew(BottomSlot, SDataprepEmptyActionStepNode, SharedThis(this))
		.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP( this, &SDataprepGraphActionNode::GetBottomWidgetText ) ))
	];

	if(TrackNodePtr.IsValid())
	{
		BottomSlot->SetParentTrackNode(TrackNodePtr);
	}
}

void SDataprepGraphActionNode::OnStepsChanged()
{
	if(DataprepActionPtr.IsValid())
	{
		PopulateActionStepListWidget();
		ParentTrackNodePtr.Pin()->RefreshLayout();
	}
}

FText SDataprepGraphActionNode::GetBottomWidgetText() const
{
	if(InsertIndex == DataprepActionPtr->GetStepsCount())
	{
		FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
		if(ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown() || DraggedIndex == INDEX_NONE)
		{
			return LOCTEXT("DataprepEmptyActionStepCopyLabel", "Add Step");
		}
		else
		{
			return LOCTEXT("DataprepEmptyActionStepMoveLabel", "Move Step");
		}
	}

	return LOCTEXT("DataprepEmptyActionStepNoLabel", "-----");
}

void SDataprepGraphActionNode::UpdateExecutionOrder()
{
	ensure(Cast<UDataprepGraphActionNode>(GraphNode));
	ExecutionOrder = Cast<UDataprepGraphActionNode>(GraphNode)->GetExecutionOrder();
}

// SDataprepGraphActionGroupNode implementation

void SDataprepGraphActionGroupNode::Construct(const FArguments& InArgs, UDataprepGraphActionGroupNode* InActionNode)
{
	SDataprepGraphBaseActionNode::Initialize(InArgs._DataprepEditor, InActionNode->GetExecutionOrder(), InActionNode);
}

void SDataprepGraphActionGroupNode::UpdateGraphNode()
{
	static bool bUseNew = true;
	if(!bUseNew)
	{
		SGraphNode::UpdateGraphNode();
		return;
	}

	// Reset SGraphNode members.
	InputPins.Empty();
	OutputPins.Empty();
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	TAttribute<FText> NodeTitle = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(GraphNode, &UEdGraphNode::GetNodeTitle, ENodeTitleType::EditableTitle));
	SAssignNew(InlineEditableText, SInlineEditableTextBlock)
		.Style(FDataprepEditorStyle::Get(), "DataprepAction.TitleInlineEditableText")
		.Text(NodeTitle)
		.OnVerifyTextChanged(this, &SDataprepGraphActionGroupNode::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &SDataprepGraphActionGroupNode::OnNameTextCommited)
		.IsReadOnly(this, &SDataprepGraphActionGroupNode::IsNameReadOnly)
		.IsSelected(this, &SDataprepGraphActionGroupNode::IsSelectedExclusively);

	PopulateActionsListWidget();

	TAttribute<FMargin> OuterPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataprepGraphBaseActionNode::GetOuterPadding));

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.WidthOverride(SDataprepGraphBaseActionNode::DefaultWidth)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.Padding(OuterPadding)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground( FDataprepEditorStyle::GetColor( "DataprepAction.Group.OutlineColor" ) )
				]

				+ SOverlay::Slot()
				.Padding(FDataprepEditorStyle::GetMargin( "DataprepAction.Body.Padding" ))
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(FDataprepEditorStyle::GetColor( "DataprepAction.BackgroundColor" ))
				]

				+ SOverlay::Slot()
				.Padding( FDataprepEditorStyle::GetMargin( "DataprepAction.Steps.Padding" ) )
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.f, 5.f, 5.f, 2.f)
						[
							InlineEditableText.ToSharedRef()
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f, 2.f)
					[
						SNew( SSeparator )
						.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
						.Thickness(1.f)
						.Orientation(EOrientation::Orient_Horizontal)
						.ColorAndOpacity(FDataprepEditorStyle::GetColor("Dataprep.TextSeparatorActionNode.Color"))
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							ActionsListWidgetPtr.ToSharedRef()
						]
					]
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(OuterPadding)
				[
					SNew(SImage)
					.ColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f))
					.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.Node.Body"))
					.Visibility_Lambda([&]()
					{
						if (const UDataprepGraphActionGroupNode* ActionNode = Cast<UDataprepGraphActionGroupNode>(GraphNode))
						{
							if (!ActionNode->IsGroupEnabled())
							{
								return EVisibility::Visible;
							}
						}
						return EVisibility::Collapsed;
					})
				]
			]
		]
	];
}

void SDataprepGraphActionGroupNode::PopulateActionsListWidget()
{
	if(!ActionsListWidgetPtr.IsValid())
	{
		ActionsListWidgetPtr = SNew(SVerticalBox);
	}
	else
	{
		ActionsListWidgetPtr->ClearChildren();
	}

	const float InterActionSpacing = 2.f;

	UDataprepGraphActionGroupNode* GroupNode = Cast<UDataprepGraphActionGroupNode>(GraphNode);

	const int32 ActionsCount = GroupNode->GetActionsCount();
	const UClass* GraphActionNodeClass = UDataprepGraphActionNode::StaticClass();
	
	EdGraphActionNodes.Reset(ActionsCount);

	TSharedPtr<SDataprepGraphTrackNode> TrackNodePtr = ParentTrackNodePtr.Pin();

	ActionGraphNodes.SetNum(ActionsCount);

	TSharedPtr<SGraphPanel> GraphPanelPtr = GetOwnerPanel();

	for ( int32 Index = 0; Index < ActionsCount; ++Index )
	{
		EdGraphActionNodes.Emplace(NewObject<UDataprepGraphActionNode>( GraphNode->GetGraph(), GraphActionNodeClass, NAME_None, RF_Transactional ));
		UDataprepGraphActionNode* ActionNode = EdGraphActionNodes.Last().Get();

		ActionNode->CreateNewGuid();
		ActionNode->PostPlacedNewNode();

		ActionNode->NodePosX = GraphNode->NodePosX;
		ActionNode->NodePosY = GraphNode->NodePosY;

		UDataprepActionAsset* ActionAsset = GroupNode->GetAction(Index);
		UDataprepAsset* DataprepAsset = GroupNode->GetDataprepAsset();

		ActionNode->Initialize(DataprepAsset, ActionAsset, DataprepAsset->GetActionIndex(ActionAsset));

		TSharedPtr<SDataprepGraphActionNode> ActionGraphNode = SNew(SDataprepGraphActionNode, ActionNode/*, SharedThis(this)*/)
			.DataprepEditor(DataprepEditor);

		if(TrackNodePtr.IsValid())
		{
			ActionGraphNode->SetParentTrackNode(TrackNodePtr);
		}

		ActionsListWidgetPtr->AddSlot()
		.AutoHeight()
		[
			ActionGraphNode.ToSharedRef()
		];

		ActionGraphNodes[Index] = ActionGraphNode;
	}

	if(GraphPanelPtr.IsValid())
	{
		for ( TSharedPtr<SDataprepGraphActionNode>& ActionGraphNode : ActionGraphNodes )
		{
			ActionGraphNode->SetOwner(GraphPanelPtr.ToSharedRef());
		}
	}
}

void SDataprepGraphActionGroupNode::SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode)
{
	SDataprepGraphBaseActionNode::SetParentTrackNode(InParentTrackNode);

	if(ActionsListWidgetPtr.IsValid())
	{
		FChildren* ListChildren = ActionsListWidgetPtr->GetChildren();

		// Update parent track on step widgets
		for(TSharedPtr<SDataprepGraphActionNode>& ActionGraphNode : ActionGraphNodes)
		{
			ActionGraphNode->SetParentTrackNode(InParentTrackNode);
		}
	}
}

FReply SDataprepGraphActionGroupNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SGraphNodeResizable::OnMouseButtonDown( MyGeometry, MouseEvent );

	if ( !Reply.IsEventHandled() )
	{
		if( MouseEvent.GetEffectingButton() ==  EKeys::LeftMouseButton )
		{
			GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );
			return FReply::Handled().DetectDrag( AsShared(), EKeys::LeftMouseButton );
		}

		// Take ownership of the mouse if right mouse button clicked to display contextual menu
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			if ( !GetOwnerPanel()->SelectionManager.SelectedNodes.Contains( GraphNode ) )
			{
				GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );
			}
			return FReply::Handled();
		}
	}

	return Reply;
}

FReply SDataprepGraphActionGroupNode::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SGraphNodeResizable::OnMouseButtonUp( MyGeometry, MouseEvent );

	if ( !Reply.IsEventHandled() )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			ensure(OwnerGraphPanelPtr.IsValid());

			const FVector2D Position = MouseEvent.GetScreenSpacePosition();
			OwnerGraphPanelPtr.Pin()->SummonContextMenu(Position, Position, GraphNode, nullptr, TArray<UEdGraphPin*>());

			// Release mouse capture
			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return Reply;
}

FReply SDataprepGraphActionGroupNode::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (UDataprepGraphActionGroupNode* Node = Cast<UDataprepGraphActionGroupNode>(GraphNode))
	{
		return FReply::Handled().BeginDragDrop(FDragDropActionNode::New(SharedThis(ParentTrackNodePtr.Pin().Get()), SharedThis(this)));
	}

	return FReply::Unhandled();
}

FCursorReply SDataprepGraphActionGroupNode::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	FCursorReply CursorReply = SGraphNodeResizable::OnCursorQuery( MyGeometry, CursorEvent );

	if ( !CursorReply.IsEventHandled() )
	{
		TOptional<EMouseCursor::Type> TheCursor = GetCursor();
		return ( TheCursor.IsSet() )
			? FCursorReply::Cursor( TheCursor.GetValue() )
			: FCursorReply::Unhandled();
	}

	return CursorReply;
}

void SDataprepGraphActionGroupNode::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Track node is not notified of drag left, do it
	TSharedPtr<FDragDropActionNode> DragOperation = DragDropEvent.GetOperationAs<FDragDropActionNode>();
	if (ParentTrackNodePtr.IsValid() && DragOperation.IsValid())
	{
		ParentTrackNodePtr.Pin()->OnDragLeave(DragDropEvent);
	}

	DraggedIndex = INDEX_NONE;

	SGraphNode::OnDragEnter(MyGeometry, DragDropEvent);
}

TSharedRef<SWidget> SDataprepGraphActionGroupNode::CreateNodeContentArea()
{
	PopulateActionsListWidget();

	return SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		ActionsListWidgetPtr.ToSharedRef()
	];
}

FReply SDataprepGraphActionGroupNode::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	SetCursor(EMouseCursor::Default);

	DraggedIndex = INDEX_NONE;

	TSharedPtr<FDragDropActionNode> DragOperation = DragDropEvent.GetOperationAs<FDragDropActionNode>();
	if (DragOperation.IsValid())
	{
		return FReply::Handled().EndDragDrop();
	}

	return SGraphNode::OnDrop(MyGeometry, DragDropEvent);
}

const FSlateBrush* SDataprepGraphActionGroupNode::GetShadowBrush(bool bSelected) const
{
	return FAppStyle::GetNoBrush();
}

void SDataprepGraphActionGroupNode::UpdateExecutionOrder()
{
	ensure(Cast<UDataprepGraphActionGroupNode>(GraphNode));
	ExecutionOrder = Cast<UDataprepGraphActionGroupNode>(GraphNode)->GetExecutionOrder();
}

int32 SDataprepGraphActionGroupNode::GetNumActions() const
{
	int32 NumActions = 0;
	for (int32 Index = 0; Index < ActionGraphNodes.Num(); ++Index)
	{
		if (SDataprepGraphActionNode* Node = ActionGraphNodes[Index].Get())
		{
			NumActions += Node->GetDataprepAction() ? 1 : 0;
		}
	}
	return NumActions;
}

#undef LOCTEXT_NAMESPACE
