// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeResizable.h"
#include "UObject/StrongObjectPtr.h"

class SDataprepGraphTrackNode;
class SDataprepGraphActionProxyNode;
class SDataprepGraphActionStepNode;
class SVerticalBox;
class UDataprepActionAsset;
class UDataprepGraphActionNode;
class UDataprepGraphActionStepNode;
class FDataprepEditor;

/**
 * Base class for graph action nodes
 */
class SDataprepGraphBaseActionNode : public SGraphNodeResizable
{
public:
	void Initialize(TWeakPtr<FDataprepEditor> InDataprepEditor, int32 InExecutionOrder, UEdGraphNode* InNode);

	virtual void SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode)
	{
		ParentTrackNodePtr = InParentTrackNode;
	}

	virtual UDataprepActionAsset* GetDataprepAction() = 0;

	virtual bool IsActionGroup() const { return false; }

	TSharedPtr<SDataprepGraphTrackNode> GetParentTrackNode() { return ParentTrackNodePtr.Pin(); }

	int32 GetExecutionOrder() const { return ExecutionOrder; }
	
	virtual void UpdateExecutionOrder() = 0;

	/** Update the proxy node with relative position in track node */
	void UpdateProxyNode(const FVector2D& Position);

	FMargin GetOuterPadding() const;

	// SWidget interface
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	//~ Begin SPanel Interface
	virtual FVector2D ComputeDesiredSize(float f) const override;
	//~ End SPanel Interface

	//~ Begin SGraphNodeResizable Interface
	virtual FVector2f GetNodeMinimumSize2f() const override;
	virtual FVector2f GetNodeMaximumSize2f() const override;
	//~ Begin SGraphNodeResizable Interface

protected:
	friend SDataprepGraphActionProxyNode;

	/** Pointer to the proxy SGraphNode inserted in the graph panel */
	TSharedPtr<SDataprepGraphActionProxyNode> ProxyNodePtr;

	/** Pointer to the SDataprepGraphTrackNode displayed in the graph editor  */
	TWeakPtr<SDataprepGraphTrackNode> ParentTrackNodePtr;

	/** A optional ptr to a dataprep editor */
	TWeakPtr<FDataprepEditor> DataprepEditor;

	/** Order in which the associated action will be executed by the Dataprep asset */
	int32 ExecutionOrder;

	/** Index of step node being dragged */
	int32 DraggedIndex;

public:
	static float DefaultWidth;
	static float DefaultHeight;

	static TSharedRef<SWidget> CreateBackground(const TAttribute<FSlateColor>& ColorAndOpacity);
};
 
/**
 * The SDataprepGraphActionNode class is the SGraphNode associated
 * to an UDataprepGraphActionNode to display the action's steps in a SDataprepGraphEditor.
 */
class SDataprepGraphActionNode : public SDataprepGraphBaseActionNode
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphActionNode) {}
		SLATE_ARGUMENT(TWeakPtr<FDataprepEditor>, DataprepEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepGraphActionNode* InActionNode);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	// End of SGraphNode interface

	// SDataprepGraphBaseActionNode interface
	virtual void UpdateExecutionOrder() override;
	virtual UDataprepActionAsset* GetDataprepAction() override { return DataprepActionPtr.Get(); }
	virtual void SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode) override;
	// End of SDataprepGraphBaseActionNode interface

	/** Callback used by insert nodes to determine their background color */
	FSlateColor GetInsertColor(int32 Index);

	/** Set index of step node being dragged */
	void SetDraggedIndex(int32 Index);

	/** Get/Set index of step node being hovered */
	int32 GetHoveredIndex() { return InsertIndex; }
	void SetHoveredIndex(int32 Index);

private:
	/** Callback to handle changes on array of steps in action */
	void OnStepsChanged();

	/** Reconstructs the list of widgets associated with the action's steps */
	void PopulateActionStepListWidget();

	FText GetBottomWidgetText() const;

private:
	/** Weak pointer to the associated action asset */
	TWeakObjectPtr<class UDataprepActionAsset> DataprepActionPtr;

	/** Pointer to widget containing all the SDataprepGraphActionStepNode representing the associated action's steps */
	TSharedPtr<SVerticalBox> ActionStepListWidgetPtr;

	/** Array of pointers to the SDataprepGraphActionStepNode representing the associated action's steps */
	TArray<TSharedPtr<SDataprepGraphActionStepNode>> ActionStepGraphNodes;

	/** Index of insert widget to be highlighted */
	int32 InsertIndex;

	/** Array of strong pointers to the UEdGraphNodes created for the action's steps */
	TArray<TStrongObjectPtr<UDataprepGraphActionStepNode>> EdGraphStepNodes;

	TSharedPtr<class SButton> ExpandActionButton;
};

class UDataprepGraphActionGroupNode;

class SDataprepGraphActionGroupNode : public SDataprepGraphBaseActionNode
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphActionGroupNode) {}
		SLATE_ARGUMENT(TWeakPtr<FDataprepEditor>, DataprepEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepGraphActionGroupNode* InActionNode);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	// End of SGraphNode interface

	// SDataprepGraphBaseActionNode interface
	virtual void SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode) override;
	virtual bool IsActionGroup() const override { return true; }
	virtual void UpdateExecutionOrder() override;
	// end SDataprepGraphBaseActionNode interface

	// Return first action
	virtual UDataprepActionAsset* GetDataprepAction()
	{  
		return ActionGraphNodes.Num() > 0 ? ActionGraphNodes[0]->GetDataprepAction() : nullptr;
	}

	UDataprepActionAsset* GetAction(int32 InIndex)
	{  
		return ActionGraphNodes.IsValidIndex(InIndex) ? ActionGraphNodes[InIndex]->GetDataprepAction() : nullptr;
	}

	int32 GetNumActions() const;

private:
	void PopulateActionsListWidget();

	/** Pointer to widget containing all the SDataprepGraphActionNode representing the associated actions in the group */
	TSharedPtr<SVerticalBox> ActionsListWidgetPtr;

	/** Array of pointers to the SDataprepGraphActionNode representing the associated actions */
	TArray<TSharedPtr<SDataprepGraphActionNode>> ActionGraphNodes;

	/** Array of strong pointers to the UEdGraphNodes created for the action's steps */
	TArray<TStrongObjectPtr<UDataprepGraphActionNode>> EdGraphActionNodes;
};
