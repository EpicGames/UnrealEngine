// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepGraphActionStepNode.h"

// Dataprep includes
#include "DataprepCoreUtils.h"
#include "DataprepEditor.h"
#include "DataprepEditorStyle.h"
#include "DataprepGraph/DataprepGraphActionNode.h"
#include "DataprepOperation.h"
#include "SchemaActions/DataprepDragDropOp.h"
#include "SelectionSystem/DataprepFilter.h"
#include "SelectionSystem/DataprepSelectionTransform.h"
#include "Widgets/DataprepGraph/SDataprepFilter.h"
#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"
#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"
#include "Widgets/DataprepGraph/SDataprepOperation.h"
#include "Widgets/DataprepGraph/SDataprepSelectionTransform.h"

// Engine Includes
#include "DragAndDrop/AssetDragDropOp.h"
#include "SGraphPanel.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

void SDataprepGraphActionStepNode::Construct(const FArguments& InArgs, UDataprepGraphActionStepNode* InActionStepNode, const TSharedPtr<SDataprepGraphActionNode>& InParent)
{
	StepIndex = InActionStepNode->GetStepIndex();

	ParentNodePtr = InParent;
	GraphNode = InActionStepNode;
	DataprepEditor = InArgs._DataprepEditor;

	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

void SDataprepGraphActionStepNode::SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode)
{
	ParentTrackNodePtr = InParentTrackNode;
}

TSharedPtr<SWidget> SDataprepGraphActionStepNode::GetStepTitleWidget() const
{
	if(SDataprepActionBlock* ActionStepBlock = ActionStepBlockPtr.Get())
	{
		TAttribute<FSlateColor> BlockColorAndOpacity = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepGraphActionStepNode::GetBlockOverlayColor));

		return SNew(SOverlay)
			+ SOverlay::Slot()
			.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.DnD.Outter.Padding" ))
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SDataprepGraphBaseActionNode::CreateBackground(BlockColorAndOpacity)
			]

			+ SOverlay::Slot()
			.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.DnD.Inner.Padding" ))
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SDataprepGraphBaseActionNode::CreateBackground(FDataprepEditorStyle::GetColor( "DataprepActionStep.BackgroundColor" ))
			]

			+ SOverlay::Slot()
			.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.DnD.Inner.Padding" ))
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.f, 10.f)
				.VAlign(VAlign_Center)
				[
					ActionStepBlock->GetTitleWidget()
				]
			];
	}

	return TSharedPtr<SWidget>();
}

void SDataprepGraphActionStepNode::UpdateGraphNode()
{
	// Reset SGraphNode members.
	InputPins.Empty();
	OutputPins.Empty();
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	TSharedRef<SWidget> ActionBlockPtr = SNullWidget::NullWidget;
	if(UDataprepGraphActionStepNode* ActionStepNode = Cast<UDataprepGraphActionStepNode>(GraphNode))
	{
		TSharedRef< FDataprepSchemaActionContext > StepData = MakeShared< FDataprepSchemaActionContext >();
		StepData->DataprepActionPtr = ActionStepNode->GetDataprepActionAsset();
		StepData->DataprepActionStepPtr =  ActionStepNode->GetDataprepActionStep();
		StepData->StepIndex = ActionStepNode->GetStepIndex();

		if ( UDataprepActionStep* ActionStep = StepData->DataprepActionStepPtr.Get())
		{
			UDataprepParameterizableObject* StepObject = ActionStep->GetStepObject();

			bool bIsPreviewed = false;
			if (TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin())
			{
				bIsPreviewed = DataprepEditorPtr->IsPreviewingStep( StepObject );
			}

			UClass* StepType = FDataprepCoreUtils::GetTypeOfActionStep( StepObject );
			if (StepType == UDataprepOperation::StaticClass())
			{
				UDataprepOperation* Operation = static_cast<UDataprepOperation*>( StepObject );
				ActionStepBlockPtr = StaticCastSharedRef<SDataprepActionBlock>( SNew(SDataprepOperation, Operation, StepData) );
			}
			else if (StepType == UDataprepFilter::StaticClass())
			{
				UDataprepFilter* Filter = static_cast<UDataprepFilter*>( StepObject );

				ActionStepBlockPtr = StaticCastSharedRef<SDataprepActionBlock>( 
					SNew( SDataprepFilter, *Filter, StepData )
						.IsPreviewed( bIsPreviewed )
					);
			}
			else if ( StepType == UDataprepFilterNoFetcher::StaticClass() )
			{
				UDataprepFilterNoFetcher* Filter = static_cast<UDataprepFilterNoFetcher*>( StepObject );

				ActionStepBlockPtr = StaticCastSharedRef<SDataprepActionBlock>( 
					SNew( SDataprepFilterNoFetcher, *Filter, StepData )
						.IsPreviewed( bIsPreviewed )
					);
			}
			else if (StepType == UDataprepSelectionTransform::StaticClass())
			{
				UDataprepSelectionTransform* SelectionTransform = static_cast<UDataprepSelectionTransform*>( StepObject );
				ActionStepBlockPtr = StaticCastSharedRef<SDataprepActionBlock>( SNew(SDataprepSelectionTransform, SelectionTransform, StepData) );
			}

			if(ActionStepBlockPtr.IsValid())
			{
				ActionBlockPtr = ActionStepBlockPtr->AsShared();
			}
		}
	}

	TAttribute<FMargin> OverlayDisabledPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataprepGraphActionStepNode::GetBlockDisabledPadding));
	TAttribute<FMargin> OverlayPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataprepGraphActionStepNode::GetBlockPadding));
	TAttribute<FMargin> ArrowOverlayPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDataprepGraphActionStepNode::GetArrowPadding));
	TAttribute<FSlateColor> BlockColorAndOpacity = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepGraphActionStepNode::GetBlockOverlayColor));

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(5.0f, 0.0f, 10.0f, 0.0f))
		[
			SNew( SSeparator )
			.SeparatorImage(FAppStyle::GetBrush( "ThinLine.Horizontal" ))
			.Thickness(2.f)
			.Orientation(EOrientation::Orient_Horizontal)
			.ColorAndOpacity(this, &SDataprepGraphActionStepNode::GetDragAndDropColor)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.Padding(OverlayPadding)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SDataprepGraphActionNode::CreateBackground(BlockColorAndOpacity)
			]

			+ SOverlay::Slot()
			.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.Padding" ))
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SDataprepGraphActionNode::CreateBackground(FDataprepEditorStyle::GetColor( "DataprepActionStep.BackgroundColor" ))
			]

			+ SOverlay::Slot()
			.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.Padding" ))
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				ActionBlockPtr
			]

			+ SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Padding(OverlayDisabledPadding)
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f))
				.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.Node.Body"))
				.Visibility_Raw( this, &SDataprepGraphActionStepNode::GetDisabledOverlayVisbility )
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(0.0f, -3.0f)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(SDataprepGraphActionStepNode::GetBlockOverlayColor())
					.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.ActionStepNode.ArrowNext"))
					.Visibility_Lambda([this]() 
					{
						if ( IsSelected() && !IsLastStep() )
						{
							return EVisibility::Visible;
						}
						return EVisibility::Hidden;
					})
				]
			]

			+ SOverlay::Slot()
			.Padding(4, 0, 4, 3)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(FDataprepEditorStyle::GetColor( "DataprepActionStep.BackgroundColor" ))
					.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.ActionStepNode.ArrowNext"))
					.Visibility_Lambda([this]() 
					{
						return IsLastStep() ? EVisibility::Collapsed : EVisibility::Visible;
					})
				]
			]

			+ SOverlay::Slot()
			.Padding(ArrowOverlayPadding)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f))
					.Image(FDataprepEditorStyle::GetBrush("DataprepEditor.ActionStepNode.ArrowNext"))
					.Visibility_Lambda([&]() 
					{
						if ( !IsLastStep() )
						{
							return GetDisabledOverlayVisbility();
						}
						return EVisibility::Collapsed;
					})
				]
			]
		]
	];
}

bool SDataprepGraphActionStepNode::IsLastStep() const
{
	if ( UDataprepGraphActionStepNode* StepNode = Cast<UDataprepGraphActionStepNode>(GraphNode) )
	{
		if ( const UDataprepActionAsset* ActionAsset = StepNode->GetDataprepActionAsset() )
		{
			return ( ( ActionAsset->GetStepsCount() - 1 ) == StepIndex );
		}
	}
	return true;
}

bool SDataprepGraphActionStepNode::IsSelected() const
{
	TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
	return OwnerPanel.IsValid() ? OwnerPanel->SelectionManager.SelectedNodes.Contains(GraphNode) : false;
}

EVisibility SDataprepGraphActionStepNode::GetDisabledOverlayVisbility() const
{
	if ( UDataprepGraphActionStepNode* StepNode = Cast<UDataprepGraphActionStepNode>(GraphNode) )
	{
		if ( const UDataprepActionAsset* ActionAsset = StepNode->GetDataprepActionAsset() )
		{
			const UDataprepActionStep* ActionStep = ActionAsset->GetStep(StepIndex).Get();

			// Don't overlay if whole action is also disabled - it will take care of that
			if ( ActionAsset->bIsEnabled && ActionStep && !ActionStep->bIsEnabled )
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

FSlateColor SDataprepGraphActionStepNode::GetBlockOverlayColor() const
{
	static const FSlateColor BlockColor = FDataprepEditorStyle::GetColor( "DataprepActionStep.Filter.OutlineColor" );

	return ActionStepBlockPtr.IsValid() ? ActionStepBlockPtr->GetOutlineColor() : BlockColor;
}

FMargin SDataprepGraphActionStepNode::GetBlockPadding()
{
	static const FMargin Selected = FDataprepEditorStyle::GetMargin( "DataprepActionStep.Outter.Selected.Padding" );
	static const FMargin Regular = FDataprepEditorStyle::GetMargin( "DataprepActionStep.Outter.Regular.Padding" );

	return IsSelected() ? Selected : Regular;
}

FMargin SDataprepGraphActionStepNode::GetBlockDisabledPadding()
{
	static const FMargin Selected = FDataprepEditorStyle::GetMargin( "DataprepActionStep.Outter.Selected.Padding" );
	static const FMargin Regular = FDataprepEditorStyle::GetMargin( "DataprepActionStep.Outter.Disabled.Padding" );

	return IsSelected() ? Selected : Regular;
}

FMargin SDataprepGraphActionStepNode::GetArrowPadding()
{
	return IsSelected() ? FMargin(3, 3, 3, 0) : FMargin(3, 0, 3, 3);
}

FSlateColor SDataprepGraphActionStepNode::GetDragAndDropColor() const
{
	return ParentNodePtr.Pin()->GetInsertColor(StepIndex);
}

FSlateColor SDataprepGraphActionStepNode::GetBorderBackgroundColor() const
{
	static const FLinearColor Selected = FDataprepEditorStyle::GetColor( "DataprepActionStep.DragAndDrop" );
	static const FLinearColor BackgroundColor = FDataprepEditorStyle::GetColor("DataprepActionStep.BackgroundColor");

	return IsSelected() ? Selected : BackgroundColor;
}

FReply SDataprepGraphActionStepNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() ==  EKeys::LeftMouseButton )
	{
		GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );

		if (ParentNodePtr.IsValid())
		{
			UDataprepActionAppearance* ParentAppearance = ParentNodePtr.Pin()->GetDataprepAction()->GetAppearance();

			if (ParentAppearance->GroupId != INDEX_NONE)
			{
				// Disallow dragging of grouped actions/steps
				return FReply::Handled();
			}
		}
		return FReply::Handled().DetectDrag( AsShared(), EKeys::LeftMouseButton );
	}

	// Take ownership of the mouse if right mouse button clicked to display contextual menu
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		if ( !GetOwnerPanel()->SelectionManager.SelectedNodes.Contains( GraphNode ) )
		{
			GetOwnerPanel()->SelectionManager.ClickedOnNode( GraphNode, MouseEvent );
		}
		return FReply::Handled();
	}

	return SGraphNode::OnMouseButtonDown( MyGeometry, MouseEvent);
}

FReply SDataprepGraphActionStepNode::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		ensure(OwnerGraphPanelPtr.IsValid());

		const FVector2D Position = MouseEvent.GetScreenSpacePosition();
		OwnerGraphPanelPtr.Pin()->SummonContextMenu( Position, Position, GraphNode, nullptr, TArray<UEdGraphPin*>() );

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SDataprepGraphActionStepNode::OnMouseMove(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply SDataprepGraphActionStepNode::OnDragDetected(const FGeometry & MyGeometry, const FPointerEvent & MouseEvent)
{
	if (UDataprepGraphActionStepNode* ActionStepNode = Cast<UDataprepGraphActionStepNode>(GraphNode))
	{
		if (UDataprepActionStep* ActionStep = ActionStepNode->GetDataprepActionStep())
		{
			if(SDataprepGraphActionNode* ActionNode = ParentNodePtr.Pin().Get())
			{
				ActionNode->SetDraggedIndex(StepIndex);
				return FReply::Handled().BeginDragDrop(FDataprepDragDropOp::New( SharedThis(ActionNode->GetParentTrackNode().Get()), SharedThis(this)));
			}
		}
	}

	return FReply::Unhandled();
}

void SDataprepGraphActionStepNode::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Is someone dragging a node?
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (DragNodeOp.IsValid())
	{
		ParentTrackNodePtr.Pin()->OnDragLeave(DragDropEvent);

		if (UDataprepActionAsset* DataprepAction = ParentNodePtr.Pin()->GetDataprepAction())
		{
			if (DataprepAction->GetAppearance()->GroupId == INDEX_NONE)
			{
				// Inform the Drag and Drop operation that we are hovering over this node.
				DragNodeOp->SetHoveredNode(GraphNode);
				ParentNodePtr.Pin()->SetHoveredIndex(StepIndex);
			}
			else
			{
				DragNodeOp->SetHoveredNode(nullptr);
				ParentNodePtr.Pin()->SetHoveredIndex(INDEX_NONE);
			}
		}
		return;
	}

	SGraphNode::OnDragEnter(MyGeometry, DragDropEvent);
}

FReply SDataprepGraphActionStepNode::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetOp.IsValid())
	{
		return FReply::Handled();
	}

	// Is someone dragging a node?
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (DragNodeOp.IsValid())
	{
		if (UDataprepActionAsset* DataprepAction = ParentNodePtr.Pin()->GetDataprepAction())
		{
			if (DataprepAction->GetAppearance()->GroupId == INDEX_NONE)
			{
				// Inform the Drag and Drop operation that we are hovering over this node.
				DragNodeOp->SetHoveredNode(GraphNode);
				ParentNodePtr.Pin()->SetHoveredIndex(StepIndex);
			}
			else
			{
				DragNodeOp->SetHoveredNode(nullptr);
				ParentNodePtr.Pin()->SetHoveredIndex(INDEX_NONE);
			}
		}
		return FReply::Handled();
	}

	return SGraphNode::OnDragOver(MyGeometry, DragDropEvent);
}

void SDataprepGraphActionStepNode::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (DragNodeOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are not this widget anymore
		DragNodeOp->SetHoveredNode(nullptr);
		ParentNodePtr.Pin()->SetHoveredIndex(INDEX_NONE);

		return;
	}

	SGraphNode::OnDragLeave(DragDropEvent);
}

FReply SDataprepGraphActionStepNode::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	ParentNodePtr.Pin()->SetDraggedIndex(INDEX_NONE);

	// Process OnDrop if done by FDataprepDragDropOp
	TSharedPtr<FDataprepDragDropOp> DragActionStepNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (DragActionStepNodeOp.IsValid() && !GetOwnerPanel()->SelectionManager.IsNodeSelected(GraphNode))
	{
		const FVector2f NodeAddPosition = NodeCoordToGraphCoord( MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() ) );
		return DragActionStepNodeOp->DroppedOnNode(DragDropEvent.GetScreenSpacePosition(), NodeAddPosition);
	}

	return SGraphNode::OnDrop(MyGeometry, DragDropEvent);
}

#undef LOCTEXT_NAMESPACE
