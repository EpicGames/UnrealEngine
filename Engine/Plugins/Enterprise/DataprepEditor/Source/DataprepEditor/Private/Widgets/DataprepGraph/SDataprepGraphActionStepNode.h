// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SGraphNode.h"

class FDataprepEditor;
class SDataprepActionBlock;
class SDataprepGraphActionNode;
class SDataprepGraphTrackNode;
class STextBlock;
class UDataprepActionAsset;
class UDataprepActionStep;
class UDataprepGraphActionStepNode;

/**
 * The SDataprepGraphActionStepNode class is the SGraphNode associated
 * to an UDataprepGraphActionStepNode to display the step of an action in a SDataprepGraphEditor.
 */
class SDataprepGraphActionStepNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphActionStepNode) {}
		SLATE_ARGUMENT(TWeakPtr<FDataprepEditor>, DataprepEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepGraphActionStepNode* InActionStepNode, const TSharedPtr<SDataprepGraphActionNode>& InParent);

	// SWidget interface
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseButtonUp( const FGeometry & MyGeometry, const FPointerEvent & MouseEvent ) override;
	virtual FReply OnMouseMove(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	// End of SWidget interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual bool IsNameReadOnly () const override { return true; }
	virtual FSlateRect GetTitleRect() const override { return FSlateRect(); }
	// End of SGraphNode interface

	int32 GetStepIndex() const { return StepIndex; }
	void SetStepIndex( int32 InStepIndex ) { StepIndex = InStepIndex; }

	/** Returns a pointer to a widget displaying only the title of the action step */
	virtual TSharedPtr<SWidget> GetStepTitleWidget() const;

	void SetParentTrackNode(TSharedPtr<SDataprepGraphTrackNode> InParentTrackNode);

	TWeakPtr<SDataprepGraphActionNode> GetParentNode() const { return ParentNodePtr; }

private:
	/**
	 * Returns a color depending whether the action step is selected or not
	 */
	FSlateColor GetBorderBackgroundColor() const;
	/**
	 * Returns a color depending whether the action step is hovered or not
	 */
	FSlateColor GetDragAndDropColor() const;

	FSlateColor GetBlockOverlayColor() const;

	FMargin GetBlockPadding();
	FMargin GetBlockDisabledPadding();

	FMargin GetArrowPadding();

	bool IsLastStep() const;

	bool IsSelected() const;

	EVisibility GetDisabledOverlayVisbility() const;

private:
	/** Pointer to the widget displaying the actual filter or operation */
	TSharedPtr<SDataprepActionBlock> ActionStepBlockPtr;

	/** Index of the represented action step in the list of the parent action */
	int32 StepIndex;

	/** Pointer to the SDataprepGraphTrackNode displayed in the graph editor  */
	TWeakPtr<SDataprepGraphTrackNode> ParentTrackNodePtr;

	/** Pointer to the SDataprepGraphTrackNode displayed in the graph editor  */
	TWeakPtr<SDataprepGraphActionNode> ParentNodePtr;

	/** Optional pointer to the dataprep asset */
	TWeakPtr<FDataprepEditor> DataprepEditor;
};
