// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepGraphEditor.h"

#include "DataprepEditor.h"
#include "DataprepEditorStyle.h"
#include "DataprepGraph/DataprepGraph.h"
#include "DataprepGraph/DataprepGraphActionNode.h"
#include "DataprepGraph/DataprepGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "SchemaActions/DataprepAllMenuActionCollector.h"
#include "SchemaActions/DataprepDragDropOp.h"
#include "SelectionSystem/DataprepObjectSelectionFilter.h"
#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"
#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"
#include "Widgets/SDataprepActionMenu.h"
#include "Widgets/SAssetsPreviewWidget.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

#ifdef ACTION_NODE_MOCKUP
static int32 MockupActionCount = 2;
#endif

const float SDataprepGraphEditor::TopPadding = 60.f;
const float SDataprepGraphEditor::BottomPadding = 15.f;
const float SDataprepGraphEditor::HorizontalPadding = 20.f;

TSharedPtr<FDataprepGraphEditorNodeFactory> SDataprepGraphEditor::NodeFactory;

namespace DataprepGraphEditorUtils
{
	void ForEachActionAndStep(const TSet<UObject*>& Nodes,  TFunctionRef<bool (UDataprepActionAsset*, UDataprepGraphActionNode*)> OnEachAction, TFunctionRef<bool (UDataprepParameterizableObject*, UDataprepActionStep*)> OnEachStep)
	{
		for (UObject* Node : Nodes)
		{
			if ( Node )
			{
				if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(Node))
				{
					if ( OnEachAction( ActionNode->GetDataprepActionAsset(), ActionNode ) )
					{
						break;
					}
				}
				else if (UDataprepGraphActionStepNode* StepNode = Cast<UDataprepGraphActionStepNode>(Node) )
				{
					if (UDataprepActionAsset* Action = StepNode->GetDataprepActionAsset())
					{
						if ( OnEachAction( Action, nullptr ) )
						{
							break;
						}

						const int32 StepIndex = StepNode->GetStepIndex();
						if (UDataprepActionStep* Step = Action->GetStep(StepIndex).Get())
						{
							if (UDataprepParameterizableObject* StepObject = Step->GetStepObject())
							{
								if ( OnEachStep( StepObject, Step ) )
								{
									break;
								}
							}
						}
					}
				}
			}
		}
	}
}

TSharedPtr<SGraphNode> FDataprepGraphNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	if (UDataprepGraphRecipeNode* RecipeNode = Cast<UDataprepGraphRecipeNode>(InNode))
	{
		return SNew(SDataprepGraphTrackNode, RecipeNode)
			.NodeFactory( AsShared() );
	}
	else if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(InNode))
	{
		return SNew(SDataprepGraphActionNode, ActionNode)
			.DataprepEditor( DataprepEditor );
	}
	else if (UDataprepGraphActionGroupNode* GroupNode = Cast<UDataprepGraphActionGroupNode>(InNode))
	{
		return SNew(SDataprepGraphActionGroupNode, GroupNode)
			.DataprepEditor( DataprepEditor );
	}

	return FGraphNodeFactory::CreateNodeWidget( InNode );
}

TSharedPtr<SGraphNode> FDataprepGraphEditorNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (UDataprepGraphRecipeNode* RecipeNode = Cast<UDataprepGraphRecipeNode>(Node))
	{
		return SNew(SDataprepGraphTrackNode, RecipeNode);
	}
	else if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(Node))
	{
		return SNew(SDataprepGraphActionNode, ActionNode);
	}
	else if (UDataprepGraphActionGroupNode* GroupNode = Cast<UDataprepGraphActionGroupNode>(Node))
	{
		return SNew(SDataprepGraphActionGroupNode, GroupNode);
	}

	return nullptr;
}

void SDataprepGraphEditor::RegisterFactories()
{
	if(!NodeFactory.IsValid())
	{
		NodeFactory  = MakeShared<FDataprepGraphEditorNodeFactory>();
		FEdGraphUtilities::RegisterVisualNodeFactory(NodeFactory);
	}
}

void SDataprepGraphEditor::UnRegisterFactories()
{
	if(NodeFactory.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(NodeFactory);
		NodeFactory.Reset();
	}
}

void SDataprepGraphEditor::Construct(const FArguments& InArgs, UDataprepAsset* InDataprepAsset)
{
	check(InDataprepAsset);
	DataprepAssetPtr = InDataprepAsset;
	DataprepEditor = InArgs._DataprepEditor;

	SGraphEditor::FGraphEditorEvents Events;
	Events.OnCreateNodeOrPinMenu = SGraphEditor::FOnCreateNodeOrPinMenu::CreateSP(this, &SDataprepGraphEditor::OnCreateNodeOrPinMenu);
	Events.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SDataprepGraphEditor::OnCreateActionMenu);
	Events.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &SDataprepGraphEditor::OnNodeVerifyTitleCommit);
	Events.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &SDataprepGraphEditor::OnNodeTitleCommitted);

	BuildCommandList();

	SGraphEditor::FArguments Arguments;
	Arguments._AdditionalCommands = GraphEditorCommands;
	Arguments._TitleBar = InArgs._TitleBar;
	Arguments._GraphToEdit = InArgs._GraphToEdit;
	Arguments._GraphEvents = Events;

	SGraphEditor::Construct( Arguments );

	DataprepAssetPtr->GetOnActionChanged().AddSP(this, &SDataprepGraphEditor::OnDataprepAssetActionChanged);

	SetCanTick(true);

	bIsComplete = false;

	LastLocalSize = FVector2D::ZeroVector;
	LastZoomAmount = BIG_NUMBER;

	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	bCachedControlKeyDown = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();

	SetNodeFactory( MakeShared<FDataprepGraphNodeFactory>( InArgs._DataprepEditor ) );
}

void SDataprepGraphEditor::NotifyGraphChanged()
{
	// Release track node widget as it is about to be re-created
	TrackGraphNodePtr.Reset();
	bIsComplete = false;

	// Reset cached size and zoom. No need for location because it is invalidated when size has changed
	LastLocalSize = FVector2D::ZeroVector;
	LastZoomAmount = BIG_NUMBER;

	SGraphEditor::NotifyGraphChanged();
}

void SDataprepGraphEditor::OnDataprepAssetActionChanged(UObject* InObject, FDataprepAssetChangeType ChangeType)
{
	switch(ChangeType)
	{
		case FDataprepAssetChangeType::ActionAdded:
		case FDataprepAssetChangeType::ActionRemoved:
		{
			TArray<UEdGraphNode*> ToDelete;
			TArray<TObjectPtr<UEdGraphNode>>& Nodes = GetCurrentGraph()->Nodes;
			for(UEdGraphNode* NodeObject : Nodes)
			{
				if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(NodeObject))
				{
					ToDelete.Add(NodeObject);
				}
			}

			for(UEdGraphNode* NodeObject : ToDelete)
			{
				Nodes.Remove(NodeObject);
			}

			NotifyGraphChanged();

			break;
		}

		case FDataprepAssetChangeType::ActionMoved:
		{
			if(SDataprepGraphTrackNode* TrackGraphNode = TrackGraphNodePtr.Pin().Get())
			{
				TrackGraphNode->OnActionsOrderChanged();
			}
			break;
		}

		case FDataprepAssetChangeType::ActionAppearanceModified:
		{
			if(SDataprepGraphTrackNode* TrackGraphNode = TrackGraphNodePtr.Pin().Get())
			{
				NotifyGraphChanged();
			}
			break;
		}
	}
}

void SDataprepGraphEditor::CacheDesiredSize(float InLayoutScaleMultiplier)
{
	SGraphEditor::CacheDesiredSize(InLayoutScaleMultiplier);

	if(!bIsComplete && !NeedsPrepass())
	{
		if(!TrackGraphNodePtr.IsValid())
		{
			// Get track SGraphNode and initialize it
			if(UDataprepAsset* DataprepAsset = DataprepAssetPtr.Get())
			{
				for(UEdGraphNode* EdGraphNode : GetCurrentGraph()->Nodes)
				{
					if(UDataprepGraphRecipeNode* TrackNode = Cast<UDataprepGraphRecipeNode>(EdGraphNode))
					{
						TrackGraphNodePtr = StaticCastSharedPtr<SDataprepGraphTrackNode>(TrackNode->GetWidget());
						break;
					}
				}
			}
		}

		if(TrackGraphNodePtr.IsValid())
		{
			bIsComplete = TrackGraphNodePtr.Pin()->RefreshLayout();
		}
	}
}

void SDataprepGraphEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Do not change the layout until all widgets have been created.
	// This happens after the first call to OnPaint on the editor
	if(bIsComplete)
	{
		if(SDataprepGraphTrackNode* TrackGraphNode = TrackGraphNodePtr.Pin().Get())
		{
			FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
			bool bControlKeyDown = ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown();
			if(bControlKeyDown != bCachedControlKeyDown)
			{
				bCachedControlKeyDown = bControlKeyDown;
				TrackGraphNode->OnControlKeyChanged(bCachedControlKeyDown);
			}
		}

		FVector2f Location;
		float ZoomAmount = 1.f;
		GetViewLocation( Location, ZoomAmount );

		UpdateLayout( AllottedGeometry.GetLocalSize(), FDeprecateSlateVector2D(Location), ZoomAmount );
	}

	SGraphEditor::Tick( AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SDataprepGraphEditor::UpdateLayout( const UE::Slate::FDeprecateVector2DParameter& LocalSize, const UE::Slate::FDeprecateVector2DParameter& Location, float ZoomAmount )
{
	if(SDataprepGraphTrackNode* TrackGraphNode = TrackGraphNodePtr.Pin().Get())
	{
		WorkingArea = TrackGraphNode->Update();

		if( !LocalSize.Equals(LastLocalSize) )
		{
			LastLocalSize = LocalSize;

			// Force a re-compute of the view location
			LastLocation = -Location;
		}

		if( !Location.Equals(LastLocation) )
		{
			FVector2f ComputedLocation(Location);

			FVector2f PanelSize = LocalSize / ZoomAmount;
			FVector2f WorkingSize = WorkingArea.GetSize();

			if(Location.X != LastLocation.X)
			{
				const float Delta = WorkingSize.X - PanelSize.X;
				const float MaxRight =  Delta > 0.f ? WorkingArea.Left + Delta : WorkingArea.Left;
				ComputedLocation.X = ComputedLocation.X < WorkingArea.Left ? WorkingArea.Left : (ComputedLocation.X >= MaxRight ? MaxRight : ComputedLocation.X);
			}

			if(Location.Y != LastLocation.Y)
			{
				const float Delta = WorkingSize.Y - PanelSize.Y;
				const float MaxBottom =  Delta > 0.f ? WorkingArea.Top + Delta : WorkingArea.Top;
				ComputedLocation.Y = ComputedLocation.Y < WorkingArea.Top ? WorkingArea.Top : (ComputedLocation.Y >= MaxBottom ? MaxBottom : ComputedLocation.Y);
			}

			LastLocation = Location;

			if(ComputedLocation != Location)
			{
				SetViewLocation( ComputedLocation, ZoomAmount );
				LastLocation = ComputedLocation;
			}
		}

		LastZoomAmount = ZoomAmount;
	}
}

void SDataprepGraphEditor::OnDragEnter(const FGeometry & MyGeometry, const FDragDropEvent & DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (TrackGraphNodePtr.IsValid() && DragNodeOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are hovering over this node.
		DragNodeOp->SetTrackNode(TrackGraphNodePtr.Pin());
	}

	SGraphEditor::OnDragEnter(MyGeometry, DragDropEvent);
}

FReply SDataprepGraphEditor::OnDragOver(const FGeometry & MyGeometry, const FDragDropEvent & DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (TrackGraphNodePtr.IsValid() && DragNodeOp.IsValid())
	{
		TrackGraphNodePtr.Pin()->OnDragOver(MyGeometry, DragDropEvent);
	}

	return SGraphEditor::OnDragOver(MyGeometry, DragDropEvent);
}

void SDataprepGraphEditor::OnDragLeave(const FDragDropEvent & DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (TrackGraphNodePtr.IsValid() && DragNodeOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are not hovering over this node anymore.
		DragNodeOp->SetTrackNode(TSharedPtr<SDataprepGraphTrackNode>());
	}

	SGraphEditor::OnDragLeave(DragDropEvent);
}

FReply SDataprepGraphEditor::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if (TrackGraphNodePtr.IsValid() && DragNodeOp.IsValid())
	{
		return FReply::Handled().EndDragDrop();
	}

	return SGraphEditor::OnDrop(MyGeometry, DragDropEvent);
}

void SDataprepGraphEditor::BuildCommandList()
{
	if (!GraphEditorCommands.IsValid())
	{
		GraphEditorCommands = MakeShareable(new FUICommandList);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::OnRenameNode),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanRenameNode)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::SelectAllNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanSelectAllNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::DeleteSelectedNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanDeleteNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::CopySelectedNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanCopyNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::CutSelectedNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanCutNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::PasteNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanPasteNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &SDataprepGraphEditor::DuplicateNodes),
			FCanExecuteAction::CreateSP(this, &SDataprepGraphEditor::CanDuplicateNodes)
		);
	}
}

void SDataprepGraphEditor::OnRenameNode()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
		if (SelectedNode != NULL && SelectedNode->bCanRenameNode && TrackGraphNodePtr.IsValid())
		{
			TrackGraphNodePtr.Pin()->RequestRename(SelectedNode);
			break;
		}
	}
}

bool SDataprepGraphEditor::CanRenameNode() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if ( SelectedNodes.Num() == 1)
	{
		if ( UDataprepGraphActionNode* SelectedNode = Cast<UDataprepGraphActionNode>(*SelectedNodes.CreateConstIterator()) )
		{
			return SelectedNode->bCanRenameNode;
		}
	}

	return false;
}

void SDataprepGraphEditor::DeleteSelectedNodes()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	TArray<int32> ActionsToDelete;
	TSet<UDataprepActionAsset*> ActionAssets;
	for(UObject* NodeObject : SelectedNodes)
	{
		if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(NodeObject))
		{
			if (ActionNode->CanUserDeleteNode() && ActionNode->GetDataprepActionAsset())
			{
				ActionsToDelete.Add(DataprepAssetPtr->GetActionIndex(ActionNode->GetDataprepActionAsset()));
				ActionAssets.Add(ActionNode->GetDataprepActionAsset());
			}
		}
	}

	UEdGraph* EdGraph = GetCurrentGraph();

	TMap<UDataprepActionAsset*, TArray<int32>> StepsToDelete;
	for(UObject* NodeObject : SelectedNodes)
	{
		if (UDataprepGraphActionStepNode* ActionStepNode = Cast<UDataprepGraphActionStepNode>(NodeObject))
		{
			UDataprepActionAsset* ActionAsset = ActionStepNode->GetDataprepActionAsset();
			if (ActionStepNode->CanUserDeleteNode() && ActionAsset && !ActionAssets.Contains(ActionAsset))
			{
				TArray<int32>& ToDelete = StepsToDelete.FindOrAdd(ActionAsset);
				ToDelete.Add(ActionStepNode->GetStepIndex());

				// Delete action if all its steps are deleted
				if(ActionAsset->GetStepsCount() == ToDelete.Num())
				{
					StepsToDelete.Remove(ActionAsset);
					int32 Index = DataprepAssetPtr->GetActionIndex(ActionAsset);
					ensure(Index != INDEX_NONE);
					ActionsToDelete.Add(Index);

					TArray<TObjectPtr<class UEdGraphNode>>& Nodes = EdGraph->Nodes;
					for(Index = 0; Index < Nodes.Num(); ++Index)
					{
						if(UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(Nodes[Index]))
						{
							if(ActionNode->GetDataprepActionAsset() == ActionAsset)
							{
								SelectedNodes.Add(Nodes[Index]);
								break;
							}
						}
					}
				}
			}
		}
	}

	FScopedTransaction Transaction(FGenericCommands::Get().Delete->GetDescription());
	bool bTransactionSuccessful = true;

	if(ActionsToDelete.Num() > 0)
	{
		bTransactionSuccessful &= DataprepAssetPtr->RemoveActions(ActionsToDelete);
	}

	if(StepsToDelete.Num() > 0)
	{
		for(TPair<UDataprepActionAsset*, TArray<int32>>& Entry : StepsToDelete)
		{
			if(UDataprepActionAsset* ActionAsset = Entry.Key)
			{
				bTransactionSuccessful &= ActionAsset->RemoveSteps(Entry.Value);
			}
		}
	}

	TArray<TObjectPtr<UEdGraphNode>>& Nodes = EdGraph->Nodes;
	for(UObject* NodeObject : SelectedNodes)
	{
		if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(NodeObject))
		{
			Nodes.Remove(ActionNode);
		}
	}

	if (!bTransactionSuccessful)
	{
		Transaction.Cancel();
	}
	else
	{
		ClearSelectionSet();
	}
}

bool SDataprepGraphEditor::CanDeleteNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (UObject* NodeObject : SelectedNodes)
	{
		// If any nodes allow deleting, then do not disable the delete option
		UEdGraphNode* Node = Cast<UEdGraphNode>(NodeObject);
		if (Node && Node->CanUserDeleteNode())
		{
			return true;
		}
	}

	return false;
}

void SDataprepGraphEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	TSet<UObject*> ActionsToCopy;

	for (UObject* NodeObject : SelectedNodes)
	{
		if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(NodeObject))
		{
			UDataprepActionAsset* ActionAsset = ActionNode->GetDataprepActionAsset();
			// Temporarily set DataprepActionAsset's owner as ActionNode to serialize it with the EdGraphNode
			ActionAsset->Rename(nullptr, ActionNode, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);

			ActionNode->PrepareForCopying();
			ActionsToCopy.Add(ActionNode);
		}
	}

	if(ActionsToCopy.Num() > 0)
	{
		FString ExportedText;

		FEdGraphUtilities::ExportNodesToText(ActionsToCopy, /*out*/ ExportedText);
		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

		// Restore DataprepActionAssets' owner to the DataprepAsset
		for (UObject* NodeObject : ActionsToCopy)
		{
			if (UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(NodeObject))
			{
				UDataprepActionAsset* ActionAsset = ActionNode->GetDataprepActionAsset();
				ActionAsset->Rename(nullptr, DataprepAssetPtr.Get(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
		}
	}
}

bool SDataprepGraphEditor::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (UObject* NodeObject : SelectedNodes)
	{
		UDataprepGraphActionNode* Node = Cast<UDataprepGraphActionNode>(NodeObject);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void SDataprepGraphEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool SDataprepGraphEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void SDataprepGraphEditor::PasteNodes()
{
	ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Create temporary graph
	FName UniqueGraphName = MakeUniqueObjectName( GetTransientPackage(), UWorld::StaticClass(), FName( *(LOCTEXT("DataprepTempGraph", "TempGraph").ToString()) ) );
	TStrongObjectPtr<UDataprepGraph> DataprepGraph = TStrongObjectPtr<UDataprepGraph>( NewObject< UDataprepGraph >(GetTransientPackage(), UniqueGraphName) );
	DataprepGraph->Schema = UDataprepGraphSchema::StaticClass();

	// Import the nodes
	// #ueent_wip: The FEdGraphUtilities::ImportNodesFromTex could be replaced
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(DataprepGraph.Get(), TextToImport, /*out*/ PastedNodes);

	TArray<const UDataprepActionAsset*> Actions;
	for(UEdGraphNode* Node : PastedNodes)
	{
		UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(Node);
		if (ActionNode && ActionNode->CanDuplicateNode() && ActionNode->GetDataprepActionAsset())
		{
			Actions.Add(ActionNode->GetDataprepActionAsset());
		}
	}

	if(Actions.Num() > 0)
	{
		FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());

		if(DataprepAssetPtr->AddActions(Actions) == INDEX_NONE)
		{
			Transaction.Cancel();
		}
	}
}

bool SDataprepGraphEditor::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(GetCurrentGraph(), ClipboardContent);
}

void SDataprepGraphEditor::DuplicateNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool SDataprepGraphEditor::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

void SDataprepGraphEditor::DeleteSelectedDuplicatableNodes()
{
	// Cache off the old selection
	const FGraphPanelSelectionSet OldSelectedNodes = GetSelectedNodes();

	// Clear the selection and only select the nodes that can be duplicated
	FGraphPanelSelectionSet CurrentSelection;
	ClearSelectionSet();

	FGraphPanelSelectionSet RemainingNodes;
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			SetNodeSelection(Node, true);
		}
		else
		{
			RemainingNodes.Add(Node);
		}
	}

	// Delete the nodes which can be duplicated
	DeleteSelectedNodes();

	// Reselect whatever is left from the original selection after the deletion
	ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(RemainingNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			SetNodeSelection(Node, true);
		}
	}
}

bool SDataprepGraphEditor::OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	bool bValid(false);

	if (NodeBeingChanged && NodeBeingChanged->bCanRenameNode)
	{
		// Clear off any existing error message 
		NodeBeingChanged->ErrorMsg.Empty();
		NodeBeingChanged->bHasCompilerMessage = false;

		TSharedPtr<INameValidatorInterface> NameEntryValidator = NodeBeingChanged->MakeNameValidator();

		check( NameEntryValidator.IsValid() );

		EValidatorResult VResult = NameEntryValidator->IsValid(NewText.ToString(), true);
		if (VResult == EValidatorResult::Ok)
		{
			bValid = true;
		}
	}

	return bValid;
}

void SDataprepGraphEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("RenameNode", "RenameNode", "Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

TSet<UObject*> SDataprepGraphEditor::GetSelectedActorsAndAssets()
{
	TSet< UObject* > ActorAndAssetsSelection;

	if ( TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin() )
	{
		const TSet< TWeakObjectPtr<UObject> >& WorldSelection = DataprepEditorPtr->GetWorldItemsSelection();

		for ( const TWeakObjectPtr<UObject>& WeakObject : WorldSelection )
		{
			if ( UObject* Object = WeakObject.Get() )
			{
				ActorAndAssetsSelection.Add( Object );
			}
		}

		if ( TSharedPtr<AssetPreviewWidget::SAssetsPreviewWidget> AssetsPreviewPtr = DataprepEditorPtr->GetAssetPreviewView().Pin() )
		{
			ActorAndAssetsSelection.Append( AssetsPreviewPtr->GetSelectedAssets() );
		}
	}

	return MoveTemp( ActorAndAssetsSelection );
}

void SDataprepGraphEditor::OnCollectCustomActions(TArray<TSharedPtr<FDataprepSchemaAction>>& OutActions)
{
	TSet< UObject* > AssetAndActorSelection = GetSelectedActorsAndAssets();

	if ( AssetAndActorSelection.Num() == 0 )
	{
		return;
	}

	FDataprepSchemaAction::FOnExecuteAction OnExcuteMenuAction;
	OnExcuteMenuAction.BindLambda( [this, AssetAndActorSelection] (const FDataprepSchemaActionContext& InContext)
	{
		if( UDataprepActionAsset* Action = InContext.DataprepActionPtr.Get() )
		{
			CreateFilterFromSelection( Action, AssetAndActorSelection );
		}
	});

	TSharedPtr<FDataprepSchemaAction> Action = MakeShared< FDataprepSchemaAction >( FText::FromString( TEXT("") )
		, FText::FromString( TEXT("Create filter from selection") ), FText::FromString( TEXT("Create filter from selection") )
		, MAX_int32, FText::FromString( TEXT("") ), OnExcuteMenuAction);

	OutActions.Add( Action );
}

void SDataprepGraphEditor::CreateFilterFromSelection(UDataprepActionAsset* InTargetAction, const TSet< UObject* >& InAssetAndActorSelection)
{
	check( InTargetAction );

	if ( InAssetAndActorSelection.Num() == 0 )
	{
		return;
	}

	TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin();
	check( DataprepEditorPtr );

	UDataprepObjectSelectionFilter* Filter = NewObject< UDataprepObjectSelectionFilter >( GetTransientPackage(), UDataprepObjectSelectionFilter::StaticClass(), NAME_None, RF_Transactional );
	Filter->SetSelection( DataprepEditorPtr->GetTransientContentFolder(), InAssetAndActorSelection.Array() );

	int32 NewFilterIndex = InTargetAction->AddStep( Filter );
	ensure( NewFilterIndex != INDEX_NONE );

	if(SDataprepGraphTrackNode* TrackGraphNode = TrackGraphNodePtr.Pin().Get())
	{
		TrackGraphNode->UpdateGraphNode();
	}
}

FActionMenuContent SDataprepGraphEditor::OnCreateActionMenu(UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	return OnCreateActionMenu(InGraph, UE::Slate::CastToVector2f(InNodePosition), InDraggedPins, bAutoExpand, InOnMenuClosed);
}

FActionMenuContent SDataprepGraphEditor::OnCreateActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	// bAutoExpand is voluntary ignored for now
	TUniquePtr<IDataprepMenuActionCollector> ActionCollector = MakeUnique<FDataprepAllMenuActionCollector>();

	TSharedRef<SDataprepActionMenu> ActionMenu =
		SNew (SDataprepActionMenu, MoveTemp(ActionCollector))
		.TransactionText(LOCTEXT("AddingANewActionNode","Add a Action Node"))
		.GraphObj(InGraph)
		.NewNodePosition(FDeprecateSlateVector2D(InNodePosition))
		.DraggedFromPins(InDraggedPins)
		.OnClosedCallback(InOnMenuClosed)
		.OnCollectCustomActions(SDataprepActionMenu::FOnCollectCustomActions::CreateSP(this, &SDataprepGraphEditor::OnCollectCustomActions));

	return FActionMenuContent( ActionMenu, ActionMenu->GetFilterTextBox() );
}

FActionMenuContent SDataprepGraphEditor::OnCreateNodeOrPinMenu(UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging)
{
	if(CurrentGraph != GetCurrentGraph())
	{
		return FActionMenuContent();
	}

	// Open contextual menu for action node
	UClass* GraphNodeClass = InGraphNode->GetClass();
	const bool bIsActionNode = GraphNodeClass == UDataprepGraphActionNode::StaticClass();
	const bool bIsActionGroupNode = GraphNodeClass == UDataprepGraphActionGroupNode::StaticClass();
	const UDataprepGraphActionStepNode* FirstStepNode = Cast<UDataprepGraphActionStepNode>( InGraphNode );
	
	if (bIsActionGroupNode)
	{
		TArray<const UDataprepGraphActionGroupNode*> ActionGroupNodes;

		ActionGroupNodes.Add(Cast<UDataprepGraphActionGroupNode>(InGraphNode));

		// Check if we have other selected nodes and if they are group nodes as well
		FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

		for (UObject* Node : SelectedNodes)
		{
			if ( Node )
			{
				if (UDataprepGraphActionGroupNode* ActionGroupNode = Cast<UDataprepGraphActionGroupNode>(Node))
				{
					ActionGroupNodes.Add(ActionGroupNode);
				}
				else
				{
					// Selection does not consist of group nodes only: halt
					ActionGroupNodes.Empty();
					break;
				}
			}
		}

		if (ActionGroupNodes.Num() > 0)
		{
			FUIAction BreakGroupAction;
			BreakGroupAction.ExecuteAction.BindLambda([this, ActionGroupNodes]() 
			{
				const FScopedTransaction Transaction(NSLOCTEXT("BreakActions", "BreakActions", "Ungroup Actions"));

				DataprepAssetPtr->Modify();

				for (int32 GroupNodeIndex = 0; GroupNodeIndex < ActionGroupNodes.Num(); ++GroupNodeIndex)
				{
					const UDataprepGraphActionGroupNode* ActionGroupNode = ActionGroupNodes[GroupNodeIndex];

					for (int32 Index = 0; Index < ActionGroupNode->GetActionsCount(); ++Index)
					{
						UDataprepActionAsset* Action = ActionGroupNode->GetAction(Index);
						Action->Modify();
						Action->GetAppearance()->Modify();
						Action->GetAppearance()->GroupId = INDEX_NONE;
					}
				}
				NotifyGraphChanged();
			});

			const bool bShouldDisable = ActionGroupNodes[0]->IsGroupEnabled();
			const FText EnableOrDisableText = bShouldDisable ? FText::FromString("Disable") : FText::FromString("Enable");
			FUIAction EnableOrDisableGroupAction;
			EnableOrDisableGroupAction.ExecuteAction.BindLambda([this, ActionGroupNodes, bShouldDisable, EnableOrDisableText]() 
			{
				const FScopedTransaction Transaction(FText::Format(NSLOCTEXT("EnableOrDisableGroup", "EnableOrDisableGroup", "{0} Action Group"), EnableOrDisableText));

				DataprepAssetPtr->Modify();

				for (int32 GroupNodeIndex = 0; GroupNodeIndex < ActionGroupNodes.Num(); ++GroupNodeIndex)
				{
					const UDataprepGraphActionGroupNode* ActionGroupNode = ActionGroupNodes[GroupNodeIndex];

					if (ActionGroupNode->GetActionsCount() > 0)
					{
						for (int32 Index = 0; Index < ActionGroupNode->GetActionsCount(); ++Index)
						{
							UDataprepActionAsset* Action = ActionGroupNode->GetAction(Index);
							Action->Modify();
							Action->GetAppearance()->Modify();
							Action->GetAppearance()->bGroupIsEnabled = !bShouldDisable;
						}
					}
				}

				NotifyGraphChanged();
			});

			MenuBuilder->BeginSection( FName( TEXT("CommonSection") ), LOCTEXT("CommonSection", "Common") );
			{
				MenuBuilder->AddMenuEntry(LOCTEXT( "BreakGroup", "Ungroup Actions" ),
										  LOCTEXT( "BreakGroupTooltip", "Break group to single actions" ),
										  FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Pipeline.UngroupActions"),
										  BreakGroupAction);

				const FName EnableDisableIcon = bShouldDisable ? FName("DataprepEditor.Pipeline.DisableActionGroup") : FName("DataprepEditor.Pipeline.EnableActionGroup");

				MenuBuilder->AddMenuEntry( FText::Format( LOCTEXT( "EnableOrDisableEnableGroupLabel", "{0} Action Group" ), EnableOrDisableText ),
										   FText::Format( LOCTEXT( "EnableOrDisableEnableGroupTooltip", "{0} Action Group" ), EnableOrDisableText ),
										   FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), EnableDisableIcon),
										   EnableOrDisableGroupAction);
			}
			MenuBuilder->EndSection();

			return FActionMenuContent(MenuBuilder->MakeWidget());
		}
		else
		{
			return FActionMenuContent(SNullWidget::NullWidget, SNullWidget::NullWidget);
		}
	}
	else if( bIsActionNode || FirstStepNode )
	{
		FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
		// Actions and ActionNodes are in sync (thus same size) - Actions[N] comes from ActionNodes[N]
		TArray<UDataprepActionAsset*> Actions;
		TArray<UDataprepGraphActionNode*> ActionNodes;

		TArray<UDataprepActionStep*> ActionSteps;

		auto OnSaveActions = [&Actions, &ActionNodes](UDataprepActionAsset* Action, UDataprepGraphActionNode* ActionNode) -> bool
		{
			const bool bIsFromActionNode = ActionNode != nullptr;

			if (bIsFromActionNode)
			{
				Actions.Add(Action);
				ActionNodes.Add(ActionNode);
			}
			return false;
		};

		auto OnSaveActionSteps = [&ActionSteps](UDataprepParameterizableObject* StepObject, UDataprepActionStep* ActionStep) -> bool
		{
			ActionSteps.Add(ActionStep);
			return false;
		};

		DataprepGraphEditorUtils::ForEachActionAndStep(SelectedNodes, OnSaveActions, OnSaveActionSteps);

		MenuBuilder->BeginSection( FName( TEXT("CommonSection") ), LOCTEXT("CommonSection", "Common") );
		{
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Delete);

			// Add enable/disable menu item
			if (Actions.Num() > 0 || ActionSteps.Num() > 0)
			{
				bool bShouldDisable = true;

				// Figure out the operation (disable or enable) from the clicked node.
				// Note: all of the selected items (steps and/or actions) will be set according to this operation
				if (bIsActionNode)
				{
					const UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>(InGraphNode);
					const UDataprepActionAsset* ActionAsset = ActionNode->GetDataprepActionAsset();
					bShouldDisable = ActionAsset->bIsEnabled;
				}
				else
				{
					const UDataprepActionAsset* Action = FirstStepNode->GetDataprepActionAsset();
					const int32 StepIndex = FirstStepNode->GetStepIndex();
					if ( Action )
					{
						ensure( StepIndex >= 0 && StepIndex < Action->GetStepsCount() );
						UDataprepActionStep* ActionStep = Action->GetStep(StepIndex).Get();
						bShouldDisable = ActionStep->bIsEnabled;
					}
				}

				FUIAction EnableOrDisableItemsAction;
				EnableOrDisableItemsAction.ExecuteAction.BindLambda([bShouldDisable, Actions, ActionSteps, InGraphNode]() 
				{
					FScopedTransaction Transaction( LOCTEXT("ActionEnableDisableTransaction", "Enabled/disabled actions") );

					for (UDataprepActionAsset* ActionAsset : Actions)
					{
						ActionAsset->Modify();
						ActionAsset->bIsEnabled = !bShouldDisable;
					}
					for (UDataprepActionStep* ActionStep : ActionSteps)
					{
						ActionStep->Modify();
						ActionStep->bIsEnabled = !bShouldDisable;
					}
				});

				FText Label = bShouldDisable ? FText::FromString("Disable") : FText::FromString("Enable");
				const FName IconStyle = bShouldDisable ? FName("DataprepEditor.Pipeline.DisableActions") : FName("DataprepEditor.Pipeline.EnableActions");
				MenuBuilder->AddMenuEntry(FText::Format( LOCTEXT( "EnableOrDisableItemsAction", "{0}" ), Label ),
										  FText::Format( LOCTEXT( "EnableOrDisableItemsActionTooltip", "{0} steps/actions" ), Label ),
										  FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), IconStyle),
										  EnableOrDisableItemsAction);
			}

			// Add collapse actions menu item
			if (bIsActionNode && Actions.Num() > 1)
			{
				ActionNodes.Sort([](const UDataprepGraphActionNode& L, const UDataprepGraphActionNode& R) { return L.GetExecutionOrder() < R.GetExecutionOrder(); });

				int32 PrevExecutionOrder = 0;
				bool bActionsAreAdjacent = true;
				for (int32 ActionIndex = 0; ActionIndex < ActionNodes.Num(); ++ActionIndex)
				{
					const int32 CurrExecutionOrder = ActionNodes[ActionIndex]->GetExecutionOrder();
					if (ActionIndex > 0 && (CurrExecutionOrder - PrevExecutionOrder) != 1)
					{
						bActionsAreAdjacent = false;
						break;
					}

					PrevExecutionOrder = CurrExecutionOrder;
				}

				if (bActionsAreAdjacent)
				{
					TArray<UDataprepActionAsset*> ActionsToCollapse;

					for (int32 ActionIndex = 0; ActionIndex < ActionNodes.Num(); ++ActionIndex)
					{
						ActionsToCollapse.Add(ActionNodes[ActionIndex]->GetDataprepActionAsset());
					}

					FUIAction CollapseActions;
					CollapseActions.ExecuteAction.BindLambda([this, ActionsToCollapse]() 
					{
						// Generate id for the new group: first get the ids of all current groups

						TArray<int32> GroupIds;

						int32 NewGroupId = 0;

						for(int32 ActionIndex = 0; ActionIndex < DataprepAssetPtr->GetActionCount(); ++ActionIndex)
						{
							if(UDataprepActionAsset* ActionAsset = DataprepAssetPtr->GetAction(ActionIndex))
							{
								if (ActionAsset->GetAppearance()->GroupId != INDEX_NONE)
								{
									NewGroupId = FMath::Max(NewGroupId, ActionAsset->GetAppearance()->GroupId);
								}
							}
						}

						NewGroupId++;

						// Apply groups to action assets
						{
							const FScopedTransaction Transaction(NSLOCTEXT("CollapseActions", "CollapseActions", "Group Actions"));

							DataprepAssetPtr->Modify();

							for (UDataprepActionAsset* Action : ActionsToCollapse)
							{
								Action->Modify();
								Action->GetAppearance()->Modify();
								Action->GetAppearance()->GroupId = NewGroupId;
								Action->GetAppearance()->bGroupIsEnabled = true;
							}
						}

						NotifyGraphChanged();
					});

					FText Label = FText::FromString("Group Actions");
					MenuBuilder->AddMenuEntry(FText::Format( LOCTEXT( "CollapseActionsAction", "{0}" ), Label ),
											  FText::Format( LOCTEXT( "CollapseActionsActionTooltip", "{0} steps/actions" ), Label ),
											  FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Pipeline.GroupActions"),
											  CollapseActions);
				}
			}
		}
		MenuBuilder->EndSection();

		// Add "create filter from selection" menu
		if ( bIsActionNode && Actions.Num() == 1 )
		{
			TSet< UObject* > AssetAndActorSelection = GetSelectedActorsAndAssets();

			if ( AssetAndActorSelection.Num() > 0 )
			{
				MenuBuilder->BeginSection( FName( TEXT("FilterSection") ), LOCTEXT("FilterSection", "Filter") );
				{
					FUIAction CreateFilterAction;
					CreateFilterAction.ExecuteAction.BindLambda( [this, Actions, AssetAndActorSelection]()
					{
						if ( UDataprepActionAsset* Action = Actions[0] )
						{
							CreateFilterFromSelection( Action, AssetAndActorSelection );
						}
					});
					
					MenuBuilder->AddMenuEntry(LOCTEXT("CreateFilter", "Create Filter From Selection"),
						LOCTEXT("CreateFilterTooltip", "Create filter from selected assets and actors"),
						FSlateIcon(),
						CreateFilterAction);
				}
			}
		}

		if ( FirstStepNode )
		{
			// Todo revisit this code the implementation is a bit ruff
			// If all the all to node are filters (also we might need a more robust code path ex: register extension base on the base class/type of the steps?)
			bool bIsSelectionOnlyFilters = true;
			bool bAreFilterFromSameAction = true;
			
			TArray<UDataprepParameterizableObject*> Filters;
			Filters.Reserve( SelectedNodes.Num() );

			const UDataprepActionAsset* ClickedAction = FirstStepNode->GetDataprepActionAsset();

			// Check the if the selection is all from the same action and only from step node
			auto OnEachAction = [&bAreFilterFromSameAction, &bIsSelectionOnlyFilters, ClickedAction] (UDataprepActionAsset* Action, UDataprepGraphActionNode* ActionNode) -> bool
				{
					const bool bIsFromActionNode = ActionNode != nullptr;

					if ( bIsFromActionNode )
					{
						bIsSelectionOnlyFilters = false;
						return true;
					}
					else
					{
						bAreFilterFromSameAction &= Action == ClickedAction;
					}
					return false;
				};

			// Check if the selection is only filters
			auto OnEachStepForFilterOnly = [&bIsSelectionOnlyFilters, &Filters](UDataprepParameterizableObject* StepObject, UDataprepActionStep* ActionStep) -> bool
				{
					if ( UDataprepFilter* Filter = Cast<UDataprepFilter>( StepObject ) )
					{
						Filters.Add( Filter );
					}
					else if( UDataprepFilterNoFetcher* FilterNF = Cast<UDataprepFilterNoFetcher>( StepObject ) )
					{
						Filters.Add( FilterNF );
					}
					else
					{
						bIsSelectionOnlyFilters = false;
					}
					return false;
				};

			DataprepGraphEditorUtils::ForEachActionAndStep( SelectedNodes, OnEachAction, OnEachStepForFilterOnly );

			if ( bIsSelectionOnlyFilters )
			{
				MenuBuilder->BeginSection( FName( TEXT("FilterSection") ), LOCTEXT("FilterSection", "Filter") );
				{
					FUIAction InverseFilterAction;
					InverseFilterAction.ExecuteAction.BindLambda( [Filters]()
						{
							FScopedTransaction Transaction( LOCTEXT("InverseFilterTransaction", "Inverse the filter") );
							for ( UDataprepParameterizableObject* ParamObj : Filters )
							{
								if( UDataprepFilter* Filter = Cast<UDataprepFilter>( ParamObj ) )
								{
									Filter->SetIsExcludingResult( !Filter->IsExcludingResult() );
								}
								else if( UDataprepFilterNoFetcher* FilterNF = Cast<UDataprepFilterNoFetcher>( ParamObj ) )
								{
									FilterNF->SetIsExcludingResult( !FilterNF->IsExcludingResult() );
								}
							}
						});
					MenuBuilder->AddMenuEntry(LOCTEXT("InverseFilter", "Inverse Filter(s) Selection"),
						LOCTEXT("InverseFilterTooltip", "Inverse the resulting selection from a filter"),
						FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Pipeline.InversePreviewFilter"),
						InverseFilterAction);

					if ( TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin() )
					{ 

						bool bAreAllFilterPreviewed = true;

						/**
						 * Check if the filters are the same as the previewed steps from the preview system
						 * Work with the assumption that the filters array contains only unique objects
						 */
						{
							for ( UDataprepParameterizableObject* Filter : Filters )
							{
								if ( !DataprepEditorPtr->IsPreviewingStep(Filter) )
								{
									bAreAllFilterPreviewed = false;
									break;
								}
							}
							if ( Filters.Num() != DataprepEditorPtr->GetCountOfPreviewedSteps() )
							{
								bAreAllFilterPreviewed = false;
							}
						}

						if ( bAreAllFilterPreviewed )
						{
							FUIAction ClearFilterPreview;
							ClearFilterPreview.ExecuteAction.BindLambda([DataprepEditorPtr]()
								{
									DataprepEditorPtr->ClearPreviewedObjects();
								});

							MenuBuilder->AddMenuEntry(LOCTEXT("ClearFilterPreview", "Clear the previewed Filter(s)"),
								LOCTEXT("ClearFilterPreviewTooltip", "Clear the columns of the scene preview and asset preview tabs of the Filter Preview."),
								FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Pipeline.ClearPreviewFilter"),
								ClearFilterPreview);
						}
						else
						{
							FUIAction SetFilterPreview;
							SetFilterPreview.CanExecuteAction.BindLambda( [bAreFilterFromSameAction]() { return bAreFilterFromSameAction; } );
							SetFilterPreview.ExecuteAction.BindLambda( [Filters, DataprepEditorPtr]()
								{
									TArray<UDataprepParameterizableObject*> ObjectsToObserve( Filters );
									DataprepEditorPtr->SetPreviewedObjects( MakeArrayView<UDataprepParameterizableObject*>( ObjectsToObserve.GetData(), ObjectsToObserve.Num() ) );
								});

							MenuBuilder->AddMenuEntry(LOCTEXT("SetFilterPreview", "Preview Filter(s)"),
								bAreFilterFromSameAction ? LOCTEXT("SetFilterPreviewTooltip", "Change the columns of the scene preview and asset preview tabs to display a preview of what the filters would select from the current scene") : LOCTEXT("SetFilterPreviewFailTooltip", "The filters can only be previewed if they are from the same action."),
								FSlateIcon(FDataprepEditorStyle::GetStyleSetName(), "DataprepEditor.Pipeline.PreviewFilter"),
								SetFilterPreview);
						}
					}
				}
				MenuBuilder->EndSection();
			}
		}

		return FActionMenuContent(MenuBuilder->MakeWidget());
	}

	// Create contextual for graph panel when on top of track node too
	return OnCreateActionMenu(CurrentGraph, GetPasteLocation2f(), TArray<UEdGraphPin*>(), true, SGraphEditor::FActionMenuClosed());
}

#undef LOCTEXT_NAMESPACE
