// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once




#include "EdGraphUtilities.h"
#include "GraphEditor.h"
#include "NodeFactory.h"

class UDataprepActionAsset;
enum class FDataprepAssetChangeType : uint8;
struct FDataprepSchemaAction;

class FDataprepEditor;
class FUICommandList;
class SDataprepGraphActionNode;
class SDataprepGraphTrackNode;
class SDataprepGraphTrackWidget;
class UDataprepAsset;
// #ueent_toremove: Temp code for the nodes development
class UBlueprint;
class UEdGraph;

class FDataprepGraphNodeFactory : public FGraphNodeFactory, public TSharedFromThis<FDataprepGraphNodeFactory>
{
public:
	virtual ~FDataprepGraphNodeFactory() = default;

	template<class DataprepEditorPtrType>
	FDataprepGraphNodeFactory(DataprepEditorPtrType&& InDataprepEditor)
		: DataprepEditor(Forward<DataprepEditorPtrType>(InDataprepEditor))
	{}

	/** Create a widget for the supplied node */
	virtual TSharedPtr<SGraphNode> CreateNodeWidget(UEdGraphNode* InNode) override;

private:
	TWeakPtr<FDataprepEditor> DataprepEditor;
};

class FDataprepGraphEditorNodeFactory : public FGraphPanelNodeFactory
{
public:
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* Node) const override;
};

/**
 * The SDataprepGraphEditor class is a specialization of SGraphEditor
 * to display and manipulate the actions of a Dataprep asset
 */
class SDataprepGraphEditor : public SGraphEditor
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphEditor)
		: _AdditionalCommands( static_cast<FUICommandList*>(nullptr) )
		, _GraphToEdit(nullptr)
	{}

	SLATE_ARGUMENT( TSharedPtr<FUICommandList>, AdditionalCommands )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, TitleBar )
		SLATE_ATTRIBUTE( FGraphAppearanceInfo, Appearance )
		SLATE_ARGUMENT( UEdGraph*, GraphToEdit )
		SLATE_ARGUMENT( FGraphEditorEvents, GraphEvents)
		SLATE_ARGUMENT( TSharedPtr<FDataprepEditor>, DataprepEditor )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepAsset* InDataprepAsset);

	// SWidget overrides
	virtual void CacheDesiredSize(float InLayoutScaleMultiplier) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget overrides

	// SGraphEditor overrides
	virtual void NotifyGraphChanged() override;
	// End of SGraphEditor overrides

	/** Called when a change has occurred in the set of the Dataprep asset's actions */
	void OnDataprepAssetActionChanged(UObject* InObject, FDataprepAssetChangeType ChangeType);

	/** Register/un-register the association between Dataprep's UEdGraph classes and SgraphNode classes */
	static void RegisterFactories();
	static void UnRegisterFactories();

	/** Set of methods necessary for copy/paste of action nodes*/
	void OnRenameNode();
	bool CanRenameNode() const;

	bool CanSelectAllNodes() const { return true; }

	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;

	void CopySelectedNodes();
	bool CanCopyNodes() const;

	void CutSelectedNodes();
	bool CanCutNodes() const;

	void PasteNodes();
	bool CanPasteNodes() const;

	void DuplicateNodes();
	bool CanDuplicateNodes() const;

	void DeleteSelectedDuplicatableNodes();
	/** End of set of methods necessary for copy/paste of action nodes*/

	/** Get the selection of viewport/outliner actors as well as assets from the asset preview tab */
	TSet<UObject*> GetSelectedActorsAndAssets();

	/** Set of callbacks to rename an action node */
	bool OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage);
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Callback to create contextual menu on graph panel */
	UE_DEPRECATED(5.6, "Please use the version of the function accepting FVector2f.")
	FActionMenuContent OnCreateActionMenu(UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);
	FActionMenuContent OnCreateActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Callback to create contextual menu for graph nodes in graph panel */
	FActionMenuContent OnCreateNodeOrPinMenu(UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging);

	/** Callback from the contextual menu to create any entries outside of the action collector */
	void OnCollectCustomActions(TArray<TSharedPtr<FDataprepSchemaAction>>& OutActions);

	void CreateFilterFromSelection(UDataprepActionAsset* InTargetAction, const TSet< UObject* >& InAssetAndActorSelection);

private:
	/** Recompute the layout of the displayed graph after a pan, resize and/or zoom */
	void UpdateLayout( const UE::Slate::FDeprecateVector2DParameter& LocalSize, const UE::Slate::FDeprecateVector2DParameter& Location, float ZoomAmount );

	void BuildCommandList();

private:
	/** When false, indicates the graph editor has not been drawn yet */
	mutable bool bIsComplete;

	/** Last size of the window displaying the graph's canvas */
	FDeprecateSlateVector2D LastLocalSize;

	/** Last location of the upper left corner of the visible section of the graph's canvas */
	FDeprecateSlateVector2D LastLocation;

	/** Last zoom factor applied to the graph's canvas */
	float LastZoomAmount;

	/** Dimensions of pipeline being displayed */
	FSlateRect WorkingArea;

	/** The dataprep asset associated with this graph */
	TWeakObjectPtr<UDataprepAsset> DataprepAssetPtr;

	/** Size of graph being displayed */
	mutable TWeakPtr<SDataprepGraphTrackNode> TrackGraphNodePtr;

	/** Command list associated with this graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	bool bCachedControlKeyDown;

	/** Padding used on the borders of the canvas */
	// #ueent_wip: Will be moved to the Dataprep editor's style
	static const float TopPadding;
	static const float BottomPadding;
	static const float HorizontalPadding;

	/** Factory to create the associated SGraphNode classes for Dataprep graph's UEdGraph classes */
	static TSharedPtr<FDataprepGraphEditorNodeFactory> NodeFactory;

	// The Dataprep editor that owns the graph
	TWeakPtr<FDataprepEditor> DataprepEditor;

	friend SDataprepGraphTrackNode;
};
