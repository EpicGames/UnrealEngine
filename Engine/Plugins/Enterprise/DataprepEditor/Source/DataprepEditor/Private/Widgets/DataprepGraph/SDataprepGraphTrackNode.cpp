// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepGraphTrackNode.h"

#include "DataprepEditorStyle.h"
#include "DataprepGraph/DataprepGraph.h"
#include "DataprepGraph/DataprepGraphActionNode.h"
#include "SchemaActions/DataprepDragDropOp.h"

#include "Widgets/DataprepGraph/SDataprepGraphActionNode.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "NodeFactory.h"
#include "ScopedTransaction.h"
#include "SGraphPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/SOverlay.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/WindowsHWrapper.h"
#endif

#define LOCTEXT_NAMESPACE "DataprepGraphEditor"

FVector2D SDataprepGraphTrackNode::TrackAnchor( 15.f, 40.f );

typedef TFunction<float(const FVector2D&, float)> DragCallback;

class SDataprepEmptyActionNode : public SVerticalBox
{
public:
	SLATE_BEGIN_ARGS(SDataprepEmptyActionNode){}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		SVerticalBox::Construct(SVerticalBox::FArguments());

		AddSlot()
		.AutoHeight()
		.Padding(FDataprepEditorStyle::GetMargin( "DataprepActionStep.Padding" ))
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.Padding(10.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				.MinDesiredWidth(250.f)
			]

			+ SOverlay::Slot()
			.Padding(0.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SImage)
				.ColorAndOpacity(FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Outer.Hovered"))
				.Image(FAppStyle::GetBrush( "Graph.StateNode.Body" ))
			]

			+ SOverlay::Slot()
			.Padding(1.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SImage)
				.ColorAndOpacity(FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Background.Hovered"))
				.Image(FAppStyle::GetBrush( "Graph.StateNode.Body" ))
			]

			+ SOverlay::Slot()
			.Padding(10.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataprepEmptyActionLabel", "+ Add Action"))
					.TextStyle( &FDataprepEditorStyle::GetWidgetStyle<FTextBlockStyle>( "DataprepAction.TitleTextStyle" ) )
					.ColorAndOpacity(FDataprepEditorStyle::GetColor("DataprepAction.EmptyStep.Text.Hovered"))
					.Justification(ETextJustify::Center)
				]
			]
		];
	}
};

class SDataprepGraphTrackWidget : public SConstraintCanvas
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphTrackWidget) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedPtr<SDataprepGraphTrackNode> InTrackNode);

	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		int32 MaxLayerId = SConstraintCanvas::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		// If dragging an action node, fake the cursor as it is hidden
		if(IndexDragged != INDEX_NONE)
		{
			static const FSlateBrush* GrabBrush = FDataprepEditorStyle::GetBrush(TEXT("DataprepEditor.SoftwareCursor_Grab"));
			static const FSlateBrush* HandBrush = FDataprepEditorStyle::GetBrush(TEXT("DataprepEditor.SoftwareCursor_Hand"));

			++MaxLayerId;
			const FSlateBrush* Brush = bMouseInScope ? GrabBrush : HandBrush;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				MaxLayerId,
				AllottedGeometry.ToPaintGeometry( Brush->ImageSize / AllottedGeometry.Scale, FSlateLayoutTransform(GetSoftwareCursorPosition() - ( Brush->ImageSize / 2 )) ),
				Brush);
#ifdef DEBUG_DRAG
			FVector2D LocalCursorPosition = GetTickSpaceGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());
			LocalCursorPosition.Y = TrackCursorOffset.Y - 30.f;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				MaxLayerId,
				AllottedGeometry.ToPaintGeometry( LocalCursorPosition - ( Brush->ImageSize / 2 ) , Brush->ImageSize / AllottedGeometry.Scale ),
				Brush);
#endif

		}

		return MaxLayerId;
	}
	// End of SWidget interface

	const FVector2D& GetWorkingSize() const
	{
		return WorkingSize;
	}

	FVector2D GetWorkingSize()
	{
		return WorkingSize;
	}

	FVector2D GetTrackArea()
	{
		return FVector2D(WorkingSize.X, TrackDesiredHeight);
	}

	FVector2D GetCanvasArea() const
	{
		return FVector2D(CanvasOffset.Right, CanvasOffset.Bottom);
	}

	FVector2D GetLineSize() const
	{
		return LineSize;
	}

	int32 GetHoveredActionNode(float Left, float Width);

	FVector2D GetNodeSize(const TSharedRef<SWidget>& Widget) const;

	void RefreshLayout();

	void OnStartNodeDrag(int32 Index);

	void OnNodeDragged(float Delta, DragCallback ComputePan);

	void OnEndNodeDrag(FVector2D& OutSpacePostion, int32& OutDraggedIndex, int32&OutDroppedIndex);

	FVector2D GetSoftwareCursorPosition() const
	{
		return FVector2D( DragSlot->GetOffset().Left + TrackCursorOffset.X, TrackCursorOffset.Y);
	}

	FVector2D GetAbsoluteSoftwareCursorPosition() const
	{
		return GetTickSpaceGeometry().LocalToAbsolute(GetSoftwareCursorPosition());
	}

	/**
	*/
	void UpdateLayoutOnDrag();

	/** Toggles display of action nodes between copy vs move drop modes */
	void OnControlKeyDown(bool bKeyDown, FVector2D& OutScreenSpacePosition);

	void UpdateDragIndicator(FVector2D MousePosition);

	void ResetDragIndicator()
	{
		DragIndicatorIndex = INDEX_NONE;

		FMargin SlotOffset = ActionSlots[SlotCount]->GetOffset();
		TrackSlots[SlotCount]->SetOffset( FMargin(SlotOffset.Left + SDataprepGraphActionNode::DefaultWidth * 0.5f - 16.f, LineTopPadding + TrackSlotTopOffset, 32, 32));
		TrackSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);

		UpdateLine(SlotCount-1);
	}

	FVector2D UpdateActionSlot(FMargin& SlotOffset, int32 SlotIndex, const TSharedRef<SWidget>& ActionNode, bool bIsEmpty = false)
	{
		DropSlots[SlotIndex]->SetOffset(SlotOffset);
		SlotOffset.Left += InterSlotSpacing;

		const float NodeTopPadding = LineTopPadding + NodeTopOffset;
		const float TrackSlotTopPadding = LineTopPadding + TrackSlotTopOffset;

		SConstraintCanvas::FSlot* ActionSlot = ActionSlots[SlotIndex];

		FVector2D Size = GetNodeSize(ActionNode);

		ActionSlot->SetOffset(FMargin(SlotOffset.Left, NodeTopPadding, Size.X, Size.Y));
		ActionSlot->AttachWidget(bIsEmpty ? SNew(SColorBlock).Color(FLinearColor::Transparent).Size( Size ) : ActionNode);

		TrackSlots[SlotIndex]->SetOffset( FMargin(SlotOffset.Left + Size.X * 0.5f - 16.f, TrackSlotTopPadding, 32, 32));
		TrackSlots[SlotIndex]->AttachWidget(SlotIndex == IndexHovered ? TrackSlotDrop.ToSharedRef() : TrackSlotRegular.ToSharedRef());

		SlotOffset.Left += Size.X;

		return Size + FVector2D(0.f, NodeTopPadding);
	}

	void OnActionsOrderChanged(const TArray<TSharedPtr<SDataprepGraphBaseActionNode>>& InActionNodes);

	void CreateHelperWidgets();

	void UpdateLine(int32 LastIndex);

	TWeakPtr<SDataprepGraphTrackNode> TrackNodePtr;
	TSharedPtr<SConstraintCanvas> TrackCanvas;
	TArray<TSharedRef<SWidget>> ActionNodes;
	TArray<SConstraintCanvas::FSlot*> DropSlots;
	TArray<SConstraintCanvas::FSlot*> ActionSlots;
	TArray<SConstraintCanvas::FSlot*> TrackSlots;
	SConstraintCanvas::FSlot* CanvasSlot;
	SConstraintCanvas::FSlot* LineSlot;

	FVector2D WorkingSize;
	FMargin CanvasOffset;
	FVector2D LineSize;
	int32 LastDropSlot;

	TSharedPtr<SWidget> DropIndicator;
	TSharedPtr<SColorBlock> DropFiller;
	TSharedPtr<SWidget> TrackSlotRegular;
	TSharedPtr<SWidget> TrackSlotDrop;
	TSharedPtr<SWidget> TrackSlotSelected;
	TSharedPtr<SDataprepEmptyActionNode> DummyAction;

	/** Number of visible slots. THis number can only change when dragging actions */
	int32 SlotCount;

	SConstraintCanvas::FSlot* DragSlot;
	int32 IndexDragged;
	int32 IndexHovered;
	int32 DragIndicatorIndex;
	FVector2D LastDragPosition;
	FVector2D AbscissaRange;
	FVector2D TrackOffset;
	FVector2D TrackCursorOffset;
	bool bIsCopying;
	bool bMouseInScope;

	static float TrackDesiredHeight;
	static float InterSlotSpacing;
	static float LineTopPadding;
	static float NodeTopOffset;
	static float TrackSlotTopOffset;

	friend SDataprepGraphTrackNode;
};

float SDataprepGraphTrackWidget::InterSlotSpacing = 16.f;
float SDataprepGraphTrackWidget::TrackDesiredHeight = 40.f;
float SDataprepGraphTrackWidget::LineTopPadding = 10.f;
float SDataprepGraphTrackWidget::NodeTopOffset = 25.f;
float SDataprepGraphTrackWidget::TrackSlotTopOffset = -11.f;

void SDataprepGraphTrackNode::Construct(const FArguments& InArgs, UDataprepGraphRecipeNode* InNode)
{
	NodeFactory = InArgs._NodeFactory;

	bNodeDragging = false;
	bSkipNextDragUpdate = false;
	bSkipRefreshLayout = false;
	bCursorLeftOnLeft = false;
	bCursorLeftOnRight = false;
	bNodeDragJustStarted = false;
	LastDragDirection = 0.f;

	SetCursor(EMouseCursor::Default);
	GraphNode = InNode;
	check(GraphNode);

	UDataprepGraph* DataprepGraph = Cast<UDataprepGraph>(GraphNode->GetGraph());
	check(DataprepGraph);

	DataprepAssetPtr = DataprepGraph->GetDataprepAsset();
	check(DataprepAssetPtr.IsValid());

	SNodePanel::SNode::FNodeSet NodeFilter;
	SGraphNode::MoveTo( UE::Slate::CastToVector2f(TrackAnchor), NodeFilter);

	InNode->SetWidget(SharedThis(this));

	UpdateGraphNode();
}

void SDataprepGraphTrackNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	if(UDataprepAsset* DataprepAsset = DataprepAssetPtr.Get())
	{
		TSharedPtr<SGraphPanel> GraphPanelPtr = OwnerGraphPanelPtr.Pin();

		UEdGraph* EdGraph = GraphNode->GetGraph();

		TSharedPtr<SDataprepGraphTrackNode> ParentTrackNodePtr = SharedThis(this);

		const int32 ActionsCount = DataprepAsset->GetActionCount();
		ActionNodes.Empty(ActionsCount);
		EdGraphActionNodes.Reset(ActionsCount);

		for(int32 NodeIndex = 0, ActionIndex = 0; ActionIndex < ActionsCount; ++NodeIndex, ++ActionIndex)
		{
			if(UDataprepActionAsset* ActionAsset = DataprepAsset->GetAction(ActionIndex))
			{
				UEdGraphNode* NewNode = nullptr;
				
				UDataprepActionAppearance* Appearance = ActionAsset->GetAppearance();

				if (Appearance->GroupId != INDEX_NONE)
				{
					const int32 CurrentGroupId = Appearance->GroupId;

					TArray<UDataprepActionAsset*> CurrentGroup;

					for(; ActionIndex < ActionsCount; ++ActionIndex)
					{
						if(UDataprepActionAsset* Action = DataprepAsset->GetAction(ActionIndex))
						{
							if(CurrentGroupId == Action->GetAppearance()->GroupId)
							{
								CurrentGroup.Add(Action);
							}
							else
							{
								--ActionIndex;
								break;
							}
						}
					}

					UDataprepGraphActionGroupNode* NewGroupNode = NewObject<UDataprepGraphActionGroupNode>(EdGraph, UDataprepGraphActionGroupNode::StaticClass(), NAME_None, RF_Transactional);
					NewGroupNode->CreateNewGuid();
					NewGroupNode->PostPlacedNewNode();
					NewGroupNode->NodePosX = 0;
					NewGroupNode->NodePosY = 0;

					NewGroupNode->Initialize(DataprepAssetPtr, CurrentGroup, NodeIndex);

					NewNode = NewGroupNode;
				}
				else
				{
					UDataprepGraphActionNode* NewActionNode = NewObject<UDataprepGraphActionNode>( EdGraph, UDataprepGraphActionNode::StaticClass(), NAME_None, RF_Transactional );
					NewActionNode->CreateNewGuid();
					NewActionNode->PostPlacedNewNode();
					NewActionNode->NodePosX = 0;
					NewActionNode->NodePosY = 0;

					NewActionNode->Initialize(DataprepAssetPtr, ActionAsset, NodeIndex);

					NewNode = NewActionNode;
				}

				TSharedPtr<SGraphNode> ActionGraphNode;
				if (NewNode)
				{
					EdGraphActionNodes.Emplace(NewNode);

					// #ueent_wip: Widget is created twice :-(
					if ( FGraphNodeFactory* GraphNodeFactor = NodeFactory.Get() )
					{
						ActionGraphNode = GraphNodeFactor->CreateNodeWidget( NewNode );
					}
					else
					{
						ActionGraphNode = FNodeFactory::CreateNodeWidget( NewNode );
					}
				}

				TSharedPtr< SDataprepGraphBaseActionNode > ActionWidgetPtr = StaticCastSharedPtr<SDataprepGraphBaseActionNode>( ActionGraphNode );
				if(SDataprepGraphBaseActionNode* ActionWidget = ActionWidgetPtr.Get())
				{
					if(GraphPanelPtr.IsValid())
					{
						ActionWidget->SetOwner(GraphPanelPtr.ToSharedRef());
					}

					ActionWidget->UpdateGraphNode();

					ActionWidget->SetParentTrackNode(ParentTrackNodePtr);

					ActionNodes.Add(StaticCastSharedPtr<SDataprepGraphBaseActionNode>( ActionGraphNode ));
				}
			}
		}
	}

	GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
		.Padding(0.f)
		.BorderBackgroundColor( FLinearColor( 0.3f, 0.3f, 0.3f, 1.0f ) )
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f)
			[
				SNew(SBox)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew( TrackWidgetPtr, SDataprepGraphTrackWidget, SharedThis(this) )
					]
				]
			]
		]
	];
}

void SDataprepGraphTrackNode::OnActionsOrderChanged()
{
	if(!DataprepAssetPtr.IsValid())
	{
		return;
	}

	TMap<UDataprepActionAsset*, TSharedPtr< SDataprepGraphBaseActionNode >> WidgetByAsset;

	for(TSharedPtr< SDataprepGraphBaseActionNode >& ActionWidgetPtr : ActionNodes)
	{
		if( SDataprepGraphBaseActionNode* ActionWidget = ActionWidgetPtr.Get() )
		{
			if (ActionWidget->IsActionGroup())
			{
				SDataprepGraphActionGroupNode* GroupNode = static_cast<SDataprepGraphActionGroupNode*>(ActionWidget);
				for (int32 ActionIndex = 0; ActionIndex < GroupNode->GetNumActions(); ++ActionIndex)
				{
					WidgetByAsset.Emplace(GroupNode->GetAction(ActionIndex), ActionWidgetPtr);
				}
			}
			else
			{
				WidgetByAsset.Emplace(ActionWidgetPtr->GetDataprepAction(), ActionWidgetPtr);
			}
		}
	}

	for(int32 NodeIndex = 0, ActionIndex = 0; ActionIndex < DataprepAssetPtr->GetActionCount(); ++NodeIndex, ++ActionIndex)
	{
		if(UDataprepActionAsset* ActionAsset = DataprepAssetPtr->GetAction(ActionIndex))
		{
			TSharedPtr< SDataprepGraphBaseActionNode >* ActionWidgetPtr = WidgetByAsset.Find(ActionAsset);

			UDataprepActionAppearance* Appearance = ActionAsset->GetAppearance();

			if (Appearance->GroupId != INDEX_NONE)
			{
				const int32 CurrentGroupId = Appearance->GroupId;
				TArray<UDataprepActionAsset*> CurrentGroup;

				for(; ActionIndex < DataprepAssetPtr->GetActionCount(); ++ActionIndex)
				{
					if(UDataprepActionAsset* Action = DataprepAssetPtr->GetAction(ActionIndex))
					{
						if(CurrentGroupId == Action->GetAppearance()->GroupId)
						{
							CurrentGroup.Add(Action);
						}
						else
						{
							--ActionIndex;
							break;
						}
					}
				}

				UDataprepGraphActionGroupNode* ActionNode = Cast<UDataprepGraphActionGroupNode>((*ActionWidgetPtr)->GetNodeObj());
				ensure(ActionNode != nullptr);
				ActionNode->Initialize( DataprepAssetPtr, CurrentGroup, NodeIndex );
			}
			else
			{
				UDataprepGraphActionNode* ActionNode = Cast<UDataprepGraphActionNode>((*ActionWidgetPtr)->GetNodeObj());
				ensure(ActionNode != nullptr);
				ActionNode->Initialize( DataprepAssetPtr, ActionAsset, NodeIndex);
			}

			(*ActionWidgetPtr)->UpdateExecutionOrder();
		}
	}

	// Reorder array according to new execution order
	ActionNodes.Sort([](const TSharedPtr<SDataprepGraphBaseActionNode> A, const TSharedPtr<SDataprepGraphBaseActionNode> B) { return A->GetExecutionOrder() < B->GetExecutionOrder(); });

	// Rearrange actions nodes along track
	TrackWidgetPtr->OnActionsOrderChanged(ActionNodes);

	// Invalidate graph panel for a redraw of all widgets
	OwnerGraphPanelPtr.Pin()->Invalidate(EInvalidateWidgetReason::ChildOrder | EInvalidateWidgetReason::Layout);
}

bool SDataprepGraphTrackNode::RefreshLayout()
{
	if(OwnerGraphPanelPtr.IsValid() && OwnerGraphPanelPtr.Pin()->GetManagedChildren()->Num() > 0)
	{
		ensure(TrackWidgetPtr.IsValid());

		if(!bSkipRefreshLayout && !bSkipNextDragUpdate)
		{
			TrackWidgetPtr->RefreshLayout();
		}

		bSkipRefreshLayout = false;

		return true;
	}

	return false;
}

void SDataprepGraphTrackNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	// Block track node to specific position
	SGraphNode::MoveTo( UE::Slate::CastToVector2f(TrackAnchor), NodeFilter, bMarkDirty);
}

const FSlateBrush* SDataprepGraphTrackNode::GetShadowBrush(bool bSelected) const
{
	return  FAppStyle::GetNoBrush();
}

FReply SDataprepGraphTrackNode::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SetCursor(EMouseCursor::Default);

	TSharedPtr<FDragDropActionNode> DragOperation = DragDropEvent.GetOperationAs<FDragDropActionNode>();
	if (DragOperation.IsValid())
	{
		return FReply::Handled().EndDragDrop();
	}

	TSharedPtr<FDataprepDragDropOp> DragActionNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if(DragActionNodeOp.IsValid())
	{
		int32 InsertIndex = TrackWidgetPtr->DragIndicatorIndex;
		TrackWidgetPtr->ResetDragIndicator();

		DragActionNodeOp->DoDropOnTrack(DataprepAssetPtr.Get(), InsertIndex);

		return FReply::Unhandled();
	}

	return SGraphNode::OnDrop(MyGeometry, DragDropEvent);
}

FReply SDataprepGraphTrackNode::OnDragOver(const FGeometry & MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetOp.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FDataprepDragDropOp> DragActionNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if(DragActionNodeOp.IsValid())
	{
		// It looks like there is a bug in the handling of DnD, OnDragOver is still called after leaving the track.
		// This seems to be related to the header of the graph panel. The event is going through
		// Quick fix: Detect onDragOver is called while cursor is not on top of track and manually leave.
		// #ueent_todo: Fix this at the root.
		if(!GetTickSpaceGeometry().IsUnderLocation(DragDropEvent.GetScreenSpacePosition()))
		{
			if(TrackWidgetPtr->DragIndicatorIndex != INDEX_NONE)
			{
				OnDragLeave(DragDropEvent);
			}
		}
		else
		{
			DragActionNodeOp->SetHoveredNode(GraphNode);
			TrackWidgetPtr->UpdateDragIndicator(DragDropEvent.GetScreenSpacePosition());
		}
	}

	return SGraphNode::OnDragOver(MyGeometry, DragDropEvent);
}

void SDataprepGraphTrackNode::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDataprepDragDropOp> DragActionNodeOp = DragDropEvent.GetOperationAs<FDataprepDragDropOp>();
	if(DragActionNodeOp.IsValid())
	{
		DragActionNodeOp->SetHoveredNode(nullptr);
		TrackWidgetPtr->ResetDragIndicator();
		TrackWidgetPtr->RefreshLayout();
	}

	SGraphNode::OnDragLeave(DragDropEvent);
}

void SDataprepGraphTrackNode::SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel)
{
	ensure(!OwnerGraphPanelPtr.IsValid());

	SGraphNode::SetOwner(OwnerPanel);

	for(TSharedPtr<SDataprepGraphBaseActionNode>& ActionNodePtr : ActionNodes)
	{
		if (ActionNodePtr.IsValid())
		{
			ActionNodePtr->SetOwner(OwnerPanel);
		}
	}
}

FSlateRect SDataprepGraphTrackNode::Update()
{
	FSlateRect WorkingArea;

	if(SDataprepGraphTrackWidget* TrackWidget = TrackWidgetPtr.Get())
	{
		const FVector2f WorkingSize = UE::Slate::CastToVector2f(TrackWidget->WorkingSize);
		const FVector2f TrackPosition = GetPosition2f().GetAbs();

		if(TrackWidget->DragIndicatorIndex == INDEX_NONE && !bNodeDragging)
		{
			RefreshLayout();

			SGraphPanel& GraphPanel = *OwnerGraphPanelPtr.Pin();

			// Determine canvas offset attribute in track widget's coordinates
			const FVector2f PanelSize = GraphPanel.GetTickSpaceGeometry().GetLocalSize() / GraphPanel.GetZoomAmount();

			const FVector2f TargetSize = FVector2f(FMath::Max(WorkingSize.X, PanelSize.X), FMath::Max(WorkingSize.Y, PanelSize.Y)) + TrackPosition + 20.f;

			FMargin& CanvasOffset = TrackWidgetPtr->CanvasOffset;

			CanvasOffset.Left = -TrackPosition.X - 10.f;
			CanvasOffset.Top = -TrackPosition.Y - 10.f;

			if(TargetSize.X > CanvasOffset.Right)
			{
				CanvasOffset.Right = TargetSize.X;
			}

			if(TargetSize.Y > CanvasOffset.Bottom)
			{
				CanvasOffset.Bottom = TargetSize.Y;
			}

			TrackWidgetPtr->CanvasSlot->SetOffset(CanvasOffset);
		}

		WorkingArea = FSlateRect(FVector2f::ZeroVector, WorkingSize + TrackPosition);
	}

	return WorkingArea;
}

void SDataprepGraphTrackNode::OnStartNodeDrag(const TSharedRef<SDataprepGraphBaseActionNode>& ActionNode)
{
	bNodeDragging = true;
	bSkipNextDragUpdate = false;
	bSkipRefreshLayout = false;
	bCursorLeftOnLeft = false;
	bCursorLeftOnRight = false;
	bNodeDragJustStarted = true;
	LastDragDirection = 0.f;

	TrackWidgetPtr->OnStartNodeDrag(ActionNode->GetExecutionOrder());

	TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();
	PlatformCursor->SetType(EMouseCursor::None);

	SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get();
	ensure(GraphPanel);

	GraphPanel->Invalidate(EInvalidateWidgetReason::Layout);
}

void SDataprepGraphTrackNode::OnEndNodeDrag(bool bDoDrop)
{
	bNodeDragging = false;
	bSkipNextDragUpdate = false;
	bSkipRefreshLayout = true;
	bCursorLeftOnLeft = false;
	bCursorLeftOnRight = false;
	bNodeDragJustStarted = false;
	LastDragDirection = 0.f;

	SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get();
	ensure(GraphPanel);

	GraphPanel->Invalidate(EInvalidateWidgetReason::Layout);

	FVector2D NewScreenSpacePosition;
	int32 DraggedNodeIndex;
	int32 DroppedNodeIndex;

	TrackWidgetPtr->OnEndNodeDrag(NewScreenSpacePosition, DraggedNodeIndex, DroppedNodeIndex);

	FSlateApplication::Get().GetPlatformCursor()->SetType(EMouseCursor::Default);
	FSlateApplication::Get().SetCursorPos( NewScreenSpacePosition );

	if(bDoDrop)
	{
		if(UDataprepAsset* DataprepAsset = GetDataprepAsset())
		{
			TSharedPtr<SDataprepGraphBaseActionNode> DraggedNode = ActionNodes[DraggedNodeIndex];
			FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
			const bool bCopyRequested = (ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown()) && !DraggedNode->IsActionGroup();

			FScopedTransaction Transaction( bCopyRequested ? LOCTEXT("DataprepGraphTrackNode_OnDropCopy", "Add/Insert action") : LOCTEXT("OnDropMove", "Move action") );
			bool bTransactionSuccessful = false;

			if (!DraggedNode->IsActionGroup())
			{
				if(bCopyRequested)
				{
					if(DroppedNodeIndex >= DataprepAsset->GetActionCount())
					{
						bTransactionSuccessful = DataprepAsset->AddAction(DraggedNode->GetDataprepAction()) != INDEX_NONE;
					}
					else
					{
						const int32 NewActionIndex = GetActionIndex(DroppedNodeIndex);
						bTransactionSuccessful = DataprepAsset->InsertAction(DraggedNode->GetDataprepAction(), NewActionIndex);
					}
				}
				else if( DroppedNodeIndex != DraggedNodeIndex)
				{
					const int32 DraggedActionIndex = GetActionIndex(DraggedNodeIndex);

					int32 DroppedActionIndex = DraggedActionIndex;
					if (DroppedNodeIndex > DraggedNodeIndex)
					{
						for (int32 NodeIndex = DraggedNodeIndex + 1; NodeIndex <= DroppedNodeIndex; ++NodeIndex)
						{
							DroppedActionIndex += GetNumActions(NodeIndex);
						}
					}
					else
					{
						DroppedActionIndex = GetActionIndex(DroppedNodeIndex);
					}
					
					bTransactionSuccessful = DataprepAsset->MoveAction(DraggedActionIndex, DroppedActionIndex);
				}
				else
				{
					RefreshLayout();
				}
			}
			else if( DroppedNodeIndex != DraggedNodeIndex )
			{
				// We are moving a group node: move all of its actions
				const int32 FirstDraggedActionIndex = GetActionIndex(DraggedNodeIndex);
				const int32 FirstDroppedActionIndex = GetActionIndex(DroppedNodeIndex);
				const int32 NumActions = GetNumActions(DraggedNodeIndex);
				int32 Offset;
				
				if (FirstDroppedActionIndex < FirstDraggedActionIndex)
				{
					Offset = FirstDroppedActionIndex - FirstDraggedActionIndex;
				}
				else
				{
					Offset = FirstDroppedActionIndex - (FirstDraggedActionIndex + NumActions - 1);
				}

				bTransactionSuccessful = DataprepAsset->MoveActions(FirstDraggedActionIndex, NumActions, Offset);
			}
			else
			{
				RefreshLayout();
			}

			if(!bTransactionSuccessful)
			{
				Transaction.Cancel();
			}
		}
	}
	else
	{
		RefreshLayout();
	}
}

bool SDataprepGraphTrackNode::OnNodeDragged( TSharedPtr<SDataprepGraphBaseActionNode>& ActionNodePtr, const FVector2D& DragScreenSpacePosition, const FVector2D& InScreenSpaceDelta)
{
	ensure(bNodeDragging);
	ensure(ActionNodePtr.IsValid());

	// Do not proceed until the user has actually move the mouse
	if(bNodeDragJustStarted && InScreenSpaceDelta.X == 0.f)
	{
		return TrackWidgetPtr->bMouseInScope;
	}

	if(InScreenSpaceDelta.X != 0.f)
	{
		LastDragDirection = FMath::Sign(InScreenSpaceDelta.X);
	}

	bNodeDragJustStarted = false;
	LastSetCursorPosition = TrackWidgetPtr->GetAbsoluteSoftwareCursorPosition();

	FVector2D ScreenSpaceDelta = InScreenSpaceDelta;

	// The mouse was out of the panel and is back
	if(bSkipNextDragUpdate)
	{
		const FVector2D PreviousCursorPosition = DragScreenSpacePosition + ScreenSpaceDelta;
		if(PreviousCursorPosition.Equals(LastSetCursorPosition, 0.9f) || DragScreenSpacePosition.Equals(LastSetCursorPosition, 0.9f))
		{
			return TrackWidgetPtr->bMouseInScope;
		}

		bSkipNextDragUpdate = false;
	}

	TSharedPtr<SGraphPanel> GraphPanel = GetOwnerPanel();
	const FGeometry& PanelGeometry = GraphPanel->GetTickSpaceGeometry();
	const FVector2D PanelSize = PanelGeometry.GetLocalSize();

	// If pointer is leaving panel's window, make sure it is visible and skip processing
	const float PanelMinX = PanelGeometry.AbsolutePosition.X;
	const float PanelMaxX = PanelMinX + PanelSize.X;

	if(DragScreenSpacePosition.X < PanelMinX || DragScreenSpacePosition.X > PanelMaxX)
	{
		if(TrackWidgetPtr->bMouseInScope)
		{
			TrackWidgetPtr->bMouseInScope = false;
			bCursorLeftOnLeft = DragScreenSpacePosition.X < PanelMinX;
			bCursorLeftOnRight = DragScreenSpacePosition.X > PanelMaxX;

			TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetCursorUser();
			// #ueent_todo: Is call to SetCursorVisibility useful?
			SlateUser->SetCursorVisibility(true);
			SlateUser->SetCursorPosition(FVector2D(DragScreenSpacePosition.X, LastSetCursorPosition.Y));
		}

		return TrackWidgetPtr->bMouseInScope;
	}

	const FGeometry& TrackGeometry = TrackWidgetPtr->GetTickSpaceGeometry();
	FVector2D DragTrackPosition = TrackGeometry.AbsoluteToLocal(DragScreenSpacePosition);
	FVector2D SoftwareCursorPos = TrackWidgetPtr->GetSoftwareCursorPosition();

	// Mouse pointer was out of panel's screen space, make sure to hide mouse cursor
	TrackWidgetPtr->bMouseInScope = true;

	// When cursor was out of scope, do not process drag until cursor reaches marker
	if(bCursorLeftOnLeft || bCursorLeftOnRight)
	{
		const bool bStillOutOfScope = (bCursorLeftOnLeft && DragTrackPosition.X < SoftwareCursorPos.X) ||
									  (bCursorLeftOnRight && DragTrackPosition.X > SoftwareCursorPos.X);

		// Still has not passed track marker, skip drag processing;
		if(bStillOutOfScope)
		{
			return TrackWidgetPtr->bMouseInScope;
		}

		// Adjust delta to only contain move from or to track marker's position
		ScreenSpaceDelta.X = bCursorLeftOnLeft ? DragTrackPosition.X + ScreenSpaceDelta.X - SoftwareCursorPos.X : DragTrackPosition.X - SoftwareCursorPos.X;

		bCursorLeftOnLeft = false;
		bCursorLeftOnRight = false;
	}

	const float ZoomAmount = GraphPanel->GetZoomAmount();
	const FVector2D TrackSize = TrackWidgetPtr->GetWorkingSize() * ZoomAmount;
	float PanAmount = 0.f;

	DragCallback LocalComputePanAmount = [this, GraphPanel, ZoomAmount, TrackSize, PanelSize, DragTrackPosition, &TrackGeometry, &PanAmount](const FVector2D& SlotRange, float Delta) -> float
	{
		if(TrackSize.X > PanelSize.X)
		{
			const float MaxPanSpeed = /*ZoomAmount < 1.f ? 10.f : */10.f / ZoomAmount ;
			FVector2D ScreenSpacePosition = TrackGeometry.LocalToAbsolute(FVector2D(SlotRange.X, DragTrackPosition.Y));
			PanAmount = this->ComputePanAmount(ScreenSpacePosition, MaxPanSpeed).X;

			ScreenSpacePosition = TrackGeometry.LocalToAbsolute(FVector2D(SlotRange.Y, DragTrackPosition.Y));
			PanAmount += this->ComputePanAmount(ScreenSpacePosition, MaxPanSpeed).X;

			// Adjust pan amount to boundaries of the track widget
			const float CurrentViewOffsetX = GraphPanel->GetViewOffset().X * ZoomAmount;
			const float NewViewOffsetX = (GraphPanel->GetViewOffset().X + PanAmount) * ZoomAmount;
			if(NewViewOffsetX < 0.f)
			{
				PanAmount = -GraphPanel->GetViewOffset().X;
			}
			else if(NewViewOffsetX + PanelSize.X > TrackSize.X)
			{
				PanAmount = (TrackSize.X - (CurrentViewOffsetX + PanelSize.X)) / ZoomAmount;
				if(PanAmount < 0.f)
				{
					PanAmount = 0.f;
				}
			}

			// Preventing panning from going at the opposite of current mouse move
			if(PanAmount * LastDragDirection < 0.f)
			{
				PanAmount = 0.f;
			}
		}

		return PanAmount;
	};
	
	TrackWidgetPtr->OnNodeDragged(ScreenSpaceDelta.X / ZoomAmount, LocalComputePanAmount);

	if(PanAmount != 0.f)
	{
		bSkipRefreshLayout = true;
		GraphPanel->RestoreViewSettings(GraphPanel->GetViewOffset() + (FVector2D(PanAmount,0.f) / ZoomAmount), ZoomAmount);
	}

	return true;
}

void SDataprepGraphTrackNode::OnControlKeyChanged(bool bControlKeyDown)
{
	if(bNodeDragging)
	{
		FVector2D NewScreenSpacePosition;
		TrackWidgetPtr->OnControlKeyDown(bControlKeyDown, NewScreenSpacePosition);

		if(!bControlKeyDown && NewScreenSpacePosition != FVector2D::ZeroVector)
		{
			bSkipNextDragUpdate = true;
			FSlateApplication::Get().SetCursorPos( NewScreenSpacePosition );
		}
	}
}

FVector2D SDataprepGraphTrackNode::ComputePanAmount(const FVector2D& InScreenSpacePosition, float MaxPanSpeed)
{
	// Note: Code copied from SNodePanel::RequestDeferredPan and its subsequent calls. Calling SNodePanel::RequestDeferredPan is not an option
	//		 since it changes the view offset during the call to OnPaint which bypasses the adjustments done by the SDataprepGraphEditor in
	//		 SDataprepGraphEditor::Tick 
	FVector2D PanAmount(FVector2D::ZeroVector);

	if(SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get())
	{
		// How quickly to ramp up the pan speed as the user moves the mouse further past the edge of the graph panel.
		constexpr float EdgePanSpeedCoefficient = 2.f;
		constexpr float EdgePanSpeedPower = 0.6f;
		// Never pan faster than this - probably not really required since we raise to a power of 0.6
		// Start panning before we reach the edge of the graph panel.
		constexpr float EdgePanForgivenessZone = 30.0f;

		//const FVector2D PanAmount = ComputeEdgePanAmount( GraphPanel->GetTickSpaceGeometry(), InScreenSpacePosition ) / GraphPanel->GetZoomAmount();
		const FGeometry& PanelGeometry = GraphPanel->GetTickSpaceGeometry();
		const FVector2D PanelPosition = PanelGeometry.AbsoluteToLocal( InScreenSpacePosition );
		const FVector2D PanelSize = PanelGeometry.GetLocalSize();
		const float ZoomAmount = GraphPanel->GetZoomAmount();

		if ( PanelPosition.X <= EdgePanForgivenessZone )
		{
			PanAmount.X = FMath::Max( -MaxPanSpeed, EdgePanSpeedCoefficient * -FMath::Pow(EdgePanForgivenessZone - PanelPosition.X, EdgePanSpeedPower) );
		}
		else if( PanelPosition.X >= PanelSize.X - EdgePanForgivenessZone )
		{
			PanAmount.X = FMath::Min( MaxPanSpeed, EdgePanSpeedCoefficient * FMath::Pow(PanelPosition.X - PanelSize.X + EdgePanForgivenessZone, EdgePanSpeedPower) );
		}

		if ( PanelPosition.Y <= EdgePanForgivenessZone )
		{
			PanAmount.Y = FMath::Max( -MaxPanSpeed, EdgePanSpeedCoefficient * -FMath::Pow(EdgePanForgivenessZone - PanelPosition.Y, EdgePanSpeedPower) );
		}
		else if( PanelPosition.Y >= PanelSize.Y - EdgePanForgivenessZone )
		{
			PanAmount.Y = FMath::Min( MaxPanSpeed, EdgePanSpeedCoefficient * FMath::Pow(PanelPosition.Y - PanelSize.Y + EdgePanForgivenessZone, EdgePanSpeedPower) );
		}
	}

	return PanAmount;
}

void SDataprepGraphTrackNode::RequestViewportPan(const FVector2D& InScreenSpacePosition)
{
	if(SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get())
	{
		FVector2D PanAmount = ComputePanAmount(InScreenSpacePosition);

		if(PanAmount != FVector2D::ZeroVector)
		{
			const float ZoomAmount = GraphPanel->GetZoomAmount();

			bSkipRefreshLayout = true;
			GraphPanel->RestoreViewSettings(GraphPanel->GetViewOffset() + (PanAmount / ZoomAmount), ZoomAmount);
		}
	}
}

void SDataprepGraphTrackNode::RequestRename(const UEdGraphNode* Node)
{
	if(SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get())
	{
		int32 NodeIndex = INDEX_NONE;
		
		if(const UDataprepGraphActionNode* ActionEdNode = Cast<UDataprepGraphActionNode>(Node))
		{
			NodeIndex = ActionEdNode->GetExecutionOrder();
		}
		else if (const UDataprepGraphActionGroupNode* ActionGroupEdNode = Cast<UDataprepGraphActionGroupNode>(Node))
		{
			NodeIndex = ActionGroupEdNode->GetExecutionOrder();
		}

		if(ActionNodes.IsValidIndex(NodeIndex))
		{
			TSharedPtr<SDataprepGraphBaseActionNode>& ActionNode = ActionNodes[NodeIndex];

			if(ActionNode.IsValid() && !GraphPanel->HasMouseCapture())
			{
				FSlateRect TitleRect = ActionNode->GetTitleRect();
				const FVector2D TopLeft = FVector2D( TitleRect.Left, TitleRect.Top );
				const FVector2D BottomRight = FVector2D( TitleRect.Right, TitleRect.Bottom );

				bool bTitleVisible = GraphPanel->IsRectVisible( TopLeft, BottomRight );
				if( !bTitleVisible )
				{
					bTitleVisible = GraphPanel->JumpToRect( TopLeft, BottomRight );
				}

				if( bTitleVisible )
				{
					ActionNode->RequestRename();
					//GraphPanel->JumpToNode(Node, false, true);
					ActionNode->ApplyRename();
				}
			}
		}
	}
}

void SDataprepGraphTrackNode::UpdateProxyNode(TSharedRef<SDataprepGraphBaseActionNode> ActioNodePtr, const FVector2D& ScreenSpacePosition)
{
	if(SGraphPanel* GraphPanel = OwnerGraphPanelPtr.Pin().Get())
	{
		FVector2D PanelPosition = GraphPanel->GetTickSpaceGeometry().AbsoluteToLocal(ScreenSpacePosition);
		FVector2D GraphPosition = PanelPosition / GraphPanel->GetZoomAmount() + GraphPanel->GetViewOffset();
		ActioNodePtr->UpdateProxyNode(GraphPosition);
	}
}

int32 SDataprepGraphTrackNode::GetActionIndex(int32 InNodeIndex) const
{
	int32 ActionIndex = 0;

	for (int32 NodeIndex = 0; NodeIndex < InNodeIndex; ++NodeIndex)
	{
		ActionIndex += GetNumActions(NodeIndex);
	}

	return ActionIndex;
}

int32 SDataprepGraphTrackNode::GetNumActions(int32 InNodeIndex) const
{
	if (InNodeIndex >= ActionNodes.Num())
	{
		return 0;
	}

	TSharedPtr<SDataprepGraphBaseActionNode> NodePtr = ActionNodes[InNodeIndex];
	if (SDataprepGraphBaseActionNode* Node = NodePtr.Get())
	{
		if (Node->IsActionGroup())
		{
			return static_cast<SDataprepGraphActionGroupNode*>(Node)->GetNumActions();
		}
		else
		{
			return Node->GetDataprepAction() != nullptr ? 1 : 0;
		}
	}

	return 0;
}

void SDataprepGraphTrackWidget::Construct(const FArguments& InArgs, TSharedPtr<SDataprepGraphTrackNode> InTrackNode)
{
	TrackNodePtr = InTrackNode;
	ensure(TrackNodePtr.IsValid());
	bMouseInScope = false;

	SConstraintCanvas::Construct( SConstraintCanvas::FArguments());

	CreateHelperWidgets();

	const FLinearColor TrackColor = FDataprepEditorStyle::GetColor( "DataprepAction.BackgroundColor" );

	AddSlot()
	.Anchors( FAnchors( 0.f, 0.f, 0.f, 0.f ) )
	.Offset( FMargin(-25.f, -25.f, 0.f, 0.f) )
	.AutoSize(false)
	.Alignment(FVector2D::ZeroVector)
	.ZOrder(0)
	.Expose(CanvasSlot)
	[
		SNew(SColorBlock)
		.Color( FDataprepEditorStyle::GetColor("Dataprep.Background.Black") )
		.Size( TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP( this, &SDataprepGraphTrackWidget::GetCanvasArea ) ) )
	];

	AddSlot()
	.Anchors( FAnchors( 0.f, 0.f, 0.f, 0.f ) )
	.Offset( FMargin(InterSlotSpacing +  3.f, LineTopPadding - 2.f, 0.f, 0.f) )
	.AutoSize(true)
	.Alignment(FVector2D::ZeroVector)
	.ZOrder(0)
	.Expose(LineSlot)
	[
		SNew(SColorBlock)
		.Color( TrackColor )
		.Size( TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP( this, &SDataprepGraphTrackWidget::GetLineSize ) ) )
	];

	AddSlot()
	.Anchors( FAnchors( 0.f, 0.f, 1.f, 0.f ) )
	.Offset( FMargin(0.f, 10.f, 0.f, 0.f) )
	.AutoSize(true)
	.Alignment(FVector2D::ZeroVector)
	.ZOrder(1)
	[
		SNew(SColorBlock)
		.Color(FLinearColor::Transparent)
		.Size( TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP( this, &SDataprepGraphTrackWidget::GetTrackArea ) ) )
	];

	LastDropSlot = INDEX_NONE;

	TSharedPtr<SColorBlock> NullNode = SNew(SColorBlock)
	.Color( FLinearColor::Transparent )
	.Size( FVector2D(SDataprepGraphActionNode::DefaultWidth, TrackDesiredHeight) );

	const FVector2D DefaultNodeSize(SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultHeight);

	CanvasOffset = FMargin(FVector4(FVector2D::ZeroVector, DefaultNodeSize));

	const float NodeTopPadding = LineTopPadding + NodeTopOffset;
	const float TrackSlotTopPadding = LineTopPadding + TrackSlotTopOffset;

	UDataprepAsset* DataprepAsset = InTrackNode->GetDataprepAsset();
	SlotCount = DataprepAsset ? InTrackNode->ActionNodes.Num() : 0;

	DropSlots.Reserve(SlotCount + 2);
	ActionSlots.Reserve(SlotCount + 1);
	TrackSlots.Reserve(SlotCount + 2);
	ActionNodes.Reserve(SlotCount);

	float LeftOffset = 0.f;
	if(DataprepAsset)
	{
		TArray<TSharedPtr<SDataprepGraphBaseActionNode>>& InActionNodes = InTrackNode->ActionNodes;

		for(int32 Index = 0; Index < SlotCount; ++Index)
		{
			// Add drop slot ahead of action node
			{
				SConstraintCanvas::FSlot* Slot = nullptr;
				AddSlot()
				.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
				.Offset( FMargin(LeftOffset, NodeTopPadding + 5.f, InterSlotSpacing, TrackDesiredHeight ) )
				.Alignment(FVector2D::ZeroVector)
				.ZOrder(2)
				.Expose(Slot)
				[
					DropFiller.ToSharedRef()
				];
				DropSlots.Add( Slot );
			}
			LeftOffset += InterSlotSpacing;

			const FVector2D Size = DefaultNodeSize;

			// Add slot to host of action node
			{
				TSharedRef<SWidget>& ActionNode = ActionNodes.Add_GetRef(InActionNodes[Index].IsValid() ? InActionNodes[Index]->AsShared() : NullNode->AsShared());

				SConstraintCanvas::FSlot* Slot = nullptr;
				AddSlot()
				.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
				.Offset( FMargin(LeftOffset, NodeTopPadding, Size.X, Size.Y ) )
				.Alignment(FVector2D::ZeroVector)
				.AutoSize(true)
				.ZOrder(2)
				.Expose(Slot)
				[
					ActionNode
				];
				ActionSlots.Add( Slot );
			}

			// Add track slot which is at the middle of the corresponding action
			{
				SConstraintCanvas::FSlot* Slot = nullptr;
				AddSlot()
				.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
				.Offset( FMargin(LeftOffset + SDataprepGraphActionNode::DefaultWidth * 0.5f + 16.f, TrackSlotTopPadding, 32, 32 ) )
				.Alignment(FVector2D::ZeroVector)
				.AutoSize(true)
				.ZOrder(2)
				.Expose(Slot)
				[
					TrackSlotRegular.ToSharedRef()
				];
				TrackSlots.Add( Slot );
			}
			LeftOffset += Size.X;

			CanvasOffset.Bottom = CanvasOffset.Bottom < Size.Y ? Size.Y : CanvasOffset.Bottom;
		}
	}
	else
	{
		SlotCount = 0;
	}

	// Add drop slot at end of track
	{
		SConstraintCanvas::FSlot* Slot = nullptr;
		AddSlot()
		.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
		.Offset( FMargin(LeftOffset, NodeTopPadding + 5.f, InterSlotSpacing, TrackDesiredHeight ) )
		.Alignment(FVector2D::ZeroVector)
		.ZOrder(2)
		.Expose(Slot)
		[
			SNullWidget::NullWidget
		];
		DropSlots.Add( Slot );
		LeftOffset += InterSlotSpacing;
	}

	// Add dummy action slot at end of track
	{
		SConstraintCanvas::FSlot* Slot = nullptr;
		AddSlot()
		.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
		.Offset( FMargin(LeftOffset, TrackDesiredHeight, SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultWidth ) )
		.Alignment(FVector2D::ZeroVector)
		.AutoSize(true)
		.ZOrder(2)
		.Expose(Slot)
		[
			SNullWidget::NullWidget
		];
		ActionSlots.Add( Slot );
	}

	// Add track slot which is at the middle of the corresponding action
	{
		SConstraintCanvas::FSlot* Slot = nullptr;
		AddSlot()
		.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
		.Offset( FMargin(LeftOffset + SDataprepGraphActionNode::DefaultWidth * 0.5f + 16.f, TrackSlotTopPadding, 32, 32 ) )
		.Alignment(FVector2D::ZeroVector)
		.AutoSize(true)
		.ZOrder(2)
		.Expose(Slot)
		[
			SNullWidget::NullWidget
		];
		TrackSlots.Add( Slot );
	}
	LeftOffset += SDataprepGraphActionNode::DefaultWidth;

	// Add drop slot at end of track
	{
		SConstraintCanvas::FSlot* Slot = nullptr;
		AddSlot()
		.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
		.Offset( FMargin(LeftOffset, NodeTopPadding + 5.f, InterSlotSpacing, TrackDesiredHeight ) )
		.Alignment(FVector2D::ZeroVector)
		.ZOrder(2)
		.Expose(Slot)
		[
			SNullWidget::NullWidget
		];
		DropSlots.Add( Slot );
		LeftOffset += InterSlotSpacing;
	}

	// Add slot to host dragged node???
	AddSlot()
	.Anchors( FAnchors( 0., 0.f, 0.f, 0.f ) )
	.Offset( FMargin(LeftOffset, 0.f, SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultWidth ) )
	.Alignment(FVector2D::ZeroVector)
	.AutoSize(true)
	.ZOrder(3)
	.Expose(DragSlot)
	[
		SNullWidget::NullWidget
	];

	WorkingSize = FVector2D::ZeroVector;
	CanvasOffset.Right = LeftOffset;

	IndexHovered = INDEX_NONE;
	IndexDragged = INDEX_NONE;
	DragIndicatorIndex = INDEX_NONE;
}

void SDataprepGraphTrackWidget::CreateHelperWidgets()
{
	const FLinearColor TrackColor = FDataprepEditorStyle::GetColor( "DataprepAction.BackgroundColor" );
	const FLinearColor DragAndDropColor = FDataprepEditorStyle::GetColor( "DataprepAction.DragAndDrop" );

	DropIndicator = SNew(SBorder)
		.BorderBackgroundColor( DragAndDropColor )
		.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
		.HAlign(EHorizontalAlignment::HAlign_Center)
		[
			SNew(SBox)
			.HeightOverride(TrackDesiredHeight - 2.f)
			.WidthOverride(InterSlotSpacing - 2.f)
		];

	DropFiller = SNew(SColorBlock)
		.Color(FLinearColor::Transparent)
		.Size( FVector2D(InterSlotSpacing, TrackDesiredHeight) );

	TrackSlotRegular = SNew(SOverlay)
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SBox)
			.WidthOverride(22.f)
			.HeightOverride(22.f)
		]
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(TrackColor)
			.Image(FDataprepEditorStyle::GetBrush( "DataprepEditor.TrackNode.Slot" ))
		];

	TrackSlotDrop = SNew(SOverlay)
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SBox)
			.WidthOverride(22.f)
			.HeightOverride(22.f)
		]
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(DragAndDropColor)
			.Image(FDataprepEditorStyle::GetBrush( "DataprepEditor.TrackNode.Slot" ))
		];

	DummyAction = SNew(SDataprepEmptyActionNode);
}

int32 SDataprepGraphTrackWidget::GetHoveredActionNode(float LeftCorner, float Width)
{
	const float CenterAbscissa = LeftCorner + Width * 0.5f;

	// Compute origin abscissa of last action node on track
	for(int32 Index = 0; Index < SlotCount; ++Index)
	{
		const FMargin SlotOffset = DropSlots[Index]->GetOffset();
		const float RelativeAbscissa = CenterAbscissa - (SlotOffset.Left + SlotOffset.Right);
		const float ActionSlotWidth = ActionSlots[Index]->GetOffset().Right;
		if(RelativeAbscissa > -SlotOffset.Right && RelativeAbscissa <= ActionSlotWidth)
		{
			return Index;
		}
	}

	return SlotCount - 1;
}

void SDataprepGraphTrackWidget::OnActionsOrderChanged(const TArray<TSharedPtr<SDataprepGraphBaseActionNode>>& InActionNodes)
{
	ensure(InActionNodes.Num() == ActionNodes.Num());

	for(int32 Index = 0; Index < ActionNodes.Num(); ++Index)
	{
		ActionNodes[Index] = InActionNodes[Index]->AsShared();
	}

	RefreshLayout();
}

void SDataprepGraphTrackWidget::RefreshLayout()
{
	const float NodeTopPadding = LineTopPadding + NodeTopOffset;

	LineSize = FVector2D::ZeroVector;
	WorkingSize = FVector2D(SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultHeight);
	FMargin DropSlotOffset = DropSlots[0]->GetOffset();

	float MaxWidth = SDataprepGraphActionNode::DefaultWidth;

	for(int32 Index = 0; Index < SlotCount; ++Index)
	{
		// Update action's proxy node registered to graph panel
		const FVector2D LocalNodePosition(DropSlotOffset.Left + InterSlotSpacing, NodeTopPadding);
		const FVector2D AbsoluteNodePosition = GetTickSpaceGeometry().LocalToAbsolute(LocalNodePosition);
		TrackNodePtr.Pin()->UpdateProxyNode(StaticCastSharedRef<SDataprepGraphBaseActionNode>(ActionNodes[Index]), AbsoluteNodePosition);

		const FVector2D Size = UpdateActionSlot(DropSlotOffset, Index, ActionNodes[Index]);

		if(WorkingSize.Y < Size.Y)
		{
			WorkingSize.Y = Size.Y;
		}

		if(MaxWidth < Size.X)
		{
			MaxWidth = Size.X;
		}
	}

	DropSlots[SlotCount]->SetOffset(DropSlotOffset);
	DropSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);
	DropSlotOffset.Left += InterSlotSpacing;

	ActionSlots[SlotCount]->SetOffset( FMargin(DropSlotOffset.Left, NodeTopPadding, 0., 0.));
	ActionSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);

	TrackSlots[SlotCount]->SetOffset( FMargin(DropSlotOffset.Left + SDataprepGraphActionNode::DefaultWidth * 0.5f - 11.f, LineTopPadding + TrackSlotTopOffset, 22, 22));
	TrackSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);

	WorkingSize.X = DropSlotOffset.Left + SDataprepGraphActionNode::DefaultWidth + InterSlotSpacing;
	WorkingSize.Y += NodeTopPadding;

	UpdateLine(SlotCount-1);

	Invalidate(EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::ChildOrder);
}

void SDataprepGraphTrackWidget::UpdateLayoutOnDrag()
{
	FMargin SlotOffset = DropSlots[0]->GetOffset();

	if(bIsCopying)
	{
		for(int32 Index = 0; Index < IndexHovered; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index]);
		}

		UpdateActionSlot(SlotOffset, IndexHovered, DragSlot->GetWidget(), true);

		for(int32 Index = IndexHovered + 1; Index < SlotCount; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index-1]);
		}
	}
	else if(IndexHovered < IndexDragged)
	{
		for(int32 Index = 0; Index < IndexHovered; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index]);
		}

		UpdateActionSlot(SlotOffset, IndexHovered, DragSlot->GetWidget(), true);

		for(int32 Index = IndexHovered + 1; Index <= IndexDragged; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index-1]);
		}

		for(int32 Index = IndexDragged+1; Index < SlotCount; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index]);
		}
	}
	else
	{
		for(int32 Index = 0; Index < IndexDragged; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index]);
		}

		for(int32 Index = IndexDragged; Index < IndexHovered; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index+1]);
		}

		UpdateActionSlot(SlotOffset, IndexHovered, DragSlot->GetWidget(), true);

		for(int32 Index = IndexHovered + 1; Index < SlotCount; ++Index)
		{
			UpdateActionSlot(SlotOffset, Index, ActionNodes[Index]);
		}
	}

	const FMargin DragSlotOffset = DragSlot->GetOffset();
	const FMargin HoveredDropSlotOffset = DropSlots[IndexHovered]->GetOffset();

	if(IndexHovered > 0 && DragSlotOffset.Left < HoveredDropSlotOffset.Left)
	{
		SConstraintCanvas::FSlot* OverlappedSlot = ActionSlots[IndexHovered-1];
		FMargin OverlappedSlotOffset = OverlappedSlot->GetOffset();

		const float OverlappedActionSlotLeft = HoveredDropSlotOffset.Left - OverlappedSlotOffset.Right;
		const float Delta =  HoveredDropSlotOffset.Left - DragSlotOffset.Left;

		const float MaximalLeftOffset = HoveredDropSlotOffset.Left + HoveredDropSlotOffset.Right;
		const float NewLeftOffset = OverlappedActionSlotLeft + Delta;

		if(NewLeftOffset < MaximalLeftOffset)
		{
			OverlappedSlotOffset.Left = NewLeftOffset;
			OverlappedSlot->SetOffset(OverlappedSlotOffset);
		}
	}
	else if(IndexHovered < (SlotCount-1) && DragSlotOffset.Left > (HoveredDropSlotOffset.Left + HoveredDropSlotOffset.Right) )
	{
		SConstraintCanvas::FSlot* OverlappedSlot = ActionSlots[IndexHovered+1];
		FMargin OverlappedSlotOffset = OverlappedSlot->GetOffset();
		FMargin OverlappedDropSlotOffset = DropSlots[IndexHovered+1]->GetOffset();

		const float OverlappedActionSlotLeft = OverlappedDropSlotOffset.Left + OverlappedDropSlotOffset.Right;
		const float MinimalLeftOffset = HoveredDropSlotOffset.Left + HoveredDropSlotOffset.Right;
		const float Delta = DragSlotOffset.Left - MinimalLeftOffset;

		float NewLeftOffset = OverlappedActionSlotLeft - Delta;
		if(NewLeftOffset > MinimalLeftOffset)
		{
			OverlappedSlotOffset.Left = NewLeftOffset;
			OverlappedSlot->SetOffset(OverlappedSlotOffset);
		}
	}

	UpdateLine(SlotCount-1);

	Invalidate(EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Paint);
}

void SDataprepGraphTrackWidget::OnStartNodeDrag(int32 InIndexDragged)
{
	if(!DragSlot)
	{
		ensure(false);
		return;
	}

	ensure(TrackNodePtr.IsValid());
	ensure(ActionSlots.IsValidIndex(InIndexDragged));

	IndexDragged = InIndexDragged;
	IndexHovered = InIndexDragged;
	DragIndicatorIndex = INDEX_NONE;

	FMargin Offset = ActionSlots[IndexDragged]->GetOffset();
	DragSlot->SetOffset(Offset);
	DragSlot->AttachWidget(ActionNodes[IndexDragged]);

	const FVector2D LocalCursorPosition = GetTickSpaceGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());
	TrackCursorOffset = FVector2D(LocalCursorPosition.X - Offset.Left, LocalCursorPosition.Y);

	AbscissaRange.X = ActionSlots[0]->GetOffset().Left;
	// AbscissaRange.Y will be properly update in the call to OnControlKeyDown
	AbscissaRange.Y = AbscissaRange.X;

	TSharedPtr<SDataprepGraphBaseActionNode> ActionNode = StaticCastSharedRef<SDataprepGraphBaseActionNode>(ActionNodes[IndexDragged]);

	const FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	bIsCopying = !((ModifierKeyState.IsControlDown() || ModifierKeyState.IsCommandDown()) && !ActionNode->IsActionGroup());

	FVector2D NewScreenSpacePosition;
	OnControlKeyDown(!bIsCopying, NewScreenSpacePosition);
}

void SDataprepGraphTrackWidget::OnNodeDragged(float DragDelta, DragCallback ComputePanAmount)
{
	if(!DragSlot)
	{
		ensure(false);
		return;
	}

	FMargin DragSlotOffset = DragSlot->GetOffset();

	const float DesiredAbscissa = DragSlotOffset.Left + DragDelta;
	 
	bool bValidDrag = (DesiredAbscissa >= AbscissaRange.X) && (DragSlotOffset.Left < AbscissaRange.Y || DesiredAbscissa < AbscissaRange.Y);

	if(bValidDrag)
	{
		float NodeNewAbscissa = DesiredAbscissa < AbscissaRange.X ? AbscissaRange.X : (DesiredAbscissa > AbscissaRange.Y ? AbscissaRange.Y : DesiredAbscissa);

		const FVector2D DraggedSlotSize(DragSlotOffset.Left, DragSlotOffset.Left + DragSlotOffset.Right);
		const float MouseDelta = NodeNewAbscissa - DragSlotOffset.Left;

		NodeNewAbscissa += ComputePanAmount(DraggedSlotSize, MouseDelta);

		// Keep slot's abscissa within range
		if(NodeNewAbscissa < AbscissaRange.X)
		{
			NodeNewAbscissa = AbscissaRange.X;
		}
		else if(NodeNewAbscissa > AbscissaRange.Y)
		{
			NodeNewAbscissa = AbscissaRange.Y;
		}

		if(NodeNewAbscissa != DragSlotOffset.Left)
		{
			TSharedPtr<SDataprepGraphBaseActionNode> DraggedActionNode = StaticCastSharedRef<SDataprepGraphBaseActionNode>(DragSlot->GetWidget());
			ensure(DraggedActionNode.IsValid());

			// Add offset from current position to related GraphNode
			DraggedActionNode->GetNodeObj()->NodePosX += NodeNewAbscissa - DragSlotOffset.Left;

			// Update drag slot
			DragSlotOffset.Left = NodeNewAbscissa;
			DragSlot->SetOffset(DragSlotOffset);

			// Check if center of dragged widget is over a neighboring widget by at least half its size
			IndexHovered = GetHoveredActionNode(DragSlotOffset.Left, DragSlotOffset.Right);
			UpdateLayoutOnDrag();
		}

		Invalidate(EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::ChildOrder);
	}
}

void SDataprepGraphTrackWidget::OnEndNodeDrag(FVector2D& OutScreenSpacePosition, int32& OutDraggedIndex, int32&OutDroppedIndex)
{
	if(!DragSlot)
	{
		ensure(false);
		return;
	}

	// Retrieve all data potentially useful for the drop
	OutScreenSpacePosition = bMouseInScope ? GetAbsoluteSoftwareCursorPosition() : FSlateApplication::Get().GetCursorPos();
	OutDraggedIndex = IndexDragged;
	OutDroppedIndex = IndexHovered;

	TSharedPtr<SDataprepGraphBaseActionNode> ActionNode = StaticCastSharedRef<SDataprepGraphBaseActionNode>(DragSlot->GetWidget());
	ensure(ActionNode.IsValid());

	// Reset all  members used during the drag
	bIsCopying = false;
	bMouseInScope = false;

	SlotCount = ActionNodes.Num();

	DragSlot->SetOffset(FVector2D(-100., -10.f));
	DragSlot->AttachWidget(SNullWidget::NullWidget);

	DropSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);
	ActionSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);

	IndexDragged = INDEX_NONE;
	IndexHovered = INDEX_NONE;

	TrackCursorOffset = FVector2D::ZeroVector;

	// Update the widget
	RefreshLayout();

	Invalidate(EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::ChildOrder);
}

void SDataprepGraphTrackWidget::UpdateDragIndicator(FVector2D MousePosition)
{
	int32 NewDragIndicatorIndex = INDEX_NONE;

	const float LocalAbscissa = GetTickSpaceGeometry().AbsoluteToLocal(MousePosition).X - 1.f;
	FMargin SlotOffset = ActionSlots[0]->GetOffset();
	float MaxAbscissa = SlotOffset.Left + (SlotOffset.Right * 0.25f);

	if(LocalAbscissa < MaxAbscissa)
	{
		NewDragIndicatorIndex = 0;
	}
	else
	{
		for(int32 Index = 0; Index <= SlotCount; ++Index)
		{
			SlotOffset = ActionSlots[Index]->GetOffset();
			if(LocalAbscissa < (SlotOffset.Left + SlotOffset.Right + 1.f))
			{
				NewDragIndicatorIndex = Index;
				break;
			}
		}

		if(NewDragIndicatorIndex == INDEX_NONE)
		{
			NewDragIndicatorIndex = SlotCount;
		}
	}

	if(NewDragIndicatorIndex == DragIndicatorIndex)
	{
		return;
	}

	ResetDragIndicator();

	DragIndicatorIndex = NewDragIndicatorIndex;

	SConstraintCanvas::FSlot* TrackSlot = TrackSlots[SlotCount];

	if(NewDragIndicatorIndex == SlotCount)
	{
		float TrackLeftOffset = ActionSlots[SlotCount]->GetOffset().Left + (SDataprepGraphActionNode::DefaultWidth * 0.5f) - 16.f;

		SlotOffset = TrackSlots[0]->GetOffset();

		TrackSlot->SetOffset(FMargin(TrackLeftOffset, SlotOffset.Top, SlotOffset.Right, SlotOffset.Bottom));
		TrackSlot->AttachWidget(TrackSlotDrop.ToSharedRef());

		UpdateLine(SlotCount);
	}
	else
	{
		TrackSlot->SetOffset(TrackSlots[DragIndicatorIndex]->GetOffset());
		TrackSlot->AttachWidget(TrackSlotDrop.ToSharedRef());
	}
}

void SDataprepGraphTrackWidget::OnControlKeyDown(bool bCtrlDown, FVector2D& OutScreenSpacePosition)
{
	OutScreenSpacePosition = FVector2D::ZeroVector;

	if(bIsCopying == bCtrlDown)
	{
		return;
	}

	if(IndexDragged == INDEX_NONE)
	{
		bIsCopying = false;
		return;
	}

	bIsCopying = bCtrlDown;

	SlotCount = ActionNodes.Num();

	FMargin DragSlotOffset = DragSlot->GetOffset();
	IndexHovered = GetHoveredActionNode(DragSlotOffset.Left, DragSlotOffset.Right);

	for(int32 Index = 0; Index < SlotCount; ++Index)
	{
		ActionSlots[SlotCount]->AttachWidget(ActionNodes[Index]);
	}

	if(bIsCopying)
	{
		DropSlots[SlotCount]->AttachWidget(DropFiller.ToSharedRef());
		ActionSlots[SlotCount]->AttachWidget(DummyAction.ToSharedRef());
		TrackSlots[SlotCount]->AttachWidget(TrackSlotRegular.ToSharedRef());

		// Set the maximum move on the right and new working size
		const FVector2D DragSize = GetNodeSize(DragSlot->GetWidget());
		const FMargin SlotOffset = TrackSlots[SlotCount]->GetOffset();

		AbscissaRange.Y = SlotOffset.Left - (DragSize.X * 0.5f) + InterSlotSpacing + 1.f;
		WorkingSize.X = AbscissaRange.Y + DragSize.X + 1.f;

		++SlotCount;
	}
	else
	{
		FMargin SlotOffset = DropSlots[SlotCount - 1]->GetOffset();
		AbscissaRange.Y = SlotOffset.Left + SlotOffset.Right;

		WorkingSize.X = DropSlots[SlotCount]->GetOffset().Left + SDataprepGraphActionNode::DefaultWidth + InterSlotSpacing;

		if(DragSlotOffset.Left > AbscissaRange.Y)
		{
			OutScreenSpacePosition = GetAbsoluteSoftwareCursorPosition();

			DragSlotOffset.Left = AbscissaRange.Y;
			DragSlot->SetOffset(DragSlotOffset);

			IndexHovered = GetHoveredActionNode(DragSlotOffset.Left, DragSlotOffset.Right);
		}

		DropSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);
		ActionSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);
		TrackSlots[SlotCount]->AttachWidget(SNullWidget::NullWidget);
	}

	UpdateLayoutOnDrag();

	Invalidate(EInvalidateWidgetReason::ChildOrder | EInvalidateWidgetReason::Layout);
}

FVector2D SDataprepGraphTrackWidget::GetNodeSize(const TSharedRef<SWidget>& Widget) const
{
	FVector2D Size = Widget->GetCachedGeometry().GetLocalSize();

	if(Size == FVector2D::ZeroVector)
	{
		Size = Widget->GetDesiredSize();
		if(Size == FVector2D::ZeroVector)
		{
			Size.Set(SDataprepGraphActionNode::DefaultWidth, SDataprepGraphActionNode::DefaultWidth);
		}
	}

	return Size;
}

void SDataprepGraphTrackWidget::UpdateLine(int32 LastIndex)
{
	LineSize = FVector2D(0.f, 4.f);

	if(SlotCount > 1)
	{
		FMargin OffsetDiff = TrackSlots[LastIndex]->GetOffset() - TrackSlots[0]->GetOffset();
		FMargin LineOffset = LineSlot->GetOffset();

		LineOffset.Left = TrackSlots[0]->GetOffset().Left + 16.f;
		LineOffset.Right = OffsetDiff.Left;
		LineSize.X = OffsetDiff.Left;

		LineSlot->SetOffset(LineOffset);
	}
}

#undef LOCTEXT_NAMESPACE
