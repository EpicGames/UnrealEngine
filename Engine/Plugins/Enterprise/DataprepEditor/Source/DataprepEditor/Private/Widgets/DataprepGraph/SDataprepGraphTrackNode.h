// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "DataprepAsset.h"

#include "SGraphNode.h"
#include "UObject/StrongObjectPtr.h"

class UDataprepGraphRecipeNode;

class FGraphNodeFactory;
class SDataprepGraphBaseActionNode;
class SDataprepGraphActionNode;
class SDataprepGraphEditor;
class SDataprepGraphTrackNode;
class SDataprepGraphTrackWidget;
class UDataprepAsset;
class UDataprepGraphActionK2Node;
class UDataprepGraphActionNode;
class UEdGraph;

class FDragDropActionNode : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragDropActionNode, FDragDropOperation)

	static TSharedRef<FDragDropActionNode> New(const TSharedRef<SDataprepGraphTrackNode>& InTrackNodePtr, const TSharedRef<SDataprepGraphBaseActionNode>& InDraggedNode);

	virtual ~FDragDropActionNode() {}

	// FDragDropOperation interface
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override { Impl->OnDrop(bDropWasHandled, MouseEvent); }
	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override { Impl->OnDragged(DragDropEvent); }
	virtual FCursorReply OnCursorQuery() override { return Impl->OnCursorQuery(); }
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override { return Impl->GetDefaultDecorator(); }
	virtual FVector2D GetDecoratorPosition() const override { return Impl->GetDecoratorPosition(); }
	virtual void SetDecoratorVisibility(bool bVisible) override { Impl->SetDecoratorVisibility(bVisible); }
	virtual bool IsExternalOperation() const override { return Impl->IsExternalOperation(); }
	virtual bool IsWindowlessOperation() const override { return Impl->IsWindowlessOperation(); }
	// End of FDragDropOperation interface

private:
	TSharedPtr<FDragDropActionNode> Impl;
};

/**
 * The SDataprepGraphTrackNode class is a specialization of SGraphNode
 * to handle the actions of a Dataprep asset
 */
class SDataprepGraphTrackNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SDataprepGraphTrackNode){}
		SLATE_ARGUMENT(TSharedPtr<FGraphNodeFactory>, NodeFactory)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepGraphRecipeNode* InNode);

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	// End of SGraphNode interface

	// SNodePanel::SNode interface
	virtual bool CanBeSelected(const FVector2f& /*MousePositionInNode*/) const override
	{
		return false;
	}
	virtual void SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel) override;
	virtual void MoveTo( const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true ) override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual bool ShouldAllowCulling() const override { return false; }
	// End of SNodePanel::SNode interface

	// SWidget interface
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	UDataprepAsset* GetDataprepAsset() { return DataprepAssetPtr.Get(); }
	const UDataprepAsset* GetDataprepAsset() const { return DataprepAssetPtr.Get(); }

	void OnControlKeyChanged(bool bControlKeyDown);

	/** Initiates the horizontal drag of an action node */
	void OnStartNodeDrag(const TSharedRef<SDataprepGraphBaseActionNode>& ActionNode);

	/**
	 * Terminates the horizontal drag of an action node
	 * @param bDoDrop Indicates if the drop must be done
	 */
	void OnEndNodeDrag(bool bDoDrop);

	/**
	 * Updates the position of other action nodes based on the position of the incoming node
	 * @return True if the action node was actually dragged
	 */
	bool OnNodeDragged( TSharedPtr<SDataprepGraphBaseActionNode>& ActionNodePtr, const FVector2D& DragScreenSpacePosition, const FVector2D& ScreenSpaceDelta);

	/** Update the execution order of the actions and call ReArrangeActionNodes */
	void OnActionsOrderChanged();

	/** Recomputes the position of each action node */
	bool RefreshLayout();

	/** Recompute the boundaries of the graph based on the new size and/or the new zoom factor in graph panel */
	FSlateRect Update();

	/** Start editing of action asset associated to input EdGraphNode */
	void RequestRename(const UEdGraphNode* Node);

	/**
	 * Change the view position of the panel based on input screen space position
	 * @Note: Replaces SGrpahPanel::RequestDeferredPan which perform the change one frame too early
	 */
	void RequestViewportPan(const FVector2D& ScreenSpacePosition);

	/** Helper to set the position of the graph nodes registered to the graph panel */
	void UpdateProxyNode(TSharedRef<SDataprepGraphBaseActionNode> ActioNodePtr, const FVector2D& ScreenSpacePosition);

	/** Miscellaneous values used in the display */
	static FVector2D TrackAnchor;

	/** Get action from node index, taking into account grouping */
	int32 GetActionIndex(int32 NodeIndex) const;

	/** Get the number of actions from node index. Group nodes can have more than 1 action */
	int32 GetNumActions(int32 NodeIndex) const;

private:
	FVector2D ComputePanAmount(const FVector2D& ScreenSpacePosition, float MaxPanSpeed = 200.f);

private:
	/** Pointer to the widget displaying the track */
	TSharedPtr<SDataprepGraphTrackWidget> TrackWidgetPtr;

	/** Array of action node's widgets */
	TArray<TSharedPtr<SDataprepGraphBaseActionNode>> ActionNodes;

	/** Weak pointer to the Dataprep asset holding the displayed actions */
	TWeakObjectPtr<UDataprepAsset> DataprepAssetPtr;

	/** Range for abscissa of action nodes used during drag-and-drop */
	FVector2D AbscissaRange;

	/** Indicates a drag is happening */
	bool bNodeDragging;

	/** Indicates a drag has started.  */
	bool bNodeDragJustStarted;

	/** Indicates a drag has started.  */
	float LastDragDirection;

	/**
	 * Indicates to skip the next mouse position update during a drag
	 * @remark Set to true when FSlateApplication::SetCursorPos has been called
	 */
	bool bSkipNextDragUpdate;

	/** Cached of the last position the software cursor was set to */
	FVector2D LastSetCursorPosition;

	/**
	 * Indicates to skip the next refresh of the track widget layout
	 * @remark Set to true when SNodePanel::RestoreViewSettings has been called
	 */
	bool bSkipRefreshLayout;

	/** Indicates whether the mouse pointer left the graph panel on the left */
	bool bCursorLeftOnLeft;

	/** Indicates whether the mouse pointer left the graph panel on the right */
	bool bCursorLeftOnRight;

	/** Array of strong pointers to the UEdGraphNodes created for the Dataprep asset's actions */
	TArray<TStrongObjectPtr<UEdGraphNode>> EdGraphActionNodes;

	TSharedPtr<FGraphNodeFactory> NodeFactory;

	friend SDataprepGraphTrackWidget;
};
