// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepIntegerFilter.h"

#include "DataprepAsset.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorUtils.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SelectionSystem/DataprepIntegerFilter.h"
#include "Widgets/DataprepGraph/DataprepActionWidgetsUtils.h"
#include "Widgets/DataprepWidgets.h"
#include "Widgets/Parameterization/SDataprepParameterizationLinkIcon.h"

#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "DataprepIntegerFilter"

namespace SDataprepIntegerFilterUtils
{
	void PostEditChainProperty(UDataprepIntegerFilter* Filter, FName PropertyName)
	{
		if ( Filter )
		{
			FProperty* Property = Filter->GetClass()->FindPropertyByName( PropertyName );
			check( Property );

			FEditPropertyChain EditChain;
			EditChain.AddHead( Property );
			EditChain.SetActivePropertyNode( Property );
			FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
			FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
			Filter->PostEditChangeChainProperty( EditChangeChainEvent );
		}
	}
};

void SDataprepIntegerFilter::Construct(const FArguments& InArgs, UDataprepIntegerFilter& InFilter)
{
	Filter = &InFilter;
	OldEqualValue = Filter->GetEqualValue();

	DataprepActionWidgetsUtils::GenerateListEntriesFromEnum< EDataprepIntegerMatchType >( IntMatchingOptions );

	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( Filter ) )
	{
		UClass* FilterClass = InFilter.GetClass();

		{
			FName PropertyName = FName( TEXT("IntegerMatchingCriteria") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			MatchingCriteriaParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		{
			FName PropertyName = FName( TEXT("EqualValue") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			EqualValueParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		{
			FName PropertyName = FName(TEXT("FromValue"));
			FProperty* Property = FilterClass->FindPropertyByName(PropertyName);
			check(Property);
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace(Property, PropertyName, INDEX_NONE);

			FromValueParameterizationActionData = MakeShared<FDataprepParametrizationActionData>(*DataprepAsset, InFilter, PropertyChain);
		}

		{
			FName PropertyName = FName(TEXT("ToValue"));
			FProperty* Property = FilterClass->FindPropertyByName(PropertyName);
			check(Property);
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace(Property, PropertyName, INDEX_NONE);

			ToValueParameterizationActionData = MakeShared<FDataprepParametrizationActionData>(*DataprepAsset, InFilter, PropertyChain);
		}

		OnParameterizationStatusForObjectsChangedHandle = DataprepAsset->OnParameterizedObjectsStatusChanged.AddSP( this, &SDataprepIntegerFilter::OnParameterizationStatusForObjectsChanged );
	}

	UpdateVisualDisplay();
}

SDataprepIntegerFilter::~SDataprepIntegerFilter()
{
	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( Filter ) )
	{
		DataprepAsset->OnParameterizedObjectsStatusChanged.Remove( OnParameterizationStatusForObjectsChangedHandle );
	}
}

void SDataprepIntegerFilter::UpdateVisualDisplay()
{
	TSharedPtr<SHorizontalBox> MatchingCriteriaHorizontalBox;
	TSharedPtr<SHorizontalBox> EqualValueHorizontalBox;
	TSharedPtr<SHorizontalBox> FromValueHorizontalBox;
	TSharedPtr<SHorizontalBox> ToValueHorizontalBox;
	TSharedPtr<SHorizontalBox> ToleranceHorizontalBox;

	ChildSlot
	[
		SNew( SBox )
		.MinDesiredWidth( 400.f )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.Padding( 5.f )
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepIntegerFilter::OnGetContextMenuForMatchingCriteria )
					[
						SAssignNew( MatchingCriteriaHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SAssignNew( IntMatchingCriteriaWidget, SComboBox< TSharedPtr< FListEntry > > )
							.OptionsSource( &IntMatchingOptions )
							.OnGenerateWidget( this, &SDataprepIntegerFilter::OnGenerateWidgetForMatchingCriteria )
							.OnSelectionChanged( this, &SDataprepIntegerFilter::OnSelectedCriteriaChanged )
							.OnComboBoxOpening( this, &SDataprepIntegerFilter::OnCriteriaComboBoxOpenning )
							[
								SNew( STextBlock )
								.Text( this, &SDataprepIntegerFilter::GetSelectedCriteriaText )
								.ToolTipText( this, &SDataprepIntegerFilter::GetSelectedCriteriaTooltipText )
								.Justification( ETextJustify::Center )
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding( 5.f )
				[
					SNew( SDataprepContextMenuOverride )
					.Visibility( this, &SDataprepIntegerFilter::GetSingleValueVisibility )
					.OnContextMenuOpening( this, &SDataprepIntegerFilter::OnGetContextMenuForEqualValue )
					[
						SAssignNew( EqualValueHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SSpinBox<int> )
							.Value( this, &SDataprepIntegerFilter::GetEqualValue )
							.OnValueChanged( this, &SDataprepIntegerFilter::OnEqualValueChanged )
							.OnValueCommitted( this, &SDataprepIntegerFilter::OnEqualValueComitted )
							.Justification( ETextJustify::Center )
							.MinValue( TOptional< int >() )
							.MaxValue( TOptional< int >() )
							.ContextMenuExtender( this, &SDataprepIntegerFilter::ExtendContextMenuForEqualValueBox )
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(5.f)
				[
					SNew(SHorizontalBox)
					.Visibility( this, &SDataprepIntegerFilter::GetDoubleValueVisibility )
					+ SHorizontalBox::Slot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SDataprepContextMenuOverride)
							.OnContextMenuOpening(this, &SDataprepIntegerFilter::OnGetContextMenuForFromValue)
							[
								SAssignNew(FromValueHorizontalBox, SHorizontalBox)
								+ SHorizontalBox::Slot()
								[
									SNew(SSpinBox<int>)
									.Justification(ETextJustify::Center)
									.Value(this, &SDataprepIntegerFilter::GetFromValue)
									.OnValueChanged(this, &SDataprepIntegerFilter::OnFromValueChanged)
									.OnValueCommitted(this, &SDataprepIntegerFilter::OnFromValueComitted)
									.MinValue(TOptional< int >())
									.MaxValue(TOptional< int >())
									.ContextMenuExtender( this, &SDataprepIntegerFilter::ExtendContextMenuForFromValueBox )
								]
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AndText", "And"))
							.Justification(ETextJustify::Center)
						]
						+ SHorizontalBox::Slot()
						[
							SNew(SDataprepContextMenuOverride)
							.OnContextMenuOpening(this, &SDataprepIntegerFilter::OnGetContextMenuForToValue)
							[
								SAssignNew(ToValueHorizontalBox, SHorizontalBox)
								+ SHorizontalBox::Slot()
								[
									SNew(SSpinBox<int>)
									.Justification(ETextJustify::Center)
									.Value(this, &SDataprepIntegerFilter::GetToValue)
									.OnValueChanged(this, &SDataprepIntegerFilter::OnToValueChanged)
									.OnValueCommitted(this, &SDataprepIntegerFilter::OnToValueComitted)
									.MinValue(TOptional< int >())
									.MaxValue(TOptional< int >())
									.ContextMenuExtender( this, &SDataprepIntegerFilter::ExtendContextMenuForToValueBox )
								]
							]
						]
					]
				]
			]
		]
	];

	if ( MatchingCriteriaParameterizationActionData && MatchingCriteriaParameterizationActionData->IsValid() )
	{
		if ( MatchingCriteriaParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, MatchingCriteriaParameterizationActionData->PropertyChain ) )
		{
			MatchingCriteriaHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, MatchingCriteriaParameterizationActionData->DataprepAsset, Filter, MatchingCriteriaParameterizationActionData->PropertyChain )
				];
		}
	}

	if ( EqualValueParameterizationActionData && EqualValueParameterizationActionData->IsValid() )
	{
		if ( EqualValueParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, EqualValueParameterizationActionData->PropertyChain ) )
		{
			EqualValueHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, EqualValueParameterizationActionData->DataprepAsset, Filter, EqualValueParameterizationActionData->PropertyChain )
				];
		}
	}

	if (FromValueParameterizationActionData && FromValueParameterizationActionData->IsValid())
	{
		if (FromValueParameterizationActionData->DataprepAsset->IsObjectPropertyBinded(Filter, FromValueParameterizationActionData->PropertyChain))
		{
			FromValueHorizontalBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
				.AutoWidth()
				[
					SNew(SDataprepParameterizationLinkIcon, FromValueParameterizationActionData->DataprepAsset, Filter, FromValueParameterizationActionData->PropertyChain)
				];
		}
	}

	if (ToValueParameterizationActionData && ToValueParameterizationActionData->IsValid())
	{
		if (ToValueParameterizationActionData->DataprepAsset->IsObjectPropertyBinded(Filter, ToValueParameterizationActionData->PropertyChain))
		{
			ToValueHorizontalBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
				.AutoWidth()
				[
					SNew(SDataprepParameterizationLinkIcon, ToValueParameterizationActionData->DataprepAsset, Filter, ToValueParameterizationActionData->PropertyChain)
				];
		}
	}
}

EVisibility SDataprepIntegerFilter::GetDoubleValueVisibility() const
{
	check(Filter);
	return Filter->GetIntegerMatchingCriteria() == EDataprepIntegerMatchType::InBetween ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDataprepIntegerFilter::GetSingleValueVisibility() const
{
	check(Filter);
	return Filter->GetIntegerMatchingCriteria() != EDataprepIntegerMatchType::InBetween ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SDataprepIntegerFilter::OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const
{
	return SNew( STextBlock )
		.Text( ListEntry->Get<0>() )
		.ToolTipText( ListEntry->Get<1>() );
}

FText SDataprepIntegerFilter::GetSelectedCriteriaText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepIntegerMatchType >();
	return Enum->GetDisplayNameTextByValue( static_cast<uint8>( Filter->GetIntegerMatchingCriteria() ) );
}

FText SDataprepIntegerFilter::GetSelectedCriteriaTooltipText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepIntegerMatchType >();
	return Enum->GetToolTipTextByIndex( Enum->GetIndexByValue( static_cast<uint8>( Filter->GetIntegerMatchingCriteria() ) ) );
}

void SDataprepIntegerFilter::OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType)
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepIntegerMatchType >();
	EDataprepIntegerMatchType IntMatchType = static_cast< EDataprepIntegerMatchType >( Enum->GetValueByIndex( ListEntry->Get<2>() ) );

	if ( IntMatchType != Filter->GetIntegerMatchingCriteria() )
	{
		FScopedTransaction Transaction( LOCTEXT("SelectionCriteriaChangedTransaction", "Changed the Integer Selection Criteria") );
		Filter->SetIntegerMatchingCriteria( IntMatchType );

		SDataprepIntegerFilterUtils::PostEditChainProperty( Filter, TEXT("IntegerMatchingCriteria") );
	}
}

void SDataprepIntegerFilter::OnCriteriaComboBoxOpenning()
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepIntegerMatchType >();
	int32 EnumValueMapping = Enum->GetIndexByValue( static_cast<uint8>( Filter->GetIntegerMatchingCriteria() ) );

	TSharedPtr<FListEntry> ItemToSelect;
	for ( const TSharedPtr<FListEntry>& Entry : IntMatchingOptions )
	{
		if ( Entry->Get<2>() == EnumValueMapping )
		{
			ItemToSelect = Entry;
			break;
		}
	}

	check( IntMatchingCriteriaWidget );
	IntMatchingCriteriaWidget->SetSelectedItem( ItemToSelect );
}

TSharedPtr<SWidget> SDataprepIntegerFilter::OnGetContextMenuForMatchingCriteria()
{
	return FDataprepEditorUtils::MakeContextMenu( MatchingCriteriaParameterizationActionData );
}

void SDataprepIntegerFilter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}

int SDataprepIntegerFilter::GetEqualValue() const
{
	check( Filter );
	return Filter->GetEqualValue();
}

void SDataprepIntegerFilter::OnEqualValueChanged(int NewEqualValue)
{
	check( Filter );
	Filter->SetEqualValue( NewEqualValue );
}

void SDataprepIntegerFilter::OnEqualValueComitted(int NewEqualValue, ETextCommit::Type CommitType)
{
	check( Filter );

	if ( OldEqualValue != NewEqualValue )
	{
		// Trick for the transaction 
		Filter->SetEqualValue( OldEqualValue );
		FScopedTransaction Transaction( LOCTEXT("EqualValueChangedTransaction","Change the Equal Value") );
		Filter->SetEqualValue( NewEqualValue );

		SDataprepIntegerFilterUtils::PostEditChainProperty( Filter, TEXT("EqualValue") );
		OldEqualValue = NewEqualValue;
	}
}

TSharedPtr<SWidget> SDataprepIntegerFilter::OnGetContextMenuForEqualValue()
{
	return FDataprepEditorUtils::MakeContextMenu( EqualValueParameterizationActionData );
}

void SDataprepIntegerFilter::ExtendContextMenuForEqualValueBox(FMenuBuilder& MenuBuilder)
{
	if ( EqualValueParameterizationActionData && EqualValueParameterizationActionData->IsValid() && Filter )
	{
		FDataprepEditorUtils::PopulateMenuForParameterization( MenuBuilder, *EqualValueParameterizationActionData->DataprepAsset, *Filter, EqualValueParameterizationActionData->PropertyChain );
	}
}

int SDataprepIntegerFilter::GetFromValue() const
{
	check(Filter);
	return Filter->GetFromValue();
}

void SDataprepIntegerFilter::OnFromValueChanged(int NewFromValue)
{
	check(Filter);
	Filter->SetFromValue(NewFromValue);
}

void SDataprepIntegerFilter::OnFromValueComitted(int NewFromValue, ETextCommit::Type CommitType)
{
	check(Filter);

	if (OldFromValue != NewFromValue)
	{
		// Trick for the transaction 
		Filter->SetFromValue(OldFromValue);
		FScopedTransaction Transaction(LOCTEXT("FromValueChangedTransaction", "Change the From Value"));
		Filter->SetFromValue(NewFromValue);

		SDataprepIntegerFilterUtils::PostEditChainProperty(Filter, TEXT("FromValue"));
		OldFromValue = NewFromValue;
	}
}

TSharedPtr<SWidget> SDataprepIntegerFilter::OnGetContextMenuForFromValue()
{
	return FDataprepEditorUtils::MakeContextMenu(FromValueParameterizationActionData);
}

void SDataprepIntegerFilter::ExtendContextMenuForFromValueBox(FMenuBuilder& MenuBuilder)
{
	if (FromValueParameterizationActionData && FromValueParameterizationActionData->IsValid() && Filter)
	{
		FDataprepEditorUtils::PopulateMenuForParameterization(MenuBuilder, *FromValueParameterizationActionData->DataprepAsset, *Filter, FromValueParameterizationActionData->PropertyChain);
	}
}

int SDataprepIntegerFilter::GetToValue() const
{
	check(Filter);
	return Filter->GetToValue();
}

void SDataprepIntegerFilter::OnToValueChanged(int NewFromValue)
{
	check(Filter);
	Filter->SetToValue(NewFromValue);
}

void SDataprepIntegerFilter::OnToValueComitted(int NewToValue, ETextCommit::Type CommitType)
{
	check(Filter);

	if (OldToValue != NewToValue)
	{
		// Trick for the transaction 
		Filter->SetToValue(OldToValue);
		FScopedTransaction Transaction(LOCTEXT("ToValueChangedTransaction", "Change the To Value"));
		Filter->SetToValue(NewToValue);

		SDataprepIntegerFilterUtils::PostEditChainProperty(Filter, TEXT("ToValue"));
		OldToValue = NewToValue;
	}
}

TSharedPtr<SWidget> SDataprepIntegerFilter::OnGetContextMenuForToValue()
{
	return FDataprepEditorUtils::MakeContextMenu(ToValueParameterizationActionData);
}

void SDataprepIntegerFilter::ExtendContextMenuForToValueBox(FMenuBuilder& MenuBuilder)
{
	if (ToValueParameterizationActionData && ToValueParameterizationActionData->IsValid() && Filter)
	{
		FDataprepEditorUtils::PopulateMenuForParameterization(MenuBuilder, *ToValueParameterizationActionData->DataprepAsset, *Filter, ToValueParameterizationActionData->PropertyChain);
	}
}

void SDataprepIntegerFilter::OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects)
{
	if ( !Objects || Objects->Contains( Filter ) )
	{
		UpdateVisualDisplay();
	}
}

#undef LOCTEXT_NAMESPACE
