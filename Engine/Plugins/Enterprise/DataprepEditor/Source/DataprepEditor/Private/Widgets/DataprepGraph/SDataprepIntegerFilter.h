// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FMenuBuilder;
class SComboButton;
class UDataprepIntegerFilter;

template<class t>
class SComboBox;

struct FDataprepParametrizationActionData;

class SDataprepIntegerFilter : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SDataprepIntegerFilter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepIntegerFilter& InFilter);

	virtual ~SDataprepIntegerFilter();

private:

	// Refresh the ui
	void UpdateVisualDisplay();

	// The string matching option for the combo box (Displayed text, Tooltip, mapping for the UEnum)
	using FListEntry = TTuple<FText, FText, int32>;

	// Those function are for the float matching criteria display
	TSharedRef<SWidget> OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const;
	FText GetSelectedCriteriaText() const;
	FText GetSelectedCriteriaTooltipText() const;
	void OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType);
	void OnCriteriaComboBoxOpenning();
	TSharedPtr<SWidget> OnGetContextMenuForMatchingCriteria();

	// Those functions are for the int that will be compare against the fetched int
	int GetEqualValue() const;
	void OnEqualValueChanged(int NewEqualValue);
	void OnEqualValueComitted(int NewEqualValue, ETextCommit::Type CommitType);
	TSharedPtr<SWidget> OnGetContextMenuForEqualValue();
	void ExtendContextMenuForEqualValueBox(FMenuBuilder& MenuBuilder);

	int GetFromValue() const;
	void OnFromValueChanged(int NewEqualValue);
	void OnFromValueComitted(int NewEqualValue, ETextCommit::Type CommitType);
	TSharedPtr<SWidget> OnGetContextMenuForFromValue();
	void ExtendContextMenuForFromValueBox(FMenuBuilder& MenuBuilder);

	int GetToValue() const;
	void OnToValueChanged(int NewEqualValue);
	void OnToValueComitted(int NewEqualValue, ETextCommit::Type CommitType);
	TSharedPtr<SWidget> OnGetContextMenuForToValue();
	void ExtendContextMenuForToValueBox(FMenuBuilder& MenuBuilder);

	// Event for dataprep parameterization
	void OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects);

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepIntegerFilter");
	}
	//~ End FGCObject interface

	EVisibility GetSingleValueVisibility() const;
	EVisibility GetDoubleValueVisibility() const;

	float OldEqualValue;
	float OldFromValue;
	float OldToValue;

	TObjectPtr<UDataprepIntegerFilter> Filter;

	TArray<TSharedPtr<FListEntry>> IntMatchingOptions;

	TSharedPtr<SComboBox<TSharedPtr<FListEntry>>> IntMatchingCriteriaWidget;

	TSharedPtr<FDataprepParametrizationActionData> MatchingCriteriaParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> EqualValueParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> FromValueParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> ToValueParameterizationActionData;

	FDelegateHandle OnParameterizationStatusForObjectsChangedHandle;
};
