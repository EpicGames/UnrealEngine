// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepObjectSelectionFilter.h"
#include "SelectionSystem/DataprepObjectSelectionFilter.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "DataprepObjectSelectionFilter"

void SDataprepObjectSelectionFilter::Construct(const FArguments& InArgs, UDataprepObjectSelectionFilter& InFilter)
{
	Filter = &InFilter;

	FString ObjectNames;

	const int32 MaxObjectNamesLen = 250;
	int32 ObjectNamesUsed = 0;

	for( const FString& Name : Filter->GetCachedNames() )
	{
		if( Name.IsEmpty() )
		{
			continue;
		}

		if( ObjectNames.Len() > 0 )
		{
			ObjectNames.Append( ", " );
		}

		ObjectNames.Append( Name );

		++ObjectNamesUsed;

		if( ObjectNames.Len() >= MaxObjectNamesLen )
		{
			break;
		}
	}

	if( ObjectNamesUsed < Filter->GetCachedNames().Num() )
	{
		ObjectNames.Append( "..." );
	}
	
	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.Padding( 3, 2, 3, 0)
		.AutoHeight()
		[
			SNew( STextBlock )
			.ColorAndOpacity( FLinearColor::Yellow )
			.Text( FText::Format( LOCTEXT("NumActors", "Num Actors {0}"), FText::AsNumber( Filter->GetNumActors() ) ) )
		]
		+ SVerticalBox::Slot()
		.Padding( 3, 2, 3, 2)
		.AutoHeight()
		[
			SNew( STextBlock )
			.ColorAndOpacity( FLinearColor::Yellow )
			.Text( FText::Format( LOCTEXT("NumAssets", "Num Assets {0}"), FText::AsNumber( Filter->GetNumAssets() ) ) )
		]
		+ SVerticalBox::Slot()
		.Padding( 3, 5, 3, 0)
		.AutoHeight()
		[
			SNew( STextBlock )
			.AutoWrapText( true )
			.WrappingPolicy( ETextWrappingPolicy::AllowPerCharacterWrapping )
			.Font( FCoreStyle::GetDefaultFontStyle( "Italic", 10 ) )
			.Text( FText::FromString( ObjectNames ) )
		]
	];
}

void SDataprepObjectSelectionFilter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}

#undef LOCTEXT_NAMESPACE
