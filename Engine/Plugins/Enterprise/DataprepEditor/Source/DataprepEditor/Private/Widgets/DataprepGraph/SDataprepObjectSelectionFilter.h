// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class SComboButton;
class UDataprepObjectSelectionFilter;

template<class t>
class SComboBox;

struct FDataprepParametrizationActionData;

class SDataprepObjectSelectionFilter : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SDataprepObjectSelectionFilter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepObjectSelectionFilter& InFilter);

private:

	TObjectPtr<UDataprepObjectSelectionFilter> Filter;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepObjectSelectionFilter");
	}
	//~ End FGCObject interface
};
