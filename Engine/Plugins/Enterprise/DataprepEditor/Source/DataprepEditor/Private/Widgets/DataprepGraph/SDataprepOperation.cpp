// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataprepOperation.h"

#include "DataprepEditorStyle.h"
#include "DataprepOperation.h"
#include "Widgets/DataprepWidgets.h"


void SDataprepOperation::Construct(const FArguments& InArgs, UDataprepOperation* InOperation, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext)
{
	Operation = InOperation;

	TAttribute<FText> TooltipTextAttribute = MakeAttributeSP( this, &SDataprepOperation::GetTooltipText );
	SetToolTipText( TooltipTextAttribute );

	SDataprepActionBlock::Construct( SDataprepActionBlock::FArguments(), InDataprepActionContext );
}

FSlateColor SDataprepOperation::GetOutlineColor() const
{
	return FDataprepEditorStyle::GetColor( "DataprepActionStep.Operation.OutlineColor" );
}

FText SDataprepOperation::GetBlockTitle() const
{
	return Operation ? Operation->GetDisplayOperationName() : FText::FromString( TEXT("Operation is Nullptr!") ) ;
}

TSharedRef<SWidget> SDataprepOperation::GetContentWidget()
{
	return SNew( SDataprepDetailsView ).Object( Operation );
}

void SDataprepOperation::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Operation );
}

FText SDataprepOperation::GetTooltipText() const
{
	FText TooltipText;
	if ( Operation )
	{
		TooltipText = Operation->GetTooltip();
	}
	return TooltipText;
}
