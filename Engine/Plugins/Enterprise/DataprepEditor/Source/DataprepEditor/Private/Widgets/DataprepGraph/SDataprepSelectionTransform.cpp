// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataprepSelectionTransform.h"

#include "DataprepEditorStyle.h"
#include "SelectionSystem/DataprepSelectionTransform.h"
#include "Widgets/DataprepWidgets.h"


void SDataprepSelectionTransform::Construct(const FArguments& InArgs, UDataprepSelectionTransform* InTransform, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext)
{
	SelectionTransform = InTransform;

	TAttribute<FText> TooltipTextAttribute = MakeAttributeSP(this, &SDataprepSelectionTransform::GetTooltipText);
	SetToolTipText(TooltipTextAttribute);

	SDataprepActionBlock::Construct(SDataprepActionBlock::FArguments(), InDataprepActionContext);
}

FSlateColor SDataprepSelectionTransform::GetOutlineColor() const
{
	return FDataprepEditorStyle::GetColor("DataprepActionStep.SelectionTransform.OutlineColor");
}

FText SDataprepSelectionTransform::GetBlockTitle() const
{
	return SelectionTransform ? SelectionTransform->GetDisplayTransformName() : FText::FromString(TEXT("Transform is Nullptr!"));
}

TSharedRef<SWidget> SDataprepSelectionTransform::GetContentWidget()
{
	return SNew(SDataprepDetailsView).Object(SelectionTransform);
}

void SDataprepSelectionTransform::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SelectionTransform);
}

FText SDataprepSelectionTransform::GetTooltipText() const
{
	FText TooltipText;
	if (SelectionTransform)
	{
		TooltipText = SelectionTransform->GetTooltip();
	}
	return TooltipText;
}
