// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DataprepGraph/SDataprepActionBlock.h"

#include "UObject/GCObject.h"

class UDataprepSelectionTransform;
struct FDataprepSchemaActionContext;

class SDataprepSelectionTransform : public SDataprepActionBlock, public FGCObject
{
	SLATE_BEGIN_ARGS(SDataprepSelectionTransform){}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepSelectionTransform* InTransform, const TSharedRef<FDataprepSchemaActionContext>& InDataprepActionContext);

protected:

	// SDataprepActionBlock interface
	virtual FSlateColor GetOutlineColor() const override;
	virtual FText GetBlockTitle() const override;
	virtual TSharedRef<SWidget> GetContentWidget() override;
	//~ end of SDataprepActionBlock interface

private:

	FText GetTooltipText() const;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepSelectionTransform");
	}

	TObjectPtr<UDataprepSelectionTransform> SelectionTransform;
};
