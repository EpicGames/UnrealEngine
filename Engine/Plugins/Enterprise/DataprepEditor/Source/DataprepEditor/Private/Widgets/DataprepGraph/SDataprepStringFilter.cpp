// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataprepGraph/SDataprepStringFilter.h"

#include "DataprepAsset.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorUtils.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SelectionSystem/DataprepStringFilter.h"
#include "SelectionSystem/DataprepStringsArrayFilter.h"
#include "Widgets/DataprepGraph/DataprepActionWidgetsUtils.h"
#include "Widgets/DataprepWidgets.h"
#include "Widgets/Parameterization/SDataprepParameterizationLinkIcon.h"

#include "GraphEditorSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "DataprepStringFilter"

template <class FilterType>
void SDataprepStringFilter<FilterType>::Construct(const FArguments& InArgs, FilterType& InFilter)
{
	Filter = &InFilter;
	OldUserString = Filter->GetUserString();

	DataprepActionWidgetsUtils::GenerateListEntriesFromEnum< EDataprepStringMatchType >( StringMatchingOptions );

	OnUserStringArrayPostEditHandle = Filter->GetStringArray()->GetOnPostEdit().AddSP( this, &SDataprepStringFilter<FilterType>::OnUserStringArrayPropertyChanged );

	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( &InFilter ) )
	{
		UClass* FilterClass = InFilter.GetClass();

		{
			FName PropertyName = FName( TEXT("StringMatchingCriteria") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			MatchingCriteriaParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain );
		}

		{
			FName PropertyName = FName( TEXT("UserString") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			UserStringParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain);
		}

		{
			FName PropertyName = FName( TEXT("bMatchInArray") );
			FProperty* Property = FilterClass->FindPropertyByName( PropertyName );
			check( Property );
			TArray<FDataprepPropertyLink> PropertyChain;
			PropertyChain.Emplace( Property, PropertyName, INDEX_NONE );

			MatchInArrayParameterizationActionData = MakeShared<FDataprepParametrizationActionData>( *DataprepAsset, InFilter, PropertyChain);
		}

		OnParameterizationStatusForObjectsChangedHandle = DataprepAsset->OnParameterizedObjectsStatusChanged.AddSP( this, &SDataprepStringFilter<FilterType>::OnParameterizationStatusForObjectsChanged );
	}

	UpdateVisualDisplay();
}

template <class FilterType>
SDataprepStringFilter<FilterType>::~SDataprepStringFilter()
{
	Filter->GetStringArray()->GetOnPostEdit().Remove( OnUserStringArrayPostEditHandle );

	if ( UDataprepAsset* DataprepAsset = FDataprepCoreUtils::GetDataprepAssetOfObject( Filter ) )
	{
		DataprepAsset->OnParameterizedObjectsStatusChanged.Remove( OnParameterizationStatusForObjectsChangedHandle );
	}
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::UpdateVisualDisplay()
{
	TSharedPtr<SHorizontalBox> MatchingCriteriaHorizontalBox;
	TSharedPtr<SHorizontalBox> UserStringHorizontalBox;
	TSharedPtr<SHorizontalBox> MatchInArrayHorizontalBox;
	TSharedPtr<SHorizontalBox> MatchingArrayHorizontalBox;

	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

	ChildSlot
	[
		SNew( SBox )
		.MinDesiredWidth( 400.f )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.Padding( 5.f )
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepStringFilter<FilterType>::OnGetContextMenuForMatchInArray )
					[
						SAssignNew( MatchInArrayHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( SButton )
							.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
							.ToolTipText_Lambda( [this]()
							{
								return Filter->GetMatchInArray() ?
									LOCTEXT( "SwitchToSingleTooltip", "Switch to single" ) : LOCTEXT( "SwitchToArrayTooltip", "Switch to array" );
							})
							.Cursor( EMouseCursor::Default )
							.OnClicked( this, &SDataprepStringFilter<FilterType>::OnMatchInArrayClicked )
							.Content()
							[
								SNew( SImage )
								.Image_Lambda( [this]()
								{
									return Filter->GetMatchInArray() ? 
										FAppStyle::GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon")) : FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
								})
								.ColorAndOpacity( Settings->StringPinTypeColor )
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding( 0, 0, 6, 0 )
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepStringFilter<FilterType>::OnGetContextMenuForMatchingCriteria )
					[
						SAssignNew( MatchingCriteriaHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SAssignNew( StringMatchingCriteriaWidget, SComboBox< TSharedPtr< FListEntry > > )
							.OptionsSource( &StringMatchingOptions )
							.OnGenerateWidget( this, &SDataprepStringFilter::OnGenerateWidgetForMatchingCriteria )
							.OnSelectionChanged( this, &SDataprepStringFilter::OnSelectedCriteriaChanged )
							.Cursor( EMouseCursor::Default )
							.OnComboBoxOpening( this, &SDataprepStringFilter::OnCriteriaComboBoxOpenning )
							[
								SNew( STextBlock )
								.Text( this, &SDataprepStringFilter::GetSelectedCriteriaText )
								.ToolTipText( this, &SDataprepStringFilter::GetSelectedCriteriaTooltipText )
								.Justification( ETextJustify::Center )
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew( SDataprepContextMenuOverride )
					.OnContextMenuOpening( this, &SDataprepStringFilter<FilterType>::OnGetContextMenuForUserString )
					[
						SAssignNew( UserStringHorizontalBox, SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SEditableTextBox )
							.Text( this, &SDataprepStringFilter::GetUserString )
							.ContextMenuExtender( this, &SDataprepStringFilter::ExtendContextMenuForUserStringBox )
							.OnTextChanged( this, &SDataprepStringFilter::OnUserStringChanged )
							.OnTextCommitted( this, &SDataprepStringFilter::OnUserStringComitted )
							.Justification( ETextJustify::Center )
							.Visibility_Lambda([this]() -> EVisibility
							{
								if ( Filter )
								{
									return Filter->GetMatchInArray() ? EVisibility::Collapsed : EVisibility::Visible;
								}
								return EVisibility::Visible;
							})
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding( 5.f )
			.AutoHeight()
			[
				SAssignNew( MatchInArrayHorizontalBox, SHorizontalBox )
				+ SHorizontalBox::Slot()
				[
					SNew( SDataprepDetailsView )
					.Object( Filter ? Filter->GetStringArray() : nullptr )
					.Cursor( EMouseCursor::Default )
					.Visibility_Lambda([this]() -> EVisibility
					{
						if ( Filter )
						{
							return Filter->GetMatchInArray() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						return EVisibility::Collapsed;
					})
				]
			]
		]
	];

	if ( MatchingCriteriaParameterizationActionData && MatchingCriteriaParameterizationActionData->IsValid() )
	{
		if ( MatchingCriteriaParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, MatchingCriteriaParameterizationActionData->PropertyChain ) )
		{
			MatchingCriteriaHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, MatchingCriteriaParameterizationActionData->DataprepAsset, Filter, MatchingCriteriaParameterizationActionData->PropertyChain )
				];
		}
	}

	if ( UserStringParameterizationActionData && UserStringParameterizationActionData->IsValid() )
	{
		if ( UserStringParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, UserStringParameterizationActionData->PropertyChain ) )
		{
			UserStringHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, UserStringParameterizationActionData->DataprepAsset, Filter, UserStringParameterizationActionData->PropertyChain )
					.Visibility_Lambda([this]() -> EVisibility
					{
						if ( Filter )
						{
							return Filter->GetMatchInArray() ? EVisibility::Collapsed : EVisibility::Visible;
						}
						return EVisibility::Collapsed;
					})
				];
		}
	}

	if ( MatchInArrayParameterizationActionData && MatchInArrayParameterizationActionData->IsValid() )
	{
		if ( MatchInArrayParameterizationActionData->DataprepAsset->IsObjectPropertyBinded( Filter, MatchInArrayParameterizationActionData->PropertyChain ) )
		{
			MatchInArrayHorizontalBox->AddSlot()
				.HAlign( HAlign_Right )
				.VAlign( VAlign_Center )
				.Padding( FMargin(5.f, 0.f, 0.f, 0.f) )
				.AutoWidth()
				[
					SNew( SDataprepParameterizationLinkIcon, MatchInArrayParameterizationActionData->DataprepAsset, Filter, MatchInArrayParameterizationActionData->PropertyChain )
				];
		}
	}
}

template <class FilterType>
TSharedRef<SWidget> SDataprepStringFilter<FilterType>::OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const
{
	return SNew(STextBlock)
		.Text( ListEntry->Get<0>() )
		.ToolTipText( ListEntry->Get<1>() );
}

template <class FilterType>
FText SDataprepStringFilter<FilterType>::GetSelectedCriteriaText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepStringMatchType >();
	return Enum->GetDisplayNameTextByValue( static_cast<uint8>( Filter->GetStringMatchingCriteria() ) );
}

template <class FilterType>
FText SDataprepStringFilter<FilterType>::GetSelectedCriteriaTooltipText() const
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepStringMatchType >();
	return  Enum->GetToolTipTextByIndex( Enum->GetIndexByValue( static_cast<uint8>( Filter->GetStringMatchingCriteria() ) ) );
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnCriteriaComboBoxOpenning()
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepStringMatchType >();
	int32 EnumValueMapping = Enum->GetIndexByValue( static_cast<uint8>( Filter->GetStringMatchingCriteria() ) );

	TSharedPtr<FListEntry> ItemToSelect;
	for ( const TSharedPtr<FListEntry>& Entry : StringMatchingOptions )
	{
		if ( Entry->Get<2>() == EnumValueMapping )
		{
			ItemToSelect = Entry;
			break;
		}
	}

	check( StringMatchingCriteriaWidget );
	StringMatchingCriteriaWidget->SetSelectedItem( ItemToSelect );
}

template <class FilterType>
TSharedPtr<SWidget> SDataprepStringFilter<FilterType>::OnGetContextMenuForMatchingCriteria()
{
	return FDataprepEditorUtils::MakeContextMenu( MatchingCriteriaParameterizationActionData );
}

template <class FilterType>
TSharedPtr<SWidget> SDataprepStringFilter<FilterType>::OnGetContextMenuForMatchInArray()
{
	return FDataprepEditorUtils::MakeContextMenu( MatchInArrayParameterizationActionData );
}

template <class FilterType>
FReply SDataprepStringFilter<FilterType>::OnMatchInArrayClicked() 
{
	check( Filter );

	FScopedTransaction Transaction( LOCTEXT("MatchInArrayChangedTransaction","Changed match in array") );

	Filter->SetMatchInArray( !Filter->GetMatchInArray() );

	FProperty* Property = Filter->GetClass()->FindPropertyByName( TEXT("bMatchInArray") );
	check( Property );

	FEditPropertyChain EditChain;
	EditChain.AddHead( Property );
	EditChain.SetActivePropertyNode( Property );
	FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
	FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
	Filter->PostEditChangeChainProperty( EditChangeChainEvent );

	return FReply::Handled();
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType)
{
	check( Filter );
	UEnum* Enum = StaticEnum< EDataprepStringMatchType >();
	EDataprepStringMatchType StringMatchType = static_cast< EDataprepStringMatchType >( Enum->GetValueByIndex( ListEntry->Get<2>() ) );

	if ( StringMatchType != Filter->GetStringMatchingCriteria() )
	{	
		FScopedTransaction Transaction( LOCTEXT("SelectionCriteriaChangedTransaction","Changed the String Selection Criteria") );
		Filter->SetStringMatchingCriteria( StringMatchType );

		FProperty* Property = Filter->GetClass()->FindPropertyByName( TEXT("StringMatchingCriteria") );
		check( Property );

		FEditPropertyChain EditChain;
		EditChain.AddHead( Property );
		EditChain.SetActivePropertyNode( Property );
		FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
		FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
		Filter->PostEditChangeChainProperty( EditChangeChainEvent );
	}
}

template <class FilterType>
FText SDataprepStringFilter<FilterType>::GetUserString() const
{
	check( Filter );
	return FText::FromString( Filter->GetUserString() );
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnUserStringChanged(const FText& NewText)
{
	check( Filter );
	Filter->SetUserString( NewText.ToString() );
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::ExtendContextMenuForUserStringBox(FMenuBuilder& MenuBuilder)
{
	FDataprepEditorUtils::PopulateMenuForParameterization( MenuBuilder, *UserStringParameterizationActionData->DataprepAsset,
		*Filter, UserStringParameterizationActionData->PropertyChain );
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnUserStringComitted(const FText& NewText, ETextCommit::Type CommitType)
{
	check( Filter );
	FString NewUserString = NewText.ToString();
	if ( OldUserString != NewUserString )
	{
		Filter->SetUserString( OldUserString );
		FScopedTransaction Transaction( LOCTEXT("SelectionStringChangedTransaction","Changed the Selection String") );
		Filter->SetUserString( NewUserString );

		FProperty* Property = Filter->GetClass()->FindPropertyByName( TEXT("UserString") );
		check( Property );

		FEditPropertyChain EditChain;
		EditChain.AddHead( Property );
		EditChain.SetActivePropertyNode( Property );
		FPropertyChangedEvent EditPropertyChangeEvent( Property, EPropertyChangeType::ValueSet );
		FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
		Filter->PostEditChangeChainProperty( EditChangeChainEvent );

		OldUserString = NewUserString;
	}
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnUserStringArrayPropertyChanged(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	check( Filter );

	FEditPropertyChain EditChain;
	EditChain.AddHead( PropertyChangedChainEvent.Property );
	EditChain.SetActivePropertyNode( PropertyChangedChainEvent.Property );
	FPropertyChangedEvent EditPropertyChangeEvent( PropertyChangedChainEvent.Property, PropertyChangedChainEvent.ChangeType );
	FPropertyChangedChainEvent EditChangeChainEvent( EditChain, EditPropertyChangeEvent );
	Filter->PostEditChangeChainProperty( EditChangeChainEvent );
}

template <class FilterType>
TSharedPtr<SWidget> SDataprepStringFilter<FilterType>::OnGetContextMenuForUserString()
{
	return FDataprepEditorUtils::MakeContextMenu( UserStringParameterizationActionData );
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects)
{
	if ( !Objects || Objects->Contains( Filter ) )
	{
		UpdateVisualDisplay();
	}
}

template <class FilterType>
void SDataprepStringFilter<FilterType>::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( Filter );
}

// Explicit template instantiation
template class SDataprepStringFilter<UDataprepStringFilter>;
template class SDataprepStringFilter<UDataprepStringsArrayFilter>;

#undef LOCTEXT_NAMESPACE
