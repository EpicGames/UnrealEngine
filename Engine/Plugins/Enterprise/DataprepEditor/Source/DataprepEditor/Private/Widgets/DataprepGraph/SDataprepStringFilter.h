// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FMenuBuilder;
class SComboButton;

template<class t>
class SComboBox;

struct FDataprepParametrizationActionData;
class UDataprepParameterizableObject;

template <class FilterType>
class SDataprepStringFilter : public SCompoundWidget,  public FGCObject
{
	SLATE_BEGIN_ARGS(SDataprepStringFilter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, FilterType& InFilter);

	virtual ~SDataprepStringFilter();

private:

	void UpdateVisualDisplay();

	// The string matching option for the combo box ( Displayed text, Tooltip, mapping for the UEnum)
	using FListEntry = TTuple<FText, FText, int32>;

	// Those function are for the string matching criteria display
	TSharedRef<SWidget> OnGenerateWidgetForMatchingCriteria(TSharedPtr<FListEntry> ListEntry) const;
	FText GetSelectedCriteriaText() const;
	FText GetSelectedCriteriaTooltipText() const;
	void OnSelectedCriteriaChanged(TSharedPtr<FListEntry> ListEntry, ESelectInfo::Type SelectionType);
	void OnCriteriaComboBoxOpenning();
	TSharedPtr<SWidget> OnGetContextMenuForMatchingCriteria();
	TSharedPtr<SWidget> OnGetContextMenuForMatchInArray();

	// This function is for the string that will be compare against the fetched string
	FText GetUserString() const;
	void OnUserStringChanged(const FText& NewText);
	void OnUserStringComitted(const FText& NewText, ETextCommit::Type CommitType);
	void OnUserStringArrayPropertyChanged(UDataprepParameterizableObject& Object, FPropertyChangedChainEvent& PropertyChangedChainEvent);
	FReply OnMatchInArrayClicked();

	void ExtendContextMenuForUserStringBox(FMenuBuilder& MenuBuilder);
	TSharedPtr<SWidget> OnGetContextMenuForUserString();

	void OnParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects);

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepStringFilter");
	}
	//~ End FGCObject interface

	FString OldUserString;

	TObjectPtr<FilterType> Filter;

	TArray<TSharedPtr<FListEntry>> StringMatchingOptions;

	TSharedPtr<SComboBox<TSharedPtr<FListEntry>>> StringMatchingCriteriaWidget;

	TSharedPtr<FDataprepParametrizationActionData> MatchingCriteriaParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> UserStringParameterizationActionData;
	TSharedPtr<FDataprepParametrizationActionData> MatchInArrayParameterizationActionData;

	FDelegateHandle OnParameterizationStatusForObjectsChangedHandle;

	FDelegateHandle OnUserStringArrayPostEditHandle;
};
