// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepWidgets.h"

#include "DataprepAsset.h"
#include "DataprepAssetInstance.h"
#include "DataprepEditorUtils.h"
#include "SelectionSystem/DataprepStringFilter.h"
#include "Widgets/Parameterization/SDataprepParameterizationLinkIcon.h"

#include "DetailColumnSizeData.h"
#include "DetailLayoutBuilder.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SGridPanel.h"
#include "PropertyCustomizationHelpers.h"
#define LOCTEXT_NAMESPACE "DataprepSlateHelper"

namespace DataprepWidgetUtils
{
	class SCustomSplitter : public SSplitter
	{
	public:
		SLATE_BEGIN_ARGS(SCustomSplitter) {}
			SLATE_ARGUMENT( TSharedPtr<SWidget>, NameWidget )
			SLATE_ARGUMENT( TSharedPtr<SWidget>, ValueWidget )
			SLATE_ARGUMENT( TSharedPtr< FDetailColumnSizeData >, ColumnSizeData )
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs)
		{
			SSplitter::FArguments Args;

			SSplitter::Construct(Args
				.Style(FAppStyle::Get(), "DetailsView.Splitter")
				.PhysicalSplitterHandleSize(1.0f)
				.HitDetectionSplitterHandleSize(5.0f)
				.ResizeMode( ESplitterResizeMode::Fill ));

			ColumnSizeData = InArgs._ColumnSizeData;

			AddSlot()
			.Value(ColumnSizeData->GetNameColumnWidth())
			.OnSlotResized(ColumnSizeData->GetOnNameColumnResized())
			[
				InArgs._NameWidget.ToSharedRef()
			];

			AddSlot()
			.Value(InArgs._ColumnSizeData->GetValueColumnWidth())
			.OnSlotResized(ColumnSizeData->GetOnValueColumnResized())
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding( 5.0f, 2.5f, 2.0f, 2.5f )
				.HAlign( HAlign_Fill )
				.VAlign( VAlign_Fill )
				.FillWidth(1.f)
				[
					InArgs._ValueWidget.ToSharedRef()
				]
			];
		}

		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
		{
			if(Children.Num() > 1)
			{
				const SSplitter::FSlot& LeftChild = Children[0];

				const FVector2D AllottedSize = AllottedGeometry.GetLocalSize();

				FVector2D LocalPosition = FVector2D::ZeroVector;
				FVector2D LocalSize(AllottedSize.X * ColumnSizeData->GetNameColumnWidth().Get(0), AllottedSize.Y);

				ArrangedChildren.AddWidget( EVisibility::Visible, AllottedGeometry.MakeChild( LeftChild.GetWidget(), LocalPosition, LocalSize ));

				const SSplitter::FSlot& RightChild = Children[1];

				LocalPosition = FVector2D(LocalSize.X, 0.f);
				LocalSize.X = AllottedSize.X * ColumnSizeData->GetValueColumnWidth().Get(0);

				ArrangedChildren.AddWidget( EVisibility::Visible, AllottedGeometry.MakeChild( RightChild.GetWidget(), LocalPosition, LocalSize ));
			}
		}

	private:
		TSharedPtr< FDetailColumnSizeData > ColumnSizeData;
	};

	TSharedRef<SWidget> CreatePropertyWidget( TSharedPtr<SWidget> NameWidget, TSharedPtr<SWidget> ValueWidget, TSharedPtr< FDetailColumnSizeData > ColumnSizeData, float Spacing, bool bResizableColumn = true)
	{
		if (bResizableColumn)
		{
			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, Spacing)
			[
				SNew(SCustomSplitter)
				.NameWidget(NameWidget)
				.ValueWidget(ValueWidget)
				.ColumnSizeData(ColumnSizeData)
			];
		}
		else
		{
			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(0.0f, 0.0f, 0.0f, Spacing)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					NameWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SHorizontalBox)
					.Clipping(EWidgetClipping::OnDemand)
					+ SHorizontalBox::Slot()
					.Padding(5.0f, 2.5f, 2.0f, 2.5f)
					[
						SNew(DataprepWidgetUtils::SConstrainedBox)
						[
							ValueWidget.ToSharedRef()
						]
					]
				]
			];
		}
	}
}

int32 SDataprepDetailsView::DesiredWidth = 400.f;

void SDataprepCategoryWidget::ToggleExpansion()
{
	bIsExpanded = !bIsExpanded;
	CategoryContent->SetVisibility(bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed);
}

const FSlateBrush* SDataprepCategoryWidget::GetBackgroundImage() const
{
	if (IsHovered())
	{
		return bIsExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered") : FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
	} 
	else
	{
		return bIsExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop") : FAppStyle::GetBrush("DetailsView.CollapsedCategory");
	}
}

void SDataprepCategoryWidget::Construct( const FArguments& InArgs, TSharedRef< SWidget > InContent, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	const float MyContentTopPadding = 2.0f;
	const float MyContentBottomPadding = 2.0f;

	const float ChildSlotPadding = 2.0f;
	const float BorderVerticalPadding = 3.0f;

	CategoryContent = InContent;

	TSharedPtr<SWidget> TitleDetail = InArgs._TitleDetail;
	if( !TitleDetail.IsValid() )
	{
		TitleDetail = SNullWidget::NullWidget;
	}

	TSharedPtr<SHorizontalBox> TitleHeader = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign( VAlign_Center )
		.Padding( 2.0f, MyContentTopPadding, 2.0f, MyContentBottomPadding )
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
		]
		+ SHorizontalBox::Slot()
		.VAlign( VAlign_Center )
		.AutoWidth()
		.Padding( 0.0f, 8.0f )
		[
			SNew(STextBlock)
			.Text( InArgs._Title )
			.Font( FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle") )
			.ShadowOffset( FVector2D( 1.0f, 1.0f ) )
		];

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBorder )
			.BorderImage( this, &SDataprepCategoryWidget::GetBackgroundImage )
			.Padding( FMargin( 0.0f, BorderVerticalPadding, 16.0f, BorderVerticalPadding ) )
			.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth( 0.5f )
				.HAlign( EHorizontalAlignment::HAlign_Left )
				[
					TitleHeader.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.FillWidth( 0.5f )
				.HAlign( EHorizontalAlignment::HAlign_Right )
				[
					TitleDetail.ToSharedRef()
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			CategoryContent.ToSharedRef()
		]
	];

	STableRow< TSharedPtr< EDataprepCategory > >::ConstructInternal(
		STableRow::FArguments()
		.Style( FAppStyle::Get(), "DetailsView.TreeView.TableRow" )
		.ShowSelection( false ),
		InOwnerTableView
	);
}

void SDataprepDetailsView::CreateDefaultWidget( int32 Index, TSharedPtr< SWidget >& NameWidget, TSharedPtr< SWidget >& ValueWidget, float LeftPadding, const FDataprepParameterizationContext& ParameterizationContext, bool bAddExpander, bool bChildNode )
{
	TSharedRef<SHorizontalBox> NameColumn = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand);

	NameWidget->SetClipping( EWidgetClipping::OnDemand );

	if ( bAddExpander && StringArrayDetailedObject )
	{
		NameColumn->AddSlot()
		.AutoWidth()
		.VAlign( VAlign_Center )
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "FlatButton.Primary" )
			.ButtonColorAndOpacity( FLinearColor::Transparent )
			.ForegroundColor( FLinearColor::White )
			.ContentPadding( FMargin(6, 2) )
			.Cursor( EMouseCursor::Default )
			.Content()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text_Lambda( [this](){ return StringArrayDetailedObject->bExpanded ? FEditorFontGlyphs::Caret_Down: FEditorFontGlyphs::Caret_Right; } )
			]
			.OnClicked_Lambda( [this]()
			{
				StringArrayDetailedObject->bExpanded = !StringArrayDetailedObject->bExpanded;
				return (FReply::Handled()); 
			} )
		];
	}

	// Add the name widget
	NameColumn->AddSlot()
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Left)
	.Padding(FMargin(LeftPadding, 0.f, 0.f, 0.f))
	[
		NameWidget.ToSharedRef()
	];

	if ( ParameterizationContext.State == EParametrizationState::IsParameterized )
	{
		NameColumn->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(5.f, 0.f, 5.f, 0.f))
		.AutoWidth()
		[
			SNew(SDataprepParameterizationLinkIcon, DataprepAssetForParameterization.Get(), DetailedObjectAsParameterizable, ParameterizationContext.PropertyChain)
		];
	}

	FOnContextMenuOpening OnContextMenuOpening;

	if ( ParameterizationContext.State == EParametrizationState::IsParameterized || ParameterizationContext.State == EParametrizationState::CanBeParameterized )
	{
		OnContextMenuOpening.BindLambda( [this, InPropertyChain = ParameterizationContext.PropertyChain] () -> TSharedPtr<SWidget>
			{
				if ( UDataprepAsset * DataprepAsset = DataprepAssetForParameterization.Get() )
					{
					if ( DetailedObjectAsParameterizable )
						{
						FMenuBuilder MenuBuilder(true, nullptr);
						FDataprepEditorUtils::PopulateMenuForParameterization(MenuBuilder, *DataprepAsset, *DetailedObjectAsParameterizable, InPropertyChain);
						return MenuBuilder.MakeWidget();
						}
				}
				return TSharedPtr<SWidget>();
			});
	}

	GridPanel->AddSlot(0, Index)
	[
		SNew(SDataprepContextMenuOverride)
		.OnContextMenuOpening(OnContextMenuOpening)
		.Visibility_Lambda([this, bChildNode]() -> EVisibility 
		{ 
			if ( bChildNode && StringArrayDetailedObject )
			{
				return StringArrayDetailedObject->bExpanded ? EVisibility::Visible : EVisibility::Collapsed; 
			}
			return EVisibility::Visible; 
		})
		[
			DataprepWidgetUtils::CreatePropertyWidget( NameColumn, ValueWidget, ColumnSizeData, Spacing, bResizableColumn )
		]
	];

	if(bColumnPadding)
	{
		// Add two more columns to align parameter widget
		GridPanel->AddSlot(1, Index)
		.Padding(5.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.IsFocusable(false)
				.Visibility(EVisibility::Hidden)
				.IsEnabled(false)
				.VAlign(VAlign_Top)
				.Content()
				[
					SNew(STextBlock)
					.Font(FDataprepEditorUtils::GetGlyphFont())
					.ColorAndOpacity(FLinearColor::Transparent)
					.Text(FEditorFontGlyphs::Exclamation_Triangle)
				]
			]
		];

		GridPanel->AddSlot(2, Index)
		.Padding(5.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.IsFocusable(false)
				.Visibility(EVisibility::Hidden)
				.IsEnabled(false)
				.VAlign(VAlign_Top)
				.Content()
				[
					SNew(STextBlock)
					.Font(FDataprepEditorUtils::GetGlyphFont())
					.ColorAndOpacity(FLinearColor::Transparent)
					.Text(FEditorFontGlyphs::Exclamation_Triangle)
				]
			]
		];
	}
}

void SDataprepDetailsView::OnPropertyChanged(const FPropertyChangedEvent& InEvent)
{
	for (const TSharedPtr< IPropertyHandle >& PropertyHandle : TrackedProperties)
	{
		if (PropertyHandle->GetProperty() == InEvent.Property)
		{
			ForceRefresh();
			break;
		}
	}
}

void SDataprepDetailsView::OnObjectReplaced(const TMap<UObject*, UObject*>& ReplacementObjectMap)
{
	if ( UObject * const* ObjectPtr = ReplacementObjectMap.Find( DetailedObject ) )
	{
		DetailedObject = *ObjectPtr;
		if ( DetailedObject->IsA<UDataprepParameterizableObject>() )
		{
			DetailedObjectAsParameterizable = StaticCastPtr<UDataprepParameterizableObject>(DetailedObject);
		}

		ForceRefresh();
	}
}

void SDataprepDetailsView::ForceRefresh()
{
	if (Generator.IsValid())
	{
		Generator->SetObjects({ DetailedObject });
	}
	Construct();
}

void SDataprepDetailsView::OnDataprepParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects)
{
	if ( !Objects || Objects->Contains( DetailedObjectAsParameterizable ) )
	{
		ForceRefresh();
	}
}

void SDataprepDetailsView::OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& TransactionObjectEvent)
{
	// Hack to support refresh the parameterization display of a dataprep instance
	if ( Object == DetailedObject || ( DetailedObject && DetailedObject->GetOuter() == Object ) )
	{
		ForceRefresh();
	}
}

void SDataprepDetailsView::AddWidgets( const TArray< TSharedRef< IDetailTreeNode > >& DetailTree, int32& Index, float LeftPadding, const FDataprepParameterizationContext& InParameterizationContext, bool bChildNodes )
{
	auto IsDetailNodeDisplayable = []( const TSharedPtr< IPropertyHandle >& PropertyHandle)
	{
		if(PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() && PropertyHandle->IsEditable())
		{
			FProperty* Property = PropertyHandle->GetProperty();

			if ( Property )
			{
				if ( FFieldVariant Outer = Property->GetOwnerVariant() )
				{
					// if the outer is a container property (array,set or map) it's editable even without the proper flags.
					//UClass* OuterClass = Outer->GetClass();
					if (Outer.IsA<FArrayProperty>() || Outer.IsA<FSetProperty>() || Outer.IsA<FMapProperty>() )
					{
						return true;
					}
				}

				return Property && !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && Property->HasAnyPropertyFlags(CPF_Edit);
			}
		}

		// Ok to display DetailNode without property because at this stage the parent property was displayable
		return true;
	};

	auto IsDetailNodeDisplayableContainerProperty = []( const TSharedPtr< IPropertyHandle >& PropertyHandle)
	{
		if( PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() && PropertyHandle->IsEditable() )
		{
			if ( FProperty* Property = PropertyHandle->GetProperty() )
			{ 
				FFieldClass* PropertyClass = Property->GetClass();
				if ( PropertyClass == FArrayProperty::StaticClass() || PropertyClass == FSetProperty::StaticClass() || PropertyClass == FMapProperty::StaticClass() )
				{
					return !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && Property->HasAnyPropertyFlags(CPF_Edit);
				}
			}
		}

		return false;
	};

	for( const TSharedRef< IDetailTreeNode >& ChildNode : DetailTree )
	{
		TSharedPtr< IPropertyHandle > PropertyHandle = ChildNode->CreatePropertyHandle();
		FDataprepParameterizationContext CurrentParameterizationContext = FDataprepParameterizationUtils::CreateContext( PropertyHandle, InParameterizationContext );
		if ( CurrentParameterizationContext.State == EParametrizationState::CanBeParameterized )
		{
			if ( UDataprepAsset* DataprepAsset = DataprepAssetForParameterization.Get() )
			{
				if ( DataprepAsset->IsObjectPropertyBinded( DetailedObjectAsParameterizable, CurrentParameterizationContext.PropertyChain ) )
				{
					CurrentParameterizationContext.State = EParametrizationState::IsParameterized;
				}
			}
		}


		if( ChildNode->GetNodeType() == EDetailNodeType::Category )
		{
			if( Index > 0 )
			{
				GridPanel->AddSlot( 0, Index )
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SSpacer)
						.Size( FVector2D( 0.f, 10.f ) )
					]
				];
				Index++;
			}

			TArray< TSharedRef< IDetailTreeNode > > Children;
			ChildNode->GetChildren( Children );
			AddWidgets( Children, Index, LeftPadding, CurrentParameterizationContext );
		}
		else if( IsDetailNodeDisplayableContainerProperty( PropertyHandle ) )
		{
			TSharedPtr< IDetailPropertyRow > DetailPropertyRow = ChildNode->GetRow();
			if( DetailPropertyRow.IsValid() )
			{
				FDetailWidgetRow Row;
				TSharedPtr< SWidget > NameWidget;
				TSharedPtr< SWidget > ValueWidget;
				DetailPropertyRow->GetDefaultWidgets( NameWidget, ValueWidget, Row, true );

				TArray< TSharedRef< IDetailTreeNode > > Children;
				ChildNode->GetChildren( Children );

				CreateDefaultWidget( Index, NameWidget, ValueWidget, LeftPadding, CurrentParameterizationContext, Children.Num() > 0, false );
				Index++;

				if( Children.Num() > 0 )
				{
					// #ueent_todo: Find a way to add collapse/expand capability for property with children
					AddWidgets( Children, Index, LeftPadding + 10.f, CurrentParameterizationContext, true );
				}

				TrackedProperties.Add( PropertyHandle );
			}
		}
		else if( IsDetailNodeDisplayable( PropertyHandle ) )
		{
			TSharedPtr< SWidget > NameWidget;
			TSharedPtr< SWidget > ValueWidget;
			EHorizontalAlignment HAlign;
			EVerticalAlignment VAlign;

			TSharedPtr< IDetailPropertyRow > DetailPropertyRow = ChildNode->GetRow();
			if( DetailPropertyRow.IsValid() )
			{
				FDetailWidgetRow Row;
				DetailPropertyRow->GetDefaultWidgets( NameWidget, ValueWidget, Row, true );
				HAlign = Row.ValueWidget.HorizontalAlignment;
				VAlign = Row.ValueWidget.VerticalAlignment;
			}
			else
			{
				FNodeWidgets NodeWidgets = ChildNode->CreateNodeWidgets();

				NameWidget = NodeWidgets.NameWidget;
				ValueWidget = NodeWidgets.ValueWidget;
				HAlign = NodeWidgets.ValueWidgetLayoutData.HorizontalAlignment;
				VAlign = NodeWidgets.ValueWidgetLayoutData.VerticalAlignment;
			}

			if( NameWidget.IsValid() && ValueWidget.IsValid() )
			{
				CreateDefaultWidget( Index, NameWidget, ValueWidget, LeftPadding, CurrentParameterizationContext, false, bChildNodes );
				Index++;

				bool bDisplayChildren = true;

				// Do not display children if the property is a FVector or FVector2D
				if( PropertyHandle.IsValid() )
				{
					FVector DummyVec;
					FVector2D DummyVec2D;

					bDisplayChildren &= PropertyHandle->GetValue( DummyVec ) == FPropertyAccess::Fail;
					bDisplayChildren &= PropertyHandle->GetValue( DummyVec2D ) == FPropertyAccess::Fail;
				}

				TArray< TSharedRef< IDetailTreeNode > > Children;
				ChildNode->GetChildren( Children );
				if( bDisplayChildren && Children.Num() > 0 )
				{
					// #ueent_todo: Find a way to add collapse/expand capability for property with children
					AddWidgets( Children, Index, LeftPadding + 10.f, CurrentParameterizationContext );
				}
			}
		}
	}
}

void SDataprepDetailsView::Construct(const FArguments& InArgs)
{
	DetailedObject = InArgs._Object;
	Spacing = InArgs._Spacing;
	bColumnPadding = InArgs._ColumnPadding;
	bResizableColumn = InArgs._ResizableColumn;

	StringArrayDetailedObject = Cast<UDataprepStringFilterMatchingArray>( DetailedObject );

	if ( InArgs._ColumnSizeData.IsValid() )
	{
		ColumnSizeData = InArgs._ColumnSizeData;
	}
	else
	{
		ColumnSizeData = MakeShared<FDetailColumnSizeData>();
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FPropertyRowGeneratorArgs Args;
	Generator = PropertyEditorModule.CreatePropertyRowGenerator(Args);

	if ( DetailedObject != nullptr )
	{
		TArray< UObject* > Objects;
		Objects.Add( DetailedObject );
		Generator->SetObjects( Objects );
	}

	OnPropertyChangedHandle = Generator->OnFinishedChangingProperties().AddSP( this, &SDataprepDetailsView::OnPropertyChanged );

	OnObjectReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SDataprepDetailsView::OnObjectReplaced);
	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddSP( this, &SDataprepDetailsView::OnObjectTransacted );

	Construct();
}

void SDataprepDetailsView::Construct()
{
	TrackedProperties.Reset();
	DataprepAssetForParameterization.Reset();

	if ( DetailedObject )
	{
		UDataprepAsset* DataprepAsset = FDataprepParameterizationUtils::GetDataprepAssetForParameterization( DetailedObject );
		if ( DataprepAsset )
		{
			OnDataprepParameterizationStatusForObjectsChangedHandle = DataprepAsset->OnParameterizedObjectsStatusChanged.AddSP( this, &SDataprepDetailsView::OnDataprepParameterizationStatusForObjectsChanged );
		}

		if ( DetailedObject->IsA<UDataprepParameterizableObject>() )
		{
			DetailedObjectAsParameterizable = StaticCastPtr<UDataprepParameterizableObject>(DetailedObject);
		}

		FDataprepParameterizationContext ParameterizationContext;
		ParameterizationContext.State = DataprepAsset && DetailedObjectAsParameterizable ? EParametrizationState::CanBeParameterized : EParametrizationState::InvalidForParameterization;
		DataprepAssetForParameterization = DataprepAsset;

		GridPanel = SNew(SGridPanel).FillColumn( 0.0f, 1.0f );
		TArray< TSharedRef< IDetailTreeNode > > RootNodes = Generator->GetRootTreeNodes();

		int32 Index = 0;
		AddWidgets(RootNodes, Index, 0.f, ParameterizationContext);

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(1.f, 5.0f)
			[
				GridPanel.ToSharedRef()
			]
		];
	}
	else
	{
		FText ErrorText = LOCTEXT( "DataprepSlateHelper_InvalidDetailedObject", "Error: Not a valid Object" );

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Font( IDetailLayoutBuilder::GetDetailFontBold() )
					.Text( ErrorText )
					.Margin(FMargin( 5.0f, 5.0f, 0.0f, 0.0f ) )
					.ColorAndOpacity( FLinearColor(1,0,0,1) )
				]
			]
		];
	}
}

SDataprepDetailsView::~SDataprepDetailsView()
{
	Generator->OnFinishedChangingProperties().Remove( OnPropertyChangedHandle );

	FCoreUObjectDelegates::OnObjectsReplaced.Remove( OnObjectReplacedHandle );
	FCoreUObjectDelegates::OnObjectTransacted.Remove( OnObjectTransactedHandle );

	if ( UDataprepAsset* DataprepAsset = DataprepAssetForParameterization.Get() )
	{
		DataprepAsset->OnParameterizedObjectsStatusChanged.Remove( OnDataprepParameterizationStatusForObjectsChangedHandle );
	}
}

void SDataprepDetailsView::SetObjectToDisplay(UObject& Object)
{
	UObject* NewObjectToDisplay = &Object;
	if ( DetailedObject != NewObjectToDisplay )
	{
		DetailedObject = NewObjectToDisplay;
		if ( DetailedObject->IsA<UDataprepParameterizableObject>() )
		{
			DetailedObjectAsParameterizable = StaticCastPtr<UDataprepParameterizableObject>(DetailedObject);
		}
		ForceRefresh();
	}
}

void SDataprepDetailsView::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( DetailedObject );
	Collector.AddReferencedObject( DetailedObjectAsParameterizable );
	for (const TSharedPtr< IPropertyHandle >& PropertyHandle : TrackedProperties)
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			Property->AddReferencedObjects(Collector);
		}
	}
}

void SDataprepContextMenuOverride::Construct(const FArguments& InArgs)
{
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	
	ChildSlot
	[
		InArgs._DefaultSlot.Widget
	];
}

FReply SDataprepContextMenuOverride::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnContextMenuOpening.IsBound() )
	{
		TSharedPtr<SWidget> ContextMenu = OnContextMenuOpening.Execute();
		if ( ContextMenu )
		{ 
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(
				AsShared(),
				WidgetPath,
				ContextMenu.ToSharedRef(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SDataprepInstanceParentWidget::Construct(const FArguments& InArgs)
{
	DataprepInstancePtr = InArgs._DataprepInstance;
	if(!DataprepInstancePtr.IsValid())
	{
		return;
	}

	ColumnSizeData = InArgs._ColumnSizeData;

	TSharedRef<SWidget> NameWidget = SNew(SHorizontalBox)
	.Clipping(EWidgetClipping::OnDemand)
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
	[
		SNew( STextBlock )
		.Text( LOCTEXT("DataprepInstanceParentWidget_Parent_Label", "Parent") )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];

	TSharedRef<SWidget> ValueWidget = SNew( SObjectPropertyEntryBox )
	.AllowedClass( UDataprepAssetInterface::StaticClass() )
	.OnObjectChanged( this, &SDataprepInstanceParentWidget::SetDataprepInstanceParent )
	.OnShouldFilterAsset( this, &SDataprepInstanceParentWidget::ShouldFilterAsset )
	.ObjectPath( this, &SDataprepInstanceParentWidget::GetDataprepInstanceParent );

	// The widget is disabled as the workflow to change the parent asset of a Dataprep instance is refined
	ValueWidget->SetEnabled( false );

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size( FVector2D( 200, 10 ) )		
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				DataprepWidgetUtils::CreatePropertyWidget( NameWidget, ValueWidget, ColumnSizeData, 0.0f )
			]
		]
	];
}

void SDataprepInstanceParentWidget::SetDataprepInstanceParent(const FAssetData& InAssetData)
{
	if(UDataprepAssetInstance* DataprepInstance = DataprepInstancePtr.Get())
	{
		if ( UDataprepAsset* DataprepAsset = Cast<UDataprepAsset>( InAssetData.GetAsset() ) )
		{
			const FScopedTransaction Transaction( LOCTEXT("DataprepInstance_SetParent", "Set Parent") );
			DataprepInstance->SetParent( DataprepAsset );
		}
	}
}

FString SDataprepInstanceParentWidget::GetDataprepInstanceParent() const
{
	FString PathName;

	if(UDataprepAssetInstance* DataprepInstance = DataprepInstancePtr.Get())
	{
		if(DataprepInstance->GetParent())
		{
			PathName = DataprepInstance->GetParent()->GetPathName();
		}
	}

	return PathName;
}

bool SDataprepInstanceParentWidget::ShouldFilterAsset(const FAssetData& InAssetData)
{
	if(UDataprepAssetInstance* DataprepInstance = DataprepInstancePtr.Get())
	{
		if ( InAssetData.GetClass() == UDataprepAssetInterface::StaticClass() )
		{
			FAssetData CurrentAssetData(DataprepInstance->GetParent());
			return CurrentAssetData != InAssetData;
		}
	}

	return false;
}

TSharedRef<SWidget> DataprepWidgetUtils::CreateParameterRow( TSharedPtr<SWidget> ParameterWidget )
{
	return  SNew(SGridPanel)
	.FillColumn(0, 1.0f)
	+ SGridPanel::Slot(0, 0)
	.Padding(10.0f, 5.0f, 0.0f, 5.0f)
	[
		ParameterWidget.ToSharedRef()
	]
	// Add two more columns to align parameter widget
	+ SGridPanel::Slot(1, 0)
	.Padding(5.0f, 5.0f, 0.0f, 5.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SButton)
			.IsFocusable(false)
			.Visibility(EVisibility::Hidden)
			.IsEnabled(false)
			.VAlign(VAlign_Top)
			.Content()
			[
				SNew(STextBlock)
				.Font(FDataprepEditorUtils::GetGlyphFont())
				.ColorAndOpacity(FLinearColor::Transparent)
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
		]
	]
	+ SGridPanel::Slot(2, 0)
	.Padding(5.0f, 5.0f, 0.0f, 5.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SButton)
			.IsFocusable(false)
			.Visibility(EVisibility::Hidden)
			.IsEnabled(false)
			.VAlign(VAlign_Top)
			.Content()
			[
				SNew(STextBlock)
				.Font(FDataprepEditorUtils::GetGlyphFont())
				.ColorAndOpacity(FLinearColor::Transparent)
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
