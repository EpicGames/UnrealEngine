// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h" // IWYU pragma: keep

class IPropertyHandle;
class STableViewBase;
struct FDataprepParameterizationContext;
template <typename ItemType> class STreeView;

class SEditableTextBox;
class SGridPanel;
class UDataprepAsset;
class UDataprepAssetInstance;
class UDataprepContentConsumer;
class UDataprepParameterizableObject;

struct FAssetData;
class UDataprepStringFilterMatchingArray;
class FDetailColumnSizeData;

namespace DataprepWidgetUtils
{
	TSharedRef<SWidget> CreateParameterRow( TSharedPtr<SWidget> ParameterWidget );

	/** Helper class to force a widget to fill in a space. Copied from SDetailSingleItemRow.cpp */
	class SConstrainedBox : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SConstrainedBox) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs)
		{
			ChildSlot
				[
					InArgs._Content.Widget
				];
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
		{
			// Voluntarily ridiculously large value to force the child widget to fill up the available space
			const float MinWidthVal = 2000;
			const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();
			return FVector2D(FMath::Max(MinWidthVal, ChildSize.X), ChildSize.Y);
		}
	};
}

enum class EDataprepCategory 
{ 
	Producers, 
	Consumers, 
	Parameterization 
};

typedef STreeView< TSharedRef<EDataprepCategory> > SDataprepCategoryTree;

class SDataprepCategoryWidget : public STableRow< TSharedPtr< EDataprepCategory > >
{
public:

	SLATE_BEGIN_ARGS(SDataprepCategoryWidget) {}
		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(TSharedPtr< SWidget >, TitleDetail)
		SLATE_ARGUMENT(TSharedPtr< FDetailColumnSizeData >, ColumnSizeData)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedRef< SWidget > InContent, const TSharedRef<STableViewBase>& InOwnerTableView );

	virtual int32 DoesItemHaveChildren() const override { return 1; }
	virtual bool IsItemExpanded() const override { return bIsExpanded; }
	virtual void ToggleExpansion() override;

	const FSlateBrush* GetBackgroundImage() const;
private:
	bool bIsExpanded = true;
	TSharedPtr< SWidget > CategoryContent;
};

class SDataprepDetailsView : public SCompoundWidget, public FGCObject
{
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SDataprepDetailsView)
		: _Object( nullptr )
		, _Spacing(0)
		, _ColumnPadding(false)
		, _ResizableColumn(true)
		{}
		SLATE_ARGUMENT(UObject*, Object)
		SLATE_ARGUMENT(float, Spacing)
		SLATE_ARGUMENT(bool, ColumnPadding)
		SLATE_ARGUMENT(bool, ResizableColumn)
		SLATE_ARGUMENT(TSharedPtr< FDetailColumnSizeData >, ColumnSizeData)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	~SDataprepDetailsView();

	void SetObjectToDisplay(UObject& Object);

	void ForceRefresh();

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDataprepDetailsView");
	}
	// End of FGCObject interface

	static int32 GetDesiredWidth() { return DesiredWidth; }
	static void SetDesiredWidth(int32 InDesiredWidth) { DesiredWidth = InDesiredWidth; }

protected:
	// Begin SWidget overrides.
	// #ueent_todo: This is temporary until we find a better solution to the splitter issue
	//				See SConstrainedBox's trick in cpp file
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();
		return FVector2D( DesiredWidth, ChildSize.Y );
	}
	// End SWidget overrides.

private:
	/** Fills up the details view with the detail nodes created by the property row manager */
	void Construct();

	/** Add widgets held by array of DetailTreeNode objects */
	void AddWidgets( const TArray< TSharedRef< class IDetailTreeNode > >& DetailTree, int32& Index, float LeftPadding, const FDataprepParameterizationContext& ParameterizationContext, bool bChildNodes = false );

	/**
	 * Inserts a generic widget for a property row into the grid panel
	 * @param Index						Row index in the grid panel
	 * @param NameWidget				The widget used in the name column
	 * @param ValueWidget				The widget used in the value column
	 * @param LeftPadding				The Padding on the left of the row
	 * @param ParameterizationContext	Parameterization context of the associated property
	 */
	void CreateDefaultWidget(int32 Index, TSharedPtr< SWidget >& NameWidget, TSharedPtr< SWidget >& ValueWidget, float LeftPadding, const FDataprepParameterizationContext& ParameterizationContext, bool bAddExpander = false, bool bChildNode = false );

	/** Callback to track property changes on array properties */
	void OnPropertyChanged( const struct FPropertyChangedEvent& InEvent );

	/** Callback used to detect the existence of a new object to display after a reinstancing process */
	void OnObjectReplaced(const TMap<UObject*, UObject*>& ReplacementObjectMap);

	void OnDataprepParameterizationStatusForObjectsChanged(const TSet<UObject*>* Objects);

	void OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& TransactionObjectEvent);

private:
	/** Row generator applied on detailed object */
	TSharedPtr< class IPropertyRowGenerator > Generator;

	/** Object to be detailed */
	TObjectPtr<UObject> DetailedObject;

	UDataprepStringFilterMatchingArray* StringArrayDetailedObject;

	/** Not null if the detailed object is parameterizable */
	TObjectPtr<UDataprepParameterizableObject> DetailedObjectAsParameterizable;

	/** Array properties tracked for changes */
	TSet< TSharedPtr< IPropertyHandle > > TrackedProperties;
	
	/** Delegate handle to track property changes on array properties */
	FDelegateHandle OnPropertyChangedHandle;

	/** Delegate handle to track new object after a re-instancing process */
	FDelegateHandle OnObjectReplacedHandle;

	/** Delegate handle to track when a object was transacted */
	FDelegateHandle OnObjectTransactedHandle;

	/** Points to the currently used column size data. Can be provided via argument as well. */
	TSharedPtr< FDetailColumnSizeData > ColumnSizeData;

	/**
	 * Pointer to a Dataprep asset used by the parameterization system.
	 * It should be nullptr when the parameterization shouldn't be shown
	 */
	TWeakObjectPtr<UDataprepAsset> DataprepAssetForParameterization;

	/** Callback used when parameterization has changed */
	FDelegateHandle OnDataprepParameterizationStatusForObjectsChangedHandle;

	/** Spacing between rows of the SDataprepDetailsView. 0 by default. */
	float Spacing;

	/**
	 * Indicates if two columns should be added
	 * This is useful when the SDataprepDetailsView widget is used along the Producers widget
	 */
	bool bColumnPadding;

	/**
	 * Indicates if the name and value widgets have a splitter and can be resized
	 */
	bool bResizableColumn;

	/** Grid panel storing the row widgets */
	TSharedPtr<SGridPanel> GridPanel;


	static int32 DesiredWidth;
};

/**
 * This is widget simply exist to open contextual menu
 */
class SDataprepContextMenuOverride : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepContextMenuOverride) {}
		SLATE_DEFAULT_SLOT(FArguments, DefaultSlot)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	FOnContextMenuOpening OnContextMenuOpening;
};

// Widget to expose parent asset of a Dataprep instance
class SDataprepInstanceParentWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepInstanceParentWidget) {}
		SLATE_ARGUMENT(TSharedPtr< FDetailColumnSizeData >, ColumnSizeData)
		SLATE_ARGUMENT(UDataprepAssetInstance*, DataprepInstance)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void SetDataprepInstanceParent(const FAssetData& InAssetData);
	FString GetDataprepInstanceParent() const;
	bool ShouldFilterAsset(const FAssetData& InAssetData);

	/** Callbacks to update splitter */
	void OnLeftColumnResized(float InNewWidth)
	{
		// This has to be bound or the splitter will take it upon itself to determine the size
		// We do nothing here because it is handled by the column size data
	}

	float OnGetLeftColumnWidth() const { return 1.0f - ColumnWidth; }
	float OnGetRightColumnWidth() const { return ColumnWidth; }
	void OnSetColumnWidth(float InWidth)  { ColumnWidth = InWidth < 0.5f ? 0.5f : InWidth; }

private:
	/** Weak pointer on DataprepAsset instance */
	TWeakObjectPtr<UDataprepAssetInstance> DataprepInstancePtr;

	/** Helps sync column resizing with another part of the UI */
	TSharedPtr< FDetailColumnSizeData > ColumnSizeData;

	/** Relative width to control splitters. */
	float ColumnWidth;
};
