// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Parameterization/SDataprepLinkToParameter.h"

#include "DataprepAsset.h"
#include "DataprepEditorUtils.h"

#include "Async/ParallelFor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
 
#define LOCTEXT_NAMESPACE "SDataprepLinkToParameter"

FText SDataprepLinkToParameter::EmptyNameErrorText = LOCTEXT("EmptyNameErrorText", "The parameter name can't be an empty string");
FText SDataprepLinkToParameter::InvalidNameErrorText = LOCTEXT("InvalidNameErrorText", "The parameter name is already used for another data type");

FText SDataprepLinkToParameter::LinkToParameterTransactionText = LOCTEXT("LinkedParameterToParameterization", "Linked property to parameterization");

void SDataprepLinkToParameter::Construct(const FArguments& InArgs, const TSharedRef<FDataprepParametrizationActionData>& InParameterizationActionData)
{
	ParameterizationActionData = InParameterizationActionData;

	
	if ( ParameterizationActionData->IsValid() )
	{
		if ( FProperty* Property = ParameterizationActionData->PropertyChain.Last().CachedProperty.Get() )
		{
			bool bIsDescribingFullProperty = ParameterizationActionData->PropertyChain.Last().ContainerIndex == INDEX_NONE;
			ParameterizationActionData->DataprepAsset->GetExistingParameterNamesForType( Property, bIsDescribingFullProperty , ValidExistingNames, InvalidNames );

			FString SuggestedName = ParameterizationActionData->PropertyChain.Last().PropertyName.ToString();

			bool bIsSuggestedNameValid = !SuggestedName.IsEmpty() && !InvalidNames.Contains( SuggestedName );
			bool bIsSuggestedNameAParameter = ValidExistingNames.Contains( SuggestedName );

			ShownItems.Reserve( ValidExistingNames.Num() + 1 );
			CreateNewOption = MakeShared<FString>( LOCTEXT("CreateNewParamerterLabel", "Create New Parameter").ToString() );
			if ( bIsSuggestedNameValid && !bIsSuggestedNameAParameter )
			{
				ShownItems.Add( CreateNewOption );
			}

			UnfilteredSuggestions.Reserve( ValidExistingNames.Num() );
			for ( const FString& String : ValidExistingNames )
			{
				UnfilteredSuggestions.Add( MakeShared<FString>( String ) );
			}

			ShownItems.Append( UnfilteredSuggestions );


			ChildSlot
			[
				SNew( SBox )
				.MinDesiredWidth( 400 )
				.MaxDesiredHeight( 400 )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.HAlign( HAlign_Fill )
					.VAlign( VAlign_Top )
					.AutoHeight()
					[
						SAssignNew( TextBox, SEditableTextBox )
						.Text( FText::FromString( SuggestedName ) )
						.HintText( LOCTEXT("ParameterNameHintText", "Enter a parameter name") )
						.OnTextChanged( this, &SDataprepLinkToParameter::OnTextChanged )
						.OnTextCommitted( this, &SDataprepLinkToParameter::OnTextCommited )
					]
					+ SVerticalBox::Slot()
					.HAlign( HAlign_Fill )
					.VAlign( VAlign_Top )
					.Padding( FMargin( 0.f, 5.f, 0.f, 0.f ) )
					[
						SAssignNew( SuggestionList, SListView<TSharedPtr<FString>> )
						.ListItemsSource( &ShownItems )
						.OnGenerateRow( this, &SDataprepLinkToParameter::OnGenerateRowForList )
						.SelectionMode( ESelectionMode::Single )
						.OnSelectionChanged( this, &SDataprepLinkToParameter::OnSelectionChanged )
					]
				]
			];

			if ( !bIsSuggestedNameValid )
			{
				if ( SuggestedName.IsEmpty() )
				{
					TextBox->SetError( EmptyNameErrorText.ToString() );
				}
				else
				{
					TextBox->SetError( InvalidNameErrorText.ToString() );
				}
			}
		}
	}
}

TSharedRef<class ITableRow> SDataprepLinkToParameter::OnGenerateRowForList(TSharedPtr<FString> Item, const TSharedRef<class STableViewBase>& OwnerTable)
{
	if ( Item == CreateNewOption )
	{
		return SNew( STableRow<TSharedPtr<FString>>, OwnerTable )
			[
				SNew( STextBlock )
				.Text( FText::FromString(*Item) )
				.TextStyle( &FCoreStyle::Get(), ISlateStyle::Join("ToolBar", ".Label") )
				.Margin( FMargin(2.f) )
			];
	}

	return SNew( STableRow<TSharedPtr<FString>>, OwnerTable )
		[
			SNew( STextBlock )
			.Text( FText ::FromString( *Item ) )
			.TextStyle( &FCoreStyle::Get(), ISlateStyle::Join( "ToolBar", ".Label" ) )
			.HighlightText_Lambda( [this]() { return FText::FromString( ParameterName ); } )
			.Margin( FMargin( 2.f ) )
		];
}

void SDataprepLinkToParameter::OnSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType)
{
	FScopedTransaction Transaction( LinkToParameterTransactionText );
	if ( InItem == CreateNewOption )
	{
		ParameterName = TextBox->GetText().ToString();
		ParameterizationActionData->DataprepAsset->BindObjectPropertyToParameterization( ParameterizationActionData->Object, ParameterizationActionData->PropertyChain, FName( *ParameterName ) );
	}
	else
	{
		ParameterizationActionData->DataprepAsset->BindObjectPropertyToParameterization( ParameterizationActionData->Object, ParameterizationActionData->PropertyChain, FName( *( *InItem.Get() ) ) );
	}

	FSlateApplication::Get().DismissAllMenus();
}

void SDataprepLinkToParameter::OnTextChanged(const FText& Text)
{
	ParameterName = Text.ToString();
	bool bHasError = SetErrorMessage();

	// Update the shown item list
	
	ShownItems.Empty( ValidExistingNames.Num() + 1 );

	if ( !bHasError && !ValidExistingNames.Contains( ParameterName ) )
	{
		ShownItems.Add( CreateNewOption );
	}

	TArray<bool> FilterResult;
	FilterResult.AddZeroed( UnfilteredSuggestions.Num() );

	ParallelFor( UnfilteredSuggestions.Num(), [this, &FilterResult](int32 Index)
		{
			FilterResult[Index] = UnfilteredSuggestions[Index]->Contains( ParameterName );
		});

	for ( int32 Index = 0; Index < FilterResult.Num(); Index++ )
	{
		if ( FilterResult[Index] )
		{
			ShownItems.Add( UnfilteredSuggestions[Index] );
		}
	}

	SuggestionList->RequestListRefresh();
}

void SDataprepLinkToParameter::OnTextCommited(const FText& Text, ETextCommit::Type CommitType)
{
	ParameterName = Text.ToString();
	bool bHasError = ParameterName.IsEmpty() || InvalidNames.Contains( ParameterName );
	
	if ( CommitType == ETextCommit::OnEnter )
	{
		if ( !bHasError &&  ParameterizationActionData->IsValid() )
		{
			FScopedTransaction Transaction( LinkToParameterTransactionText );
			ParameterizationActionData->DataprepAsset->BindObjectPropertyToParameterization( ParameterizationActionData->Object, ParameterizationActionData->PropertyChain, FName( *ParameterName ) );
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SDataprepLinkToParameter::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ( !bHadFirstTick )
	{
		FSlateApplication::Get().SetKeyboardFocus( TextBox, EFocusCause::SetDirectly );
		TextBox->SelectAllText();
		bHadFirstTick = true;
	}
}

bool SDataprepLinkToParameter::SetErrorMessage()
{
	if ( ParameterName.IsEmpty() )
	{
		TextBox->SetError( EmptyNameErrorText.ToString() );
		return true;
	}

	if ( InvalidNames.Contains( ParameterName ) )
	{
		TextBox->SetError( InvalidNameErrorText.ToString() );
		return true;
	}

	TextBox->SetError( FString() );
	return false;
}

#undef LOCTEXT_NAMESPACE
