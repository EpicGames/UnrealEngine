// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class SListView;

class UDataprepAsset;
class UDataprepParameterizableObject;

struct FDataprepParametrizationActionData;

class SDataprepLinkToParameter : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepLinkToParameter) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedRef<FDataprepParametrizationActionData>& InParameterizationActionData);

private:

	TSharedRef<class ITableRow> OnGenerateRowForList(TSharedPtr<FString> Item, const TSharedRef<class STableViewBase>& OwnerTable);

	void OnSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType);

	void OnTextChanged(const FText& Text);

	void OnTextCommited(const FText& Text, ETextCommit::Type CommitType);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Return true if there is a error */
	bool SetErrorMessage();

	// The information that will be used to create the binding
	TSharedPtr<FDataprepParametrizationActionData> ParameterizationActionData;

	TSharedPtr<FString> CreateNewOption;
	TArray<TSharedPtr<FString>> UnfilteredSuggestions;

	// The items currently shown
	TArray<TSharedPtr<FString>> ShownItems;

	// The names that can be use because they are used for other property types
	TSet<FString> InvalidNames;

	TSet<FString> ValidExistingNames;

	TSharedPtr<class SEditableTextBox> TextBox;
	TSharedPtr<class SListView<TSharedPtr<FString>>> SuggestionList;

	// The current parameter name entered by the user
	FString ParameterName;

	static FText EmptyNameErrorText;
	static FText InvalidNameErrorText;

	static FText LinkToParameterTransactionText;

	// Minor hack to set the focus on the text box
	bool bHadFirstTick = false;
};

