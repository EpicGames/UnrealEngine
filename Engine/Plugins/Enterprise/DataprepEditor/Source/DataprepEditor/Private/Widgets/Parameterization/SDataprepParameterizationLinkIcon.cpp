// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Parameterization/SDataprepParameterizationLinkIcon.h"

#include "DataprepAsset.h"

#include "EditorFontGlyphs.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDataprepParameterizationLinkIcon"

void SDataprepParameterizationLinkIcon::Construct(const FArguments& InArgs, UDataprepAsset* DataprepAsset, UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain)
{
	if ( DataprepAsset )
	{
		FName ParameterName = DataprepAsset->GetNameOfParameterForObjectProperty( Object, PropertyChain );

		if ( !ParameterName.IsNone() )
		{
			ChildSlot
			[
					SNew( STextBlock )
					.Font( FAppStyle::Get().GetFontStyle("FontAwesome.11") )
					.Text( FEditorFontGlyphs::Link )
					.Justification( ETextJustify::Center )
					.ColorAndOpacity( FColor( 170, 238, 160 ) )
					.ToolTipText( FText::Format( LOCTEXT("LinkIconTooltipText", "This property is linked to the parameter {0}"), FText::FromName( ParameterName ) ) )
			];
		}
	}
}

#undef LOCTEXT_NAMESPACE
