// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UDataprepAsset;
class UDataprepParameterizableObject;

struct FDataprepPropertyLink;

/**
 * This is widget serves to display some information about a binded property to the dataprep parameterization
 */
class SDataprepParameterizationLinkIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepParameterizationLinkIcon) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UDataprepAsset* DataprepAsset, UDataprepParameterizableObject* Object, const TArray<FDataprepPropertyLink>& PropertyChain);
};
