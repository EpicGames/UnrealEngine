// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetsPreviewWidget.h"

#include "DataprepContentConsumer.h"

#include "AssetToolsModule.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IAssetTypeActions.h"
#include "IAssetTools.h"
#include "SAssetSearchBox.h"

#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "AssetPreviewWidget"

namespace AssetPreviewWidget
{
	/** This is the default column of the asset preview it contains the label and icon of the assets and folder */
	class FAssetPreviewDefaultColumn : public IAssetPreviewColumn
	{
	public:

		FAssetPreviewDefaultColumn()
		{
			FolderOpenBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderOpen");
			FolderClosedBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderClosed");
			AssetIconBrush = FAppStyle::GetBrush("ContentBrowser.ColumnViewAssetIcon");
		}


		virtual uint8 GetCulumnPositionPriorityIndex() const override
		{
			return 128;
		}

		virtual FName GetColumnID() const override
		{
			return ColumnID;
		}

		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<SAssetsPreviewWidget>& PreviewWidget) override
		{
			return SHeaderRow::Column( ColumnID )
				.DefaultLabel( LOCTEXT("AssetLabel_HeaderText", "Asset") )
				.FillWidth( 5.0f );
		}

		virtual const TSharedRef< SWidget > ConstructRowWidget(const IAssetTreeItemPtr& TreeItem, const STableRow<IAssetTreeItemPtr>& Row, const TSharedRef<SAssetsPreviewWidget>& PreviewWidget) override
		{
			PreviewWidgetWeakPtr = PreviewWidget;

			FSlateColor IconColor(FLinearColor::White);

			if ( !TreeItem->IsFolder() )
			{
				FAssetTreeAssetItem& AssetItem = static_cast<FAssetTreeAssetItem&>( *TreeItem.Get() );
				UObject* Asset = AssetItem.AssetPtr.Get();
				if ( Asset )
				{
					static FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>( TEXT("AssetTools") );
					TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass( Asset->GetClass() ).Pin();

					if ( AssetTypeActions.IsValid() )
					{
						IconColor = FSlateColor( AssetTypeActions->GetTypeColor() );
					}
				}
			}

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 2, 0)
				.VAlign(VAlign_Center)
				[
					// Item icon
					SNew(SImage)
					.Image( this, &FAssetPreviewDefaultColumn::GetIconBrush, TreeItem )
					.ColorAndOpacity(IconColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew( STextBlock )
					.Text( FText::FromString( *( TreeItem->Name ) ) )
					.Font( FAppStyle::GetFontStyle("ContentBrowser.SourceTreeItemFont") )
					.HighlightText( PreviewWidget, &SAssetsPreviewWidget::OnGetHighlightText )
				];
		}

		const FSlateBrush* GetIconBrush(IAssetTreeItemPtr TreeItem) const
		{
			const FSlateBrush* IconBrush = AssetIconBrush;
			if ( TreeItem->IsFolder() )
			{
				const bool bExpanded = PreviewWidgetWeakPtr.Pin()->GetTreeView()->IsItemExpanded( TreeItem );
				return bExpanded ? FolderOpenBrush : FolderClosedBrush;
			}
			return IconBrush;
		}

		virtual void PopulateSearchStrings(const IAssetTreeItemPtr& Item, TArray<FString>& OutSearchStrings, const SAssetsPreviewWidget& AssetPreview) const
		{
			if ( !Item->IsFolder() )
			{
				OutSearchStrings.Append( AssetPreview.GetItemsName( static_cast<FAssetTreeAssetItem*>( Item.Get() )->AssetPtr ) );
			}
		}

		virtual void SortItems(TArray<IAssetTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
		{
			if ( SortMode == EColumnSortMode::Ascending )
			{
				OutItems.Sort( [](const IAssetTreeItemPtr& First, const IAssetTreeItemPtr& Second) { return First->Name < Second->Name; } );
			}
			
			if ( SortMode == EColumnSortMode::Descending )
			{
				OutItems.Sort( [](const IAssetTreeItemPtr& First, const IAssetTreeItemPtr& Second) { return First->Name > Second->Name; } );
			}
		}

		static const FName ColumnID;

	private:
		/** Brushes for the different folder states */
		const FSlateBrush* FolderOpenBrush;
		const FSlateBrush* FolderClosedBrush;
		const FSlateBrush* AssetIconBrush;

		TWeakPtr<SAssetsPreviewWidget> PreviewWidgetWeakPtr;
	};

	const FName FAssetPreviewDefaultColumn::ColumnID =  FName("DefaultColumn");


	/** Represents a row in the AssetPreview's tree view */
	class SAssetPreviewTableRow : public SMultiColumnTableRow<IAssetTreeItemPtr>
	{
	public:
		SLATE_BEGIN_ARGS(SAssetPreviewTableRow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, IAssetTreeItemPtr InItem, TSharedRef< SAssetsPreviewWidget > InPreviewWidget)
		{
			PreviewWidgetWeakPtr = InPreviewWidget;
			ItemWeakPtr = InItem;

			SMultiColumnTableRow<IAssetTreeItemPtr>::Construct(
				STableRow::FArguments()
				.Cursor(EMouseCursor::Default),
				OwnerTableView);
		}


		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			auto ItemPtr = ItemWeakPtr.Pin();
			if ( !ItemPtr.IsValid() )
			{
				return SNullWidget::NullWidget;
			}

			TSharedRef<SAssetsPreviewWidget> AssetPreview = PreviewWidgetWeakPtr.Pin().ToSharedRef();
			TSharedRef<SWidget> NewItemWidget = SNullWidget::NullWidget;

			auto Column = AssetPreview->GetColumn( ColumnName );
			if ( Column.IsValid() )
			{
				NewItemWidget = Column->ConstructRowWidget( ItemPtr.ToSharedRef(), *this, AssetPreview );
			}

			if( ColumnName == FAssetPreviewDefaultColumn::ColumnID )
			{
				// The first column gets the tree expansion arrow for this row
				return
					SNew( SHorizontalBox )

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6, 0, 0, 0)
					[
						SNew( SExpanderArrow, SharedThis(this) ).IndentAmount(12)
					]

					+SHorizontalBox::Slot()
					.FillWidth( 1.0f )
					[
						NewItemWidget
					];
			}
			else
			{
				// Other columns just get widget content -- no expansion arrow needed
				return NewItemWidget;
			}
		}

	private:

	private:
		IAssetTreeItemWeakPtr ItemWeakPtr;

		/** Weak reference back to the preview widget that owns us */
		TWeakPtr< SAssetsPreviewWidget > PreviewWidgetWeakPtr;
	};

	void SAssetsPreviewWidget::Construct(const FArguments& InArgs)
	{
		HeaderRow = SNew(SHeaderRow)
			.Visibility( EVisibility::Visible );

		SetupColumns();

		ChildSlot
		[
			SNew(SVerticalBox)

			// Search and commands
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 2)
			[
				SNew(SHorizontalBox)
				// Search
				+ SHorizontalBox::Slot()
				.Padding(0, 1, 0, 0)
				.FillWidth(1.0f)
				[
					SNew(SAssetSearchBox)
					.OnTextChanged(this, &SAssetsPreviewWidget::OnSearchBoxChanged)
					.OnTextCommitted(this, &SAssetsPreviewWidget::OnSearchBoxCommitted)
					.DelayChangeNotificationsWhileTyping(true)
					.HintText(LOCTEXT("SearchHint", "Search..."))
				]
			]

			+ SVerticalBox::Slot()
			.Padding( 0, 0, 0, 2 )
			[
				SAssignNew(TreeView, STreeView<IAssetTreeItemPtr>)
				.SelectionMode(ESelectionMode::Multi)
				.TreeItemsSource(&RootItems)
				.HeaderRow(HeaderRow)
				.OnGenerateRow(this, &SAssetsPreviewWidget::MakeRowWidget)
				.OnSetExpansionRecursive(this, &SAssetsPreviewWidget::OnSetExpansionRecursive)
				.OnGetChildren(this, &SAssetsPreviewWidget::OnGetChildren)
				.OnSelectionChanged(this, &SAssetsPreviewWidget::OnSelectionChangedInternal)
				.OnContextMenuOpening(this, &SAssetsPreviewWidget::OnContextMenuOpeningInternal)
			]
		];
	}

	TSet<UObject*> SAssetsPreviewWidget::GetSelectedAssets() const
	{
		TSet< UObject* > Selection;
		TArray<IAssetTreeItemPtr> SelectedItems = TreeView->GetSelectedItems();

		for (IAssetTreeItemPtr Item : SelectedItems)
		{
			if ( !Item->IsFolder() )
			{
				Selection.Add( static_cast<FAssetTreeAssetItem&>( *Item.Get() ).AssetPtr.Get() );
			}
		}

		return MoveTemp(Selection);
	}

	void SAssetsPreviewWidget::SetSelectedAssets(TSet<UObject*> InSelectionSet, ESelectInfo::Type SelectionInfo)
	{
		TreeView->ClearSelection();

		TMap<const UObject*,FAssetTreeAssetItemPtr> AssetToItem;
		AssetToItem.Reserve( UnFilteredAssets.Num() );
		for ( const FAssetTreeAssetItemPtr& AssetItem : UnFilteredAssets )
		{
			if ( UObject* Asset = AssetItem->AssetPtr.Get() )
			{
				AssetToItem.Add( Asset, AssetItem );
			}
		}

	
		TArray<IAssetTreeItemPtr> ItemsToSelect;
		ItemsToSelect.Reserve( InSelectionSet.Num() );
		for ( const UObject* Asset : InSelectionSet )
		{
			if ( FAssetTreeAssetItemPtr* Item = AssetToItem.Find( Asset ) )
			{
				ItemsToSelect.Add( *Item );
			}
		}

		TreeView->SetItemSelection( ItemsToSelect, true, SelectionInfo );
	}

	void SAssetsPreviewWidget::SetAssetsList(const TArray< TWeakObjectPtr< UObject > >& InAssetsList, const FString& InPathToReplace, const FString& InSubstitutePath)
	{
		PathPrefixToRemove = InPathToReplace;
		SubstitutePath = InSubstitutePath;

		// Make sure the root dir is displayed as "Content".
		// This is more descriptive for the end user.
		TArray<FString> Tokens;
		const FString StrContent = TEXT("Content");
		SubstitutePath.ParseIntoArray( Tokens, TEXT("/") );
		if ( Tokens.Num() > 0 && !Tokens[0].Equals( StrContent ) )
		{
			const int32 StartChar = SubstitutePath.Find(Tokens[0]);
			SubstitutePath.RemoveAt(StartChar, Tokens[0].Len(), EAllowShrinking::No);
			SubstitutePath.InsertAt(StartChar, StrContent);
		}


		UnFilteredAssets.Empty( InAssetsList.Num() );
		for ( const TWeakObjectPtr< UObject >& Asset : InAssetsList )
		{
			if ( Asset.Get() )
			{
				FAssetTreeAssetItemPtr AssetItem = MakeShared<FAssetTreeAssetItem>();
				AssetItem->Name = Asset->GetName();
				AssetItem->AssetPtr = Asset;
				UnFilteredAssets.Add( MoveTemp( AssetItem ) );
			}
		}

		Refresh();
	}

	void SAssetsPreviewWidget::ClearAssetList()
	{
		UnFilteredAssets.Empty();
		Refresh();
	}

	void SAssetsPreviewWidget::RequestSort()
	{
		bIsSortDirty = true;
	}

	void SAssetsPreviewWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if ( bRequestedRefresh )
		{
			RootItems.Empty();
			NameToRootFolder.Empty();
			CurrentProcessingAssetIndex = 0;
			bRequestedRefresh = false;
			TreeView->RequestListRefresh();
		}

		UpdateColumns();

		if ( CurrentProcessingAssetIndex < UnFilteredAssets.Num() )
		{
			RequestSort();

			const int32 AssetToProccess = FMath::Min<int32>(  UnFilteredAssets.Num() - CurrentProcessingAssetIndex, 500 );
			int32 Index = 0;
			while ( Index < AssetToProccess )
			{
				FAssetTreeAssetItemPtr& AssetItem = UnFilteredAssets[Index + CurrentProcessingAssetIndex];
				if ( DoesPassFilter( AssetItem ) )
				{
					TArray<FString> Names = GetItemsName( AssetItem->AssetPtr );
					FAssetTreeFolderItemPtr Parent = FindOrCreateParentsItem( MakeArrayView( Names.GetData(), Names.Num() - 1 ) );
					Parent->Assets.Add( AssetItem );
				}
				++Index;
			}

			CurrentProcessingAssetIndex += Index;
		}

		if ( bIsSortDirty )
		{
			Sort( RootItems );
			TreeView->RequestListRefresh();
			ExpandAllFolders();
			bIsSortDirty = false;
		}

	}

	void SAssetsPreviewWidget::ExpandAllFolders()
	{
		for ( const IAssetTreeItemPtr& Item : RootItems )
		{
			OnSetExpansionRecursive( Item, true );
		}
	}

	void SAssetsPreviewWidget::Sort(TArray<IAssetTreeItemPtr>& InItems) const
	{
		if ( const TSharedRef<IAssetPreviewColumn>* ColumnPtr = Columns.Find( SortingColumn ) ) 
		{
			const TSharedRef<IAssetPreviewColumn>&  Column = *ColumnPtr;
			Column->SortItems( InItems, SortingMode );
		}
	}

	FString SAssetsPreviewWidget::GetItemPath(const TWeakObjectPtr<UObject>& Asset) const
	{
		FString AssetSubPath = Asset->GetPathName();

		// Check if asset has not been marked to be saved in another folder
		if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( Asset.Get() ) )
		{
			if (UDataprepConsumerUserData* DataprepContentUserData = AssetUserDataInterface->GetAssetUserData< UDataprepConsumerUserData >())
			{
				FString RelativeOutput = DataprepContentUserData->GetMarker(UDataprepContentConsumer::RelativeOutput);
				if (!RelativeOutput.IsEmpty() && !SubstitutePath.IsEmpty())
				{
					AssetSubPath = FPaths::GetCleanFilename(AssetSubPath);
					return SubstitutePath / RelativeOutput / AssetSubPath;
				}
			}
		}

		if ( AssetSubPath.RemoveFromStart( PathPrefixToRemove ) && !SubstitutePath.IsEmpty() )
		{
			AssetSubPath = SubstitutePath / AssetSubPath;
		}

		return AssetSubPath;
	}

	TArray< FString > SAssetsPreviewWidget::GetItemsName(const TWeakObjectPtr<UObject>& Asset) const
	{
		TArray<FString> ItemsName;
		GetItemPath( Asset ).ParseIntoArray( ItemsName, TEXT("/"), true );
		return ItemsName;
	}

	TSharedRef< ITableRow > SAssetsPreviewWidget::MakeRowWidget(IAssetTreeItemPtr InItem, const TSharedRef< STableViewBase >& OwnerTable)
	{
		return SNew( SAssetPreviewTableRow, OwnerTable, InItem, SharedThis(this) );
	}

	void SAssetsPreviewWidget::OnGetChildren(IAssetTreeItemPtr InParent, TArray<IAssetTreeItemPtr>& OutChildren) const
	{
		if ( InParent->IsFolder() )
		{
			FAssetTreeFolderItem& FolderItem = static_cast<FAssetTreeFolderItem&>( *InParent.Get() );
			OutChildren.Reserve( FolderItem.Folders.Num() + FolderItem.Assets.Num() );

			Sort( FolderItem.Folders );

			for ( IAssetTreeItemPtr& Folder : FolderItem.Folders )
			{
				OutChildren.Add( Folder );
			}

			Sort( FolderItem.Assets );

			for ( IAssetTreeItemPtr& Asset : FolderItem.Assets )
			{
				OutChildren.Add( Asset );
			}
		}
	}

	void SAssetsPreviewWidget::OnSearchBoxChanged(const FText& InSearchText)
	{
		FilterText = InSearchText;
		FilterStrings.Reset();
		FilterText.ToString().ParseIntoArray( FilterStrings, TEXT(" ")) ;
		Refresh();
	}

	void SAssetsPreviewWidget::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
	{
		FilterText = InSearchText;
		FilterStrings.Empty();
		FilterText.ToString().ParseIntoArray( FilterStrings, TEXT(" ") );
		Refresh();
	}

	FText SAssetsPreviewWidget::OnGetHighlightText() const
	{
		return FilterText;
	}

	TSharedPtr<IAssetPreviewColumn> SAssetsPreviewWidget::GetColumn(FName ColumnID) const
	{
		if ( const TSharedRef<IAssetPreviewColumn>* Column = Columns.Find(ColumnID))
		{
			return *Column;
		}

		return {};
	}

	void SAssetsPreviewWidget::AddColumn(TSharedRef<IAssetPreviewColumn> Column)
	{
		if ( !Columns.Contains( Column->GetColumnID() ) )
		{
			PendingColumnsToAdd.Add( Column );
		}
	}

	void SAssetsPreviewWidget::RemoveColumn(FName ColumnID)
	{
		if ( Columns.Contains( ColumnID ) )
		{
			PendingColumnsToRemove.Add( ColumnID );
		}
	}

	void SAssetsPreviewWidget::OnSetExpansionRecursive(IAssetTreeItemPtr InTreeNode, bool bInIsItemExpanded)
	{
		if ( InTreeNode.IsValid() && InTreeNode->IsFolder() )
		{
			TreeView->SetItemExpansion( InTreeNode, bInIsItemExpanded );
			FAssetTreeFolderItem& Folder = static_cast<FAssetTreeFolderItem&>( *InTreeNode.Get() );
			for ( const IAssetTreeItemPtr& SubFolder : Folder.Folders )
			{
				OnSetExpansionRecursive( SubFolder, bInIsItemExpanded );
			}
		}
	}

	TSharedPtr<SWidget> SAssetsPreviewWidget::OnContextMenuOpeningInternal()
	{
		TSet< UObject* > Selection = GetSelectedAssets();

		if ( Selection.Num() > 0 && OnContextMenu().IsBound() )
		{
			return OnContextMenu().Execute( Selection );
		}

		return nullptr;
	}

	void SAssetsPreviewWidget::OnSelectionChangedInternal(IAssetTreeItemPtr ItemSelected, ESelectInfo::Type SelectionType)
	{
		if ( SelectionType != ESelectInfo::Direct )
		{
			TSet< UObject* > Selection = GetSelectedAssets();

			if ( Selection.Num() > 0 )
			{
				OnSelectionChanged().Broadcast( Selection );
			}
		}
	}

	EColumnSortMode::Type SAssetsPreviewWidget::GetColumnSortMode(const FName ColumnId) const
	{
		if ( SortingColumn == ColumnId )
		{
			return SortingMode;
		}

		return EColumnSortMode::None;
	}

	void SAssetsPreviewWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
	{
		if ( SortPriority == EColumnSortPriority::Primary )
		{
			SortingColumn = ColumnId;
			SortingMode = InSortMode;
			RequestSort();
		}
	}

	void SAssetsPreviewWidget::SetupColumns()
	{
		SHeaderRow* HeaderRowPtr = HeaderRow.Get();
		check( HeaderRowPtr );

		Columns.Add( AssetPreviewWidget::FAssetPreviewDefaultColumn::ColumnID, MakeShared<AssetPreviewWidget::FAssetPreviewDefaultColumn>() );

		SortingColumn = AssetPreviewWidget::FAssetPreviewDefaultColumn::ColumnID;
		SortingMode = EColumnSortMode::Ascending;

		TSharedRef<SAssetsPreviewWidget> AssetPreviewWidget = StaticCastSharedRef<SAssetsPreviewWidget>( AsShared() );
		for (const TPair<FName,TSharedRef<IAssetPreviewColumn>>& Pair : Columns )
		{
			HeaderRowPtr->AddColumn(
				Pair.Value->ConstructHeaderRowColumn( AssetPreviewWidget )
					.SortMode(this, &SAssetsPreviewWidget::GetColumnSortMode, AssetPreviewWidget::FAssetPreviewDefaultColumn::ColumnID)
					.OnSort(this, &SAssetsPreviewWidget::OnColumnSortModeChanged)
				);
		}
	}

	void SAssetsPreviewWidget::UpdateColumns()
	{
		SHeaderRow* HeaderRowPtr = HeaderRow.Get();
		check( HeaderRowPtr );

		if ( PendingColumnsToRemove.Num() > 0 )
		{
			for ( const FName& ColumnToRemove : PendingColumnsToRemove )
			{
				HeaderRowPtr->RemoveColumn( ColumnToRemove );
				Columns.Remove( ColumnToRemove );
			}
			PendingColumnsToRemove.Empty();
		}

		if ( PendingColumnsToAdd.Num() > 0 ) 
		{
			PendingColumnsToAdd.StableSort([](const TSharedRef<IAssetPreviewColumn>& First, const TSharedRef<IAssetPreviewColumn>& Second) -> bool
				{
					return First->GetCulumnPositionPriorityIndex() >= Second->GetCulumnPositionPriorityIndex();
				});

			TSharedRef<SAssetsPreviewWidget> AssetPreviewWidget = StaticCastSharedRef<SAssetsPreviewWidget>( AsShared() );

			int32 CurrentElementToAddIndex = 0;
			int32 CurrentPriority = PendingColumnsToAdd[CurrentElementToAddIndex]->GetCulumnPositionPriorityIndex();
			int32 CurrentColumnIndex = 0;
			TArray<int32> InsertionIndex;
			InsertionIndex.Reserve( PendingColumnsToAdd.Num() );


			for (const TPair<FName, TSharedRef<IAssetPreviewColumn>>& Pair : Columns)
			{
				while ( Pair.Value->GetCulumnPositionPriorityIndex() < CurrentPriority )
				{
					InsertionIndex.Add( CurrentElementToAddIndex + CurrentColumnIndex );
					++CurrentElementToAddIndex;
					if ( CurrentElementToAddIndex == PendingColumnsToAdd.Num() )
					{
						break;
					}
				
					CurrentPriority = PendingColumnsToAdd[CurrentElementToAddIndex]->GetCulumnPositionPriorityIndex();
				}

				if ( CurrentElementToAddIndex == PendingColumnsToAdd.Num() )
				{
					break;
				}

				++CurrentColumnIndex;
			}

			Columns.Reserve( PendingColumnsToAdd.Num() );
			for ( TSharedRef<IAssetPreviewColumn>& NewColumn : PendingColumnsToAdd )
			{
				Columns.Add( NewColumn->GetColumnID(), NewColumn );
			}

			for (int32 Index = 0; Index < InsertionIndex.Num(); ++Index)
			{
				HeaderRowPtr->InsertColumn(
					PendingColumnsToAdd[Index]->ConstructHeaderRowColumn( AssetPreviewWidget )
						.SortMode( this, &SAssetsPreviewWidget::GetColumnSortMode, PendingColumnsToAdd[CurrentElementToAddIndex]->GetColumnID() )
						.OnSort( this, &SAssetsPreviewWidget::OnColumnSortModeChanged )
					, InsertionIndex[Index]
					);
			}

			while (CurrentElementToAddIndex < PendingColumnsToAdd.Num())
			{
				HeaderRowPtr->AddColumn(
					PendingColumnsToAdd[CurrentElementToAddIndex]->ConstructHeaderRowColumn( AssetPreviewWidget )
					.SortMode( this, &SAssetsPreviewWidget::GetColumnSortMode, PendingColumnsToAdd[CurrentElementToAddIndex]->GetColumnID() )
					.OnSort( this, &SAssetsPreviewWidget::OnColumnSortModeChanged )
					);
				++CurrentElementToAddIndex;
			}

			PendingColumnsToAdd.Empty();

			Columns.ValueSort([](const TSharedRef<IAssetPreviewColumn>& First, const TSharedRef<IAssetPreviewColumn>& Second) -> bool
				{
					return First->GetCulumnPositionPriorityIndex() >= Second->GetCulumnPositionPriorityIndex();
				}
			);
		}
	}

	void SAssetsPreviewWidget::Refresh()
	{
		bRequestedRefresh = true;
	}

	FAssetTreeFolderItemPtr SAssetsPreviewWidget::FindOrCreateParentsItem(const TArrayView<FString>& ParentNames)
	{
		if ( ParentNames.Num() > 0 )
		{
			FAssetTreeFolderItemPtr LastParent = nullptr;
			FString* CurrentParentName = &ParentNames[0];
			uint32 CurrentHash = GetTypeHash( *CurrentParentName );

			// We are starting from the root
			if ( FAssetTreeFolderItemPtr* RootPtr = NameToRootFolder.FindByHash( CurrentHash, *CurrentParentName ) )
			{
				LastParent = *RootPtr;
			}
			else
			{
				FAssetTreeFolderItemPtr Root = MakeShared<FAssetTreeFolderItem>();
				Root->Name = *CurrentParentName;
				LastParent = Root;
				RootItems.Add( Root );
				NameToRootFolder.AddByHash( CurrentHash, *CurrentParentName, MoveTemp( Root ) );
			}

			check( LastParent );

			for ( int32 Index = 1; Index < ParentNames.Num(); ++Index )
			{
				CurrentParentName = &ParentNames[Index];
				CurrentHash = GetTypeHash( *CurrentParentName );
				if ( FAssetTreeFolderItemPtr* CurrentPtr = LastParent->NameToFolder.FindByHash( CurrentHash, *CurrentParentName ) )
				{
					LastParent = *CurrentPtr;
				}
				else
				{
					FAssetTreeFolderItemPtr CurrentParent = MakeShared<FAssetTreeFolderItem>();
					CurrentParent->Name = *CurrentParentName;
					LastParent->Folders.Add( CurrentParent );
					LastParent->NameToFolder.AddByHash( CurrentHash, *CurrentParentName, CurrentParent );
					LastParent = MoveTemp( CurrentParent );
				}
			}

			check( LastParent );
			return LastParent;
		}

		return {};
	}

	bool SAssetsPreviewWidget::DoesPassFilter(const FAssetTreeAssetItemPtr& AssetItem) const
	{
		if ( FilterStrings.Num() == 0 )
		{
			return true;
		}

		TArray<FString> SearchableStrings;
		for ( const TPair<FName, TSharedRef<IAssetPreviewColumn>>& Pair : Columns )
		{
			Pair.Value->PopulateSearchStrings( AssetItem, SearchableStrings, *this );
		}
		
		// All key word must match against a least one of the provided string
		bool bPassKeyWord = false;
		for ( const FString& KeyWord : FilterStrings )
		{
			bPassKeyWord = false;
			for (const FString& String : SearchableStrings)
			{
				if ( String.Contains(KeyWord) )
				{
					bPassKeyWord = true;
					break;
				}
			}

			if ( !bPassKeyWord )
			{
				break;
			}
		}

		return bPassKeyWord;
	}

	void SAssetsPreviewWidget::SelectMatchingItems(const TSet<UObject*>& InAssets)
	{
		TFunction<void(TArray<IAssetTreeItemPtr>&, const TArray<IAssetTreeItemPtr>&, const TArray<IAssetTreeItemPtr>&)> 
			SelectMatchingItemsInternal = [this, &SelectMatchingItemsInternal, &InAssets](TArray<IAssetTreeItemPtr>& FolderPath, const TArray<IAssetTreeItemPtr>& Folders, const TArray<IAssetTreeItemPtr>& Assets)
		{
			for ( const IAssetTreeItemPtr& AssetItemPtr : Assets )
			{
				FAssetTreeAssetItem& AssetItem = static_cast<FAssetTreeAssetItem&>( *AssetItemPtr.Get() );
				UObject* Asset = AssetItem.AssetPtr.Get();
				if ( Asset && InAssets.Contains(Asset) )
				{
					// Expand folder path first
					for (IAssetTreeItemPtr& FolderPtr : FolderPath)
					{
						TreeView->SetItemExpansion(FolderPtr, true);
					}
					TreeView->SetItemSelection(AssetItemPtr, true, ESelectInfo::Direct);
				}
			}

			// Recurse into child folders
			for ( const IAssetTreeItemPtr& Folder : Folders )
			{
				FolderPath.Add(Folder);
				FAssetTreeFolderItem* FolderItem = static_cast<FAssetTreeFolderItem*>( Folder.Get() );
				SelectMatchingItemsInternal(FolderPath, FolderItem->Folders, FolderItem->Assets);
				FolderPath.Pop();
			}
		};

		TreeView->ClearSelection();

		if (InAssets.Num() == 0)
		{
			return;
		}

		TArray<IAssetTreeItemPtr> RootAssets;
		TArray<IAssetTreeItemPtr> RootFolders;

		for ( const IAssetTreeItemPtr& AssetItemPtr : RootItems )
		{
			if (AssetItemPtr->IsFolder())
			{
				RootFolders.Add(AssetItemPtr);
			}
			else
			{
				RootAssets.Add(AssetItemPtr);
			}
		}

		TArray<IAssetTreeItemPtr> FolderPath;

		SelectMatchingItemsInternal(FolderPath, RootFolders, RootAssets);
	}
}

#undef LOCTEXT_NAMESPACE
