// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DataprepEditorUtils.h"

#include "Widgets/Views/STreeView.h"

template <typename ItemType> class STableRow;

class SHeaderRow;

namespace AssetPreviewWidget
{
	struct IAssetTreeItem;

	using IAssetTreeItemPtr = TSharedPtr<IAssetTreeItem>;
	using IAssetTreeItemWeakPtr = TWeakPtr<IAssetTreeItem>;

	struct FAssetTreeAssetItem;
	using FAssetTreeAssetItemPtr = TSharedPtr<FAssetTreeAssetItem>;

	struct FAssetTreeFolderItem;
	using FAssetTreeFolderItemPtr = TSharedPtr<FAssetTreeFolderItem>;


	class SAssetsPreviewWidget;

	/**
	 *	Interface for a column of the asset preview
	 */
	class IAssetPreviewColumn : public TSharedFromThis< IAssetPreviewColumn >
	{
	public:
		virtual ~IAssetPreviewColumn() = default;

		// A index for the desired position column. The higher index end up on left.
		virtual uint8 GetCulumnPositionPriorityIndex() const = 0;

		virtual FName GetColumnID() const = 0;

		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<SAssetsPreviewWidget>& PreviewWidget) = 0;

		virtual const TSharedRef< SWidget > ConstructRowWidget(const IAssetTreeItemPtr& TreeItem, const STableRow<IAssetTreeItemPtr>& Row, const TSharedRef<SAssetsPreviewWidget>& PreviewWidget) = 0;

	public:
		/** Optionally overridden interface methods */
		virtual void PopulateSearchStrings(const IAssetTreeItemPtr& Item, TArray< FString >& OutSearchStrings, const SAssetsPreviewWidget& AssetPreview) const {}

		virtual void SortItems(TArray<IAssetTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const {}
	};

	using FAssetPreviewColumn = TSharedRef<IAssetPreviewColumn>;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChanged, TSet< UObject* > /** Selected objects */)
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnContextMenu, TSet< UObject* > /** Selected objects */)

	class SAssetsPreviewWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAssetsPreviewWidget) {}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);

		void SetAssetsList(const TArray< TWeakObjectPtr< UObject > >& InAssetsList, const FString& InPathToReplace, const FString& InSubstitutePath);
		void ClearAssetList();

		void RequestSort();

		FOnSelectionChanged& OnSelectionChanged() { return OnSelectionChangedDelegate; }
		FOnContextMenu& OnContextMenu() { return OnContextMenuDelegate; }
		FDataprepEditorUtils::FOnKeyDown& OnKeyDown() { return OnKeyDownDelegate; }

		FText OnGetHighlightText() const;

		const TSharedPtr< STreeView< IAssetTreeItemPtr > > GetTreeView() const
		{
			return TreeView;
		}

		TSharedPtr<IAssetPreviewColumn> GetColumn(FName ColumnID) const;

		void AddColumn(TSharedRef<IAssetPreviewColumn> Column);

		void RemoveColumn(FName ColumnID);

		// Begin of SWidgetInterface
		void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		// End of SWidgetInterface

		TArray<FString> GetItemsName(const TWeakObjectPtr<UObject>& Asset) const;

		EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

		void SelectMatchingItems(const TSet<UObject*>& InAssets);

		TSet<UObject*> GetSelectedAssets() const;

		void SetSelectedAssets(TSet<UObject*> InSelectionSet, ESelectInfo::Type SelectionInfo);

		virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
		{
			if( OnKeyDownDelegate.IsBound() )
			{
				return OnKeyDownDelegate.Execute( MyGeometry, InKeyEvent );
			}
			return FReply::Unhandled();
		}

	private:

		friend struct FAssetTreeItem;

		void ExpandAllFolders();

		void Sort(TArray<IAssetTreeItemPtr>& InItems) const;

		FString GetItemPath(const TWeakObjectPtr<UObject>& Asset) const;

		TSharedRef< class ITableRow > MakeRowWidget(IAssetTreeItemPtr InItem, const TSharedRef< class STableViewBase >& OwnerTable);
		void OnGetChildren(IAssetTreeItemPtr InParent, TArray<IAssetTreeItemPtr>& OutChildren) const;

		void OnSearchBoxChanged(const FText& InSearchText);
		void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

		void OnSetExpansionRecursive(const IAssetTreeItemPtr InTreeNode, bool bInIsItemExpanded);

		void OnSelectionChangedInternal(IAssetTreeItemPtr ItemSelected, ESelectInfo::Type SelectionType);

		void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

		TSharedPtr<SWidget> OnContextMenuOpeningInternal();

		void SetupColumns();

		void UpdateColumns();

		void Refresh();

		/**
		 * Find or create the parents chain
		 * @param ParentNames the names of the parents for the top to bottom
		 * @return the immediate parent
		 */
		FAssetTreeFolderItemPtr FindOrCreateParentsItem(const TArrayView<FString>& ParentNames);

		// For now we only filter the assets items
		bool DoesPassFilter(const FAssetTreeAssetItemPtr& AssetItem) const;

		// The root items displayed
		TArray<IAssetTreeItemPtr> RootItems;
		TMap<FString, FAssetTreeFolderItemPtr> NameToRootFolder;

		// The list of asset unfiltered
		TArray<FAssetTreeAssetItemPtr> UnFilteredAssets;
		int32 CurrentProcessingAssetIndex = 0;

		TSharedPtr< STreeView< IAssetTreeItemPtr > > TreeView;
		TSharedPtr< SHeaderRow > HeaderRow;

		FText FilterText;
		FString PathPrefixToRemove;
		FString SubstitutePath;

		TMap<FName, TSharedRef<IAssetPreviewColumn>> Columns;

		TArray<TSharedRef<IAssetPreviewColumn>> PendingColumnsToAdd;
		TArray<FName> PendingColumnsToRemove;

		FOnSelectionChanged OnSelectionChangedDelegate;
		FOnContextMenu OnContextMenuDelegate;
		FDataprepEditorUtils::FOnKeyDown OnKeyDownDelegate;

		FName SortingColumn;
		EColumnSortMode::Type SortingMode;

		// The string used for the filtering
		TArray<FString> FilterStrings;

		bool bIsSortDirty = false;
		bool bRequestedRefresh = false;
	};

	struct IAssetTreeItem
	{
		virtual ~IAssetTreeItem() = default;

		FString Name;
		virtual bool IsFolder() const  = 0;
	};

	struct FAssetTreeAssetItem : public IAssetTreeItem
	{
		virtual bool IsFolder() const final
		{
			return false;
		}

		TWeakObjectPtr< UObject > AssetPtr;
	};

	struct FAssetTreeFolderItem : public IAssetTreeItem
	{
		virtual bool IsFolder() const final
		{
			return true;
		}

		// This is used to accelerate the construction of the tree in the set assets function
		TMap<FString, FAssetTreeFolderItemPtr> NameToFolder;

		// Childrens
		TArray<IAssetTreeItemPtr> Folders;
		TArray<IAssetTreeItemPtr> Assets;
	};

}
