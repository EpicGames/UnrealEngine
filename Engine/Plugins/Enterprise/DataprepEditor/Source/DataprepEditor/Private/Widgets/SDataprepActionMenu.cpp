// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDataprepActionMenu.h"

#include "DataprepAsset.h"
#include "DataprepGraph/DataprepGraph.h"
#include "SGraphActionMenu.h"
#include "SchemaActions/IDataprepMenuActionCollector.h"

#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Layout/SBox.h"

void SDataprepActionMenu::Construct(const FArguments& InArgs, TUniquePtr<IDataprepMenuActionCollector> InMenuActionCollector)
{
	MenuActionCollector = MoveTemp( InMenuActionCollector );
	
	Context = InArgs._DataprepActionContext;
	TransactionTextGetter = InArgs._TransactionText;
	GraphObj = InArgs._GraphObj;
	NewNodePosition = InArgs._NewNodePosition;
	DraggedFromPins = InArgs._DraggedFromPins;
	OnClosedCallback = InArgs._OnClosedCallback;
	OnCollectCustomActions = InArgs._OnCollectCustomActions;

	ChildSlot
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding( 5 )
		[
			SNew( SBox )
			.WidthOverride( 300 )
			.HeightOverride( 400 )
			[
				SAssignNew( ActionMenu, SGraphActionMenu )
				.OnActionSelected( this, &SDataprepActionMenu::OnActionSelected )
				.OnCollectAllActions( this, &SDataprepActionMenu::CollectActions )
				.OnCreateCustomRowExpander( this, &SDataprepActionMenu::OnCreateCustomRowExpander )
				.AutoExpandActionMenu( MenuActionCollector->ShouldAutoExpand() )
				.ShowFilterTextBox( true )
			]
		]
	];
}

TSharedPtr<class SEditableTextBox> SDataprepActionMenu::GetFilterTextBox()
{
	return ActionMenu->GetFilterTextBox();
}

SDataprepActionMenu::~SDataprepActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void SDataprepActionMenu::CollectActions(FGraphActionListBuilderBase& OutActions)
{
	if ( OnCollectCustomActions.IsBound() )
	{
		TArray< TSharedPtr<FDataprepSchemaAction> > CustomActions;
		OnCollectCustomActions.Execute( CustomActions );

		for ( TSharedPtr<FDataprepSchemaAction> Action : CustomActions )
		{
			OutActions.AddAction( StaticCastSharedPtr<FEdGraphSchemaAction>( Action ) );
		}
	}

	for ( TSharedPtr<FDataprepSchemaAction> Action : MenuActionCollector->CollectActions() )
	{
		OutActions.AddAction( StaticCastSharedPtr<FEdGraphSchemaAction>( Action ) );
	}
}

void SDataprepActionMenu::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	auto ExecuteAction = [this](FDataprepSchemaAction& DataprepAction, UDataprepGraph* DataprepGraph) -> bool
	{
		UDataprepAsset* DataprepAsset = DataprepGraph ? DataprepGraph->GetDataprepAsset() : nullptr;

		if ( DataprepAsset )
		{
			int32 Index = DataprepAsset->AddAction(nullptr);
			Context.DataprepActionPtr = MakeWeakObjectPtr<UDataprepActionAsset>( DataprepAsset->GetAction(Index) );
		}

		return DataprepAction.ExecuteAction( Context ) ? true : false;
	};

	for ( const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedActions )
	{
		FDataprepSchemaAction* DataprepAction = static_cast< FDataprepSchemaAction* >( Action.Get() );
		if ( DataprepAction )
		{
			if ( TransactionTextGetter.IsSet() )
			{
				FScopedTransaction Transaction( TransactionTextGetter.Get() );

				if(UDataprepGraph* DataprepGraph = Cast<UDataprepGraph>(GraphObj))
				{
					if ( !ExecuteAction( *DataprepAction, DataprepGraph ) )
					{
						Transaction.Cancel();
					}
				}
			}
			else if(UDataprepGraph* DataprepGraph = Cast<UDataprepGraph>(GraphObj))
			{
				ExecuteAction( *DataprepAction, DataprepGraph );
			}
		}
	}

	if ( SelectedActions.Num() > 0 )
	{
		FSlateApplication::Get().DismissAllMenus();
	}
}

TSharedRef<SExpanderArrow> SDataprepActionMenu::OnCreateCustomRowExpander(const FCustomExpanderData& InCustomExpanderData) const
{
	return SNew( SExpanderArrow, InCustomExpanderData.TableRow );
}

bool SDataprepActionMenu::ShouldCreateNewNode() const
{
	if ( GraphObj && !Context.DataprepActionPtr.Get() )
	{
		return true;
	}

	return false;
}
