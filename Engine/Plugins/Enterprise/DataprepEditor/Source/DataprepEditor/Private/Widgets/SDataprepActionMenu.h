// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SchemaActions/DataprepSchemaAction.h"

#include "GraphEditor.h"
#include "SchemaActions/DataprepSchemaAction.h"

namespace ESelectInfo { enum Type : int; }
struct FEdGraphSchemaAction;

class IDataprepMenuActionCollector;
class SExpanderArrow;
class UEdGraph;
class UEdGraphPin;

struct FCustomExpanderData;
struct FGraphActionListBuilderBase;

/**
 * The SDataprepActionMenu is the action menu used in Dataprep. 
 * Because of the IDataprepMenuActionCollector this can use for different actions. (example: Add an operation to dataprep action or Add a action to dataprep)
 */
class SDataprepActionMenu : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnCollectCustomActions, TArray< TSharedPtr<FDataprepSchemaAction> >& )

	SLATE_BEGIN_ARGS(SDataprepActionMenu) 
		: _TransactionText()
		, _DataprepActionContext()
		, _GraphObj(nullptr)
		, _NewNodePosition()
		, _DraggedFromPins()
		, _OnClosedCallback()
	{}
		SLATE_ATTRIBUTE( FText, TransactionText )
		SLATE_ARGUMENT( FDataprepSchemaActionContext, DataprepActionContext )
		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( FOnCollectCustomActions, OnCollectCustomActions )
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TUniquePtr<IDataprepMenuActionCollector> InMenuActionCollector);

	TSharedPtr<class SEditableTextBox> GetFilterTextBox();

	virtual ~SDataprepActionMenu();

private:

	void CollectActions(FGraphActionListBuilderBase& OutActions);

	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	TSharedRef<SExpanderArrow> OnCreateCustomRowExpander(const FCustomExpanderData& InCustomExpanderData) const;

	/**
	 * Should this action menu create a new node before executing the user selected action
	 */
	bool ShouldCreateNewNode() const;

private:
	TAttribute<FText> TransactionTextGetter;

	TUniquePtr<IDataprepMenuActionCollector> MenuActionCollector;
	TSharedPtr<class SGraphActionMenu> ActionMenu;

	// The context on which this action menu was created
	FDataprepSchemaActionContext Context;

	// Will be nullptr if this menu shouldn't modify the graph
	UEdGraph* GraphObj;
	TArray<UEdGraphPin*> DraggedFromPins;
	FVector2D NewNodePosition;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
	FOnCollectCustomActions OnCollectCustomActions;
};
