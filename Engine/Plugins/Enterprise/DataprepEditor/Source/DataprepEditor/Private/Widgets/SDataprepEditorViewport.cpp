// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataprepEditorViewport.h"

#include "AdvancedPreviewScene.h"
#include "DataprepCorePrivateUtils.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditor.h"

#include "DataprepEditorLogCategory.h"

#include "ActorEditorUtils.h"
#include "AssetViewerSettings.h"
#include "ComponentReregisterContext.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "StaticMeshResources.h"
#include "Rendering/NaniteResources.h"
#include "EngineLogs.h"
#include "StaticMeshSceneProxy.h"
#include "Engine/TextureCube.h"
#include "SceneInterface.h"
#include "EngineUtils.h"
#include "Framework/Commands/UICommandList.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MeshDescription.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "SEditorViewportToolBarMenu.h"
#include "Slate/SceneViewport.h"
#include "Viewports.h"
#include "SceneInterface.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DataprepEditorViewport"

TStrongObjectPtr<UMaterial> SDataprepEditorViewport::PreviewMaterial;
TStrongObjectPtr<UMaterial> SDataprepEditorViewport::XRayMaterial;
TStrongObjectPtr<UMaterial> SDataprepEditorViewport::BackFaceMaterial;
TStrongObjectPtr<UMaterial> SDataprepEditorViewport::PerMeshMaterial;
TStrongObjectPtr<UMaterial> SDataprepEditorViewport::ReflectionMaterial;
TStrongObjectPtr<UMaterialInstanceConstant> SDataprepEditorViewport::TransparentMaterial;
TArray<TStrongObjectPtr<UMaterialInstanceConstant>> SDataprepEditorViewport::PerMeshMaterialInstances;
int32 SDataprepEditorViewport::AssetViewerProfileIndex = INDEX_NONE;


const FColor PerMeshColor[] =
{
	FColor(255, 49,  0),
	FColor(255,135,  0),
	FColor( 11,182,255),
	FColor(  0,255,103),

	FColor(255,181,164),
	FColor(255,212,164),
	FColor(168,229,255),
	FColor(164,255,201),

	FColor(255,139,112),
	FColor(255,188,112),
	FColor(118,214,255),
	FColor(112,255,170),

	FColor(217, 41,  0),
	FColor(217,115,  0),
	FColor(  0, 95,137),
	FColor(  0,156, 63),

	FColor(167, 32,  0),
	FColor(167, 88,  0),
	FColor(  0, 73,105),
	FColor(  0,120, 49),
};

namespace ViewportDebug
{
	static bool bLogTiming = false;

	class FTimeLogger
	{
	public:
		FTimeLogger(const FString& InText)
			: StartTime( FPlatformTime::Cycles64() )
			, Text( InText )
		{
			if( bLogTiming )
			{
				UE_LOGF( LogDataprepEditor, Log, "%ls ...", *Text );
			}
		}

		~FTimeLogger()
		{
			if( bLogTiming )
			{
				// Log time spent to import incoming file in minutes and seconds
				double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

				int ElapsedMin = int(ElapsedSeconds / 60.0);
				ElapsedSeconds -= 60.0 * (double)ElapsedMin;
				UE_LOGF( LogDataprepEditor, Log, "%ls took [%d min %.3f s]", *Text, ElapsedMin, ElapsedSeconds );
			}
		}

	private:
		uint64 StartTime;
		FString Text;
	};
}

// Extension of the FStaticMeshSceneProxy class to allow wireframe display per individual mesh
// #ueent_remark: Could we unify this behavior across classes deriving from UMeshComponent?
class FStaticMeshSceneProxyExt : public FStaticMeshSceneProxy
{
public:
	FStaticMeshSceneProxyExt(UStaticMeshComponent* Component, bool bForceLODsShareStaticLighting)
		: FStaticMeshSceneProxy( Component, bForceLODsShareStaticLighting )
	{
		CustomComponent = Cast<UCustomStaticMeshComponent>(Component);
		check( CustomComponent );
	}

	virtual bool GetMeshElement(
		int32 LODIndex,
		int32 BatchIndex,
		int32 ElementIndex,
		uint8 InDepthPriorityGroup,
		bool bUseSelectionOutline,
		bool bAllowPreCulledIndices,
		FMeshBatch& OutMeshBatch) const override
	{
		if(FStaticMeshSceneProxy::GetMeshElement(LODIndex, BatchIndex, ElementIndex, InDepthPriorityGroup, bUseSelectionOutline, bAllowPreCulledIndices, OutMeshBatch))
		{
			OutMeshBatch.bWireframe = CustomComponent->bForceWireframe;
			OutMeshBatch.bUseWireframeSelectionColoring = 0;

			return true;
		}

		return false;
	}

	virtual bool GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const override
	{
		if(FStaticMeshSceneProxy::GetWireframeMeshElement(LODIndex, BatchIndex, WireframeRenderProxy, InDepthPriorityGroup, bAllowPreCulledIndices, OutMeshBatch))
		{
			OutMeshBatch.bWireframe = CustomComponent->bForceWireframe;
			OutMeshBatch.bUseWireframeSelectionColoring = 0;

			return true;
		}

		return false;
	}

private:
	UCustomStaticMeshComponent* CustomComponent;
};

namespace DataprepEditor3DPreviewUtils
{
	/** Build the render data based on the current geometry available in the static mesh */
	void BuildStaticMeshes(TSet<UStaticMesh*>& StaticMeshes, TArray<UStaticMesh*>& BuiltMeshes);

	void FindMeshComponents(const AActor * InActor, TArray<UStaticMeshComponent*>& MeshComponents, bool bRecursive );

	/** Returns array of static mesh components in world */
	template<class T>
	TArray< T* > GetComponentsFromWorld( UWorld* World )
	{
		static_assert(TPointerIsConvertibleFromTo<T, const UMeshComponent>::Value, "'T' template parameter to FindComponentByClass must be derived from UActorComponent");

		TArray< T* > Result;

		const EActorIteratorFlags Flags = EActorIteratorFlags::SkipPendingKill;
		for (TActorIterator<AActor> It( World, AActor::StaticClass(), Flags ); It; ++It)
		{
			AActor* Actor = *It;

			// Don't consider transient actors in non-play worlds
			// Don't consider the builder brush
			// Don't consider the WorldSettings actor, even though it is technically editable
			const bool bIsValid = Actor != nullptr && Actor->IsEditable() && !Actor->IsTemplate() /*&& !Actor->HasAnyFlags(RF_Transient) */&& !FActorEditorUtils::IsABuilderBrush(Actor) && !Actor->IsA(AWorldSettings::StaticClass());

			if ( bIsValid )
			{
				TArray< T* > Components;
				Actor->GetComponents<T>( Components );
				for( T* Component : Components )
				{
					// If a mesh is displayable, it should have at least one material
					if( Component->GetNumMaterials() > 0 )
					{
						Result.Add( Component );
					}
				}
			}
		}

		return Result;
	}
}

//
// SDataprepEditorViewport Class
//

SDataprepEditorViewport::SDataprepEditorViewport()
	: PreviewScene( MakeShareable( new FAdvancedPreviewScene( FPreviewScene::ConstructionValues() ) ) )
	, Extender( MakeShareable( new FExtender ) )
	, WorldToPreview( nullptr )
	, RenderingMaterialType( ERenderingMaterialType::OriginalRenderingMaterial )
	, CurrentSelectionMode( ESelectionModeType::OutlineSelectionMode )
	, bWireframeRenderingMode( false )
#ifdef VIEWPORT_EXPERIMENTAL
	, bShowOrientedBox( false )
#endif
{
}

SDataprepEditorViewport::~SDataprepEditorViewport()
{
	CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
	ClearScene();
}

void SDataprepEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<FDataprepEditor> InDataprepEditor)
{
	DataprepEditor = InDataprepEditor;

	FDataprepEditorViewportCommands::Register();

	// restore last used feature level
	UWorld* PreviewSceneWorld = PreviewScene->GetWorld();
	if (ensure(PreviewSceneWorld) && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		PreviewSceneWorld->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	// Listen to and act on changes in preview platform
	UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine);
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([&PreviewSceneWorld](EShaderPlatform NewShaderPlatform)
	{
		PreviewSceneWorld->ShaderPlatformChanged(NewShaderPlatform);
	});

	// Create root actor of 3D viewport's world
	AActor* RootActor = CreateActor(nullptr, TEXT("RootPreviewActor"));
	RootActorComponent = RootActor->GetRootComponent();

	WorldToPreview = InArgs._WorldToPreview;
	ensure(WorldToPreview);

	SEditorViewport::Construct( SEditorViewport::FArguments() );
}

void SDataprepEditorViewport::ClearScene()
{
	if (!StaticActor.IsValid())
	{
		ensure(PreviewMeshComponents.IsEmpty() && MeshComponentsMapping.IsEmpty() && MeshComponentsReverseMapping.IsEmpty() && SelectedPreviewComponents.IsEmpty());
		return;
	}

	UWorld* PreviewSceneWorld = PreviewScene->GetWorld();

	TFunction<void(AActor*)> DeleteActor = [PreviewSceneWorld](AActor* ActorToDelete)
		{
			if (IsValid(ActorToDelete))
			{
				PreviewSceneWorld->EditorDestroyActor(ActorToDelete, true);

				// Since deletion can be delayed, rename to avoid future name collision
				// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
				ActorToDelete->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		};

	DeleteActor(StaticActor.Get());
	DeleteActor(MovableActor.Get());

	StaticActor = nullptr;
	MovableActor = nullptr;

	const int32 PreviousCount = PreviewMeshComponents.Num();

	PreviewMeshComponents.Empty(PreviousCount);
	MeshComponentsMapping.Empty(PreviousCount);
	MeshComponentsReverseMapping.Empty(PreviousCount);
	SelectedPreviewComponents.Empty(SelectedPreviewComponents.Num());

	OverlayTextVerticalBox->ClearChildren();
}

void SDataprepEditorViewport::UpdateScene()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDataprepEditorViewport::UpdateScene);

	ClearScene();

	// Create actor to hold all 'Static' preview mesh components
	StaticActor = CreateActor(RootActorComponent.Get(), TEXT("StaticPreviewActor"));
	StaticActor->GetRootComponent()->SetMobility(EComponentMobility::Static);
	ensure(StaticActor.IsValid());

	// If HW supports it, create actor to hold all 'Movable' preview mesh components
	if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() > ERHIFeatureLevel::ES3_1)
	{
		MovableActor = CreateActor(RootActorComponent.Get(), TEXT("MovablePreviewActor"));
		MovableActor->GetRootComponent()->SetMobility(EComponentMobility::Movable);
	}

	ViewportDebug::FTimeLogger TimeLogger( TEXT("Updating viewport") );

	SceneBounds = FBox( FVector::ZeroVector, FVector(100.0f) );
	SceneUniformScale = 1.0f;

	// Gather all static meshes used by actors in PreviewWorld
	TArray< UStaticMeshComponent* > SceneMeshComponents = DataprepEditor3DPreviewUtils::GetComponentsFromWorld<UStaticMeshComponent>( WorldToPreview );

	bCanShowNaniteFallbackMenu = false;

	if(SceneMeshComponents.Num() > 0)
	{
		int32 InvalidStaticMeshesCount = 0;

		for( UStaticMeshComponent*& MeshComponent : SceneMeshComponents )
		{
			if( UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh() )
			{
				if( !StaticMesh->GetRenderData() || !StaticMesh->GetRenderData()->IsInitialized())
				{
					++InvalidStaticMeshesCount;
					MeshComponent = nullptr;
				}
			}
			else
			{
				++InvalidStaticMeshesCount;
				MeshComponent = nullptr;
			}
		}

		if( SceneMeshComponents.Num() > InvalidStaticMeshesCount )
		{
			FScopedSlowTask SlowTask( 100.0f, LOCTEXT("UpdateMeshes_Title", "Updating 3D viewport ...") );
			SlowTask.MakeDialog(false);

			// Unregister all actors components to avoid excessive refresh in the 3D engine while updating materials.
			for (TObjectIterator<AActor> ActorIterator; ActorIterator; ++ActorIterator)
			{
				if (ActorIterator->GetWorld())
				{
					ActorIterator->UnregisterAllComponents( /* bForReregister = */true);
				}
			}

			InitializeDefaultMaterials();

			// Materials have been updated, we can register everything back.
			for (TObjectIterator<AActor> ActorIterator; ActorIterator; ++ActorIterator)
			{
				if (ActorIterator->GetWorld())
				{
					ActorIterator->RegisterAllComponents();
				}
			}

			PreviewMeshComponents.Empty( SceneMeshComponents.Num() );

			// Compute bounding box of scene to determine camera position and scaling to apply for smooth navigation
			SlowTask.EnterProgressFrame( 10.0f, LOCTEXT("UpdateMeshes_Prepare", "Computing scene's bounding box ...") );
			SceneBounds.Init();
			for( UStaticMeshComponent* SceneMeshComponent : SceneMeshComponents )
			{
				if(SceneMeshComponent != nullptr)
				{
					const UStaticMesh* StaticMesh = SceneMeshComponent->GetStaticMesh();
					const FTransform& ComponentToWorldTransform = SceneMeshComponent->GetComponentTransform();
					SceneBounds += StaticMesh->GetExtendedBounds().GetBox().TransformBy( ComponentToWorldTransform );

					bCanShowNaniteFallbackMenu |= StaticMesh->IsNaniteEnabled();
				}
			}

			// Compute uniform scale
			FVector Extents = SceneBounds.GetExtent();
			if(Extents.GetMax() < 100.0f)
			{
				SceneUniformScale = 100.0f / ( Extents.GetMax() * 1.1f );
			}
			SceneBounds.Max *= SceneUniformScale;
			SceneBounds.Min *= SceneUniformScale;

			// Set uniform scale on root actor's root component
			RootActorComponent->SetRelativeTransform( FTransform( FRotator::ZeroRotator, FVector::ZeroVector, FVector( SceneUniformScale ) ) );

			const int32 PerMeshColorsCount = sizeof(PerMeshColor);
			int32 PerMeshColorIndex = 0;
			// Replicate mesh component from world to preview in preview world
			SlowTask.EnterProgressFrame( 90.0f, LOCTEXT("UpdateMeshes_Components", "Adding meshes to viewport ...") );
			{
				FScopedSlowTask SubSlowTask( float(SceneMeshComponents.Num() - InvalidStaticMeshesCount), LOCTEXT("UpdateMeshes_Components", "Adding meshes to viewport ...") );

				for( UStaticMeshComponent* SceneMeshComponent : SceneMeshComponents )
				{
					if(SceneMeshComponent != nullptr)
					{
						const FText Message = FText::Format( LOCTEXT("UpdateMeshes_AddOneComponent", "Adding {0} ..."), FText::FromString( SceneMeshComponent->GetOwner()->GetActorLabel() ) );
						SubSlowTask.EnterProgressFrame( 1.0f, Message );

						EComponentMobility::Type Mobility = EComponentMobility::Static;
						if(MovableActor.IsValid())
						{
							Mobility = SceneMeshComponent->Mobility;
						}

						AActor* PreviewActor = Mobility == EComponentMobility::Movable && MovableActor.IsValid() ? MovableActor.Get() : StaticActor.Get();

						UCustomStaticMeshComponent* PreviewMeshComponent = NewObject< UCustomStaticMeshComponent >( PreviewActor, NAME_None, RF_Transient );
						PreviewMeshComponent->SetMobility( Mobility );

						PreviewMeshComponent->bOverrideLightMapRes = SceneMeshComponent->bOverrideLightMapRes;
						PreviewMeshComponent->OverriddenLightMapRes = SceneMeshComponent->OverriddenLightMapRes;
						PreviewMeshComponent->bForceWireframe = bWireframeRenderingMode;
						PreviewMeshComponent->MeshColorIndex = PerMeshColorIndex % PerMeshColorsCount;
						++PerMeshColorIndex;

						UStaticMesh* StaticMesh = SceneMeshComponent->GetStaticMesh();

						FComponentReregisterContext ReregisterContext( PreviewMeshComponent );
						PreviewMeshComponent->SetStaticMesh( StaticMesh );

						PreviewMeshComponent->bDisplayNaniteFallbackMesh = bShowNaniteFallbackMenuChecked && StaticMesh->IsNaniteEnabled();

						FTransform ComponentToWorldTransform = SceneMeshComponent->GetComponentTransform();

						PreviewMeshComponent->AttachToComponent( PreviewActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform );
						PreviewMeshComponent->SetRelativeTransform( ComponentToWorldTransform );

						// Apply preview material to preview static mesh component
						for(int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
						{
							UMaterialInterface* MaterialInterface = SceneMeshComponent->GetMaterial(Index);

							if(MaterialInterface == nullptr)
							{
								MaterialInterface = StaticMesh->GetStaticMaterials()[Index].MaterialInterface;
							}

							PreviewMeshComponent->SetMaterial( Index, MaterialInterface );
						}

						PreviewMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw( this, &SDataprepEditorViewport::IsComponentSelected );

						PreviewMeshComponent->RegisterComponentWithWorld( PreviewScene->GetWorld() );

						PreviewMeshComponents.Emplace( PreviewMeshComponent );

						MeshComponentsMapping.Add( SceneMeshComponent, PreviewMeshComponent );
						MeshComponentsReverseMapping.Add( PreviewMeshComponent, SceneMeshComponent );
					}
				}
			}

#ifdef VIEWPORT_EXPERIMENTAL
			{
				ViewportDebug::FTimeLogger LapTimeLogger( TEXT("Building mesh properties") );

				TSet<UStaticMesh*> StaticMeshes;
				for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents )
				{
					if(UCustomStaticMeshComponent* PreviewMeshComponent = Cast<UCustomStaticMeshComponent>(PreviewMeshComponentPtr.Get()))
					{
						StaticMeshes.Add( PreviewMeshComponent->GetStaticMesh() );
					}
				}

				TMap<UStaticMesh*, FPrototypeOrientedBox> MeshPropertiesMap;
				TArray<UStaticMesh*> StaticMeshesToBuild = StaticMeshes.Array();

				if( StaticMeshes.Num() > 1 )
				{
					TArray<FPrototypeOrientedBox> MeshProperties;
					MeshProperties.AddDefaulted( StaticMeshes.Num() );

					ParallelFor( StaticMeshesToBuild.Num(), [&]( int32 Index ) {
						MeshProperties[Index] = MeshDescriptionPrototype::GenerateOrientedBox( StaticMeshesToBuild[Index] );
					});

					for(int32 Index = 0; Index < StaticMeshesToBuild.Num(); ++Index)
					{
						MeshPropertiesMap.Add( StaticMeshesToBuild[Index], MeshProperties[Index]);
					}
				}
				else
				{
					MeshPropertiesMap.Add( StaticMeshesToBuild[0], MeshDescriptionPrototype::GenerateOrientedBox( StaticMeshesToBuild[0] ) );
				}

				TSet<UStaticMesh*> ShouldBeInstanced;
				MeshDescriptionPrototype::IdentifyInstances( MeshPropertiesMap, ShouldBeInstanced);

				for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents )
				{
					if(UCustomStaticMeshComponent* PreviewMeshComponent = Cast<UCustomStaticMeshComponent>(PreviewMeshComponentPtr.Get()))
					{
						PreviewMeshComponent->bShouldBeInstanced = ShouldBeInstanced.Contains(PreviewMeshComponent->GetStaticMesh());
						PreviewMeshComponent->MeshProperties = MeshPropertiesMap[PreviewMeshComponent->GetStaticMesh()];
					}
				}
			}
#endif
		}
	}

	PreviewScene->SetFloorOffset(-SceneBounds.Min.Z );

	UpdateOverlayText();

	if (RenderingMaterialType != ERenderingMaterialType::OriginalRenderingMaterial)
	{
		ApplyRenderingMaterial();
	}

	SceneViewport->Invalidate();
}

void SDataprepEditorViewport::FocusViewportOnScene()
{
	EditorViewportClient->FocusViewportOnBox( SceneBounds );
}

TSharedRef<SEditorViewport> SDataprepEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SDataprepEditorViewport::GetExtenders() const
{
	return Extender;
}

void SDataprepEditorViewport::OnFloatingButtonClicked()
{
}

TSharedRef<FEditorViewportClient> SDataprepEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable( new FDataprepEditorViewportClient( SharedThis(this), PreviewScene.ToSharedRef() ) );

	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation( EditorViewportDefs::DefaultPerspectiveViewLocation );
	EditorViewportClient->SetViewRotation( EditorViewportDefs::DefaultPerspectiveViewRotation );
	EditorViewportClient->SetRealtime( true );

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SDataprepEditorViewport::BuildViewportToolbar()
{
	return SNew(SDataprepEditorViewportToolbar, SharedThis(this));
}

void SDataprepEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	FPSText = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "TextBlock.ShadowedText");

	DrawCallsText = SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "TextBlock.ShadowedText");

	Overlay->AddSlot()
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Left)
	.Padding(FMargin(10.0f, 40.0f, 10.0f, 10.0f))
	[
		SAssignNew(OverlayTextVerticalBox, SVerticalBox)
	];

	// this widget will display the current viewed feature level
	Overlay->AddSlot()
	.VAlign(VAlign_Bottom)
	.HAlign(HAlign_Right)
	.Padding(5.0f)
	[
		BuildShaderPlatformWidget()
	];
}

void SDataprepEditorViewport::UpdateOverlayText()
{
	TArray<FOverlayTextItem> TextItems;

	TSet<UStaticMesh*> StaticMeshes;

	int32 NaniteTrianglesCount = 0;
	int32 NaniteVerticesCount = 0;
	int32 TrianglesCount = 0;
	int32 VerticesCount = 0;
	for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponent : PreviewMeshComponents )
	{
		if(UStaticMeshComponent* MeshComponent = PreviewMeshComponent.Get())
		{
			UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh();

			FStaticMeshLODResources& LODResource = StaticMesh->GetRenderData()->LODResources[0];

			if (StaticMesh->IsNaniteEnabled() && StaticMesh->HasValidNaniteData())
			{
				const Nanite::FResources& Resources = *StaticMesh->GetRenderData()->NaniteResourcesPtr.Get();
				if (Resources.RootData.Num() > 0)
				{
					NaniteTrianglesCount += bShowNaniteFallbackMenuChecked ? LODResource.GetNumTriangles() : Resources.NumInputTriangles;
					NaniteVerticesCount += bShowNaniteFallbackMenuChecked ? LODResource.GetNumVertices() : Resources.NumInputVertices;
				}
			}
			else
			{
				TrianglesCount += LODResource.GetNumTriangles();
				VerticesCount += LODResource.GetNumVertices();
			}
			StaticMeshes.Add(StaticMesh);
		}
	}

	if (NaniteTrianglesCount > 0)
	{
		static FText ShowingNaniteFallback = LOCTEXT("ShowingNaniteFallback", "(Showing Fallback)");

		TextItems.Add(FOverlayTextItem(
			FText::Format(LOCTEXT("NaniteEnabled", "Nanite Enabled {0}"), bShowNaniteFallbackMenuChecked ? ShowingNaniteFallback : FText::GetEmpty())));

		if (bShowNaniteFallbackMenuChecked)
		{
			TextItems.Add(FOverlayTextItem(
				FText::Format(LOCTEXT("Fallback_Triangles", "#Fallback_Triangles: {0}"), FText::AsNumber(NaniteTrianglesCount))));

			TextItems.Add(FOverlayTextItem(
				FText::Format(LOCTEXT("Fallback_Vertices", "#Fallback_Vertices: {0}"), FText::AsNumber(NaniteVerticesCount))));
		}
		else
		{
			TextItems.Add(FOverlayTextItem(
				FText::Format(LOCTEXT("Nanite_Triangles", "#Nanite_Triangles: {0}"), FText::AsNumber(NaniteTrianglesCount))));

			TextItems.Add(FOverlayTextItem(
				FText::Format(LOCTEXT("Nanite_Vertices", "#Nanite_Vertices: {0}"), FText::AsNumber(NaniteVerticesCount))));
		}
	}

	TextItems.Add(FOverlayTextItem(
		FText::Format(LOCTEXT("Triangles_F", "#Triangles: {0}"), FText::AsNumber(TrianglesCount))));

	TextItems.Add(FOverlayTextItem(
		FText::Format(LOCTEXT("Vertices_F", "#Vertices: {0}"), FText::AsNumber(VerticesCount))));

	FVector SceneExtents = SceneBounds.GetExtent();
	TextItems.Add(FOverlayTextItem(
		FText::Format(LOCTEXT( "ApproxSize_F", "Approx Size: {0}x{1}x{2}"),
		FText::AsNumber(int32(SceneExtents.X * 2.0f)), // x2 as artists wanted length not radius
		FText::AsNumber(int32(SceneExtents.Y * 2.0f)),
		FText::AsNumber(int32(SceneExtents.Z * 2.0f)))));
	
	OverlayTextVerticalBox->ClearChildren();

	OverlayTextVerticalBox->AddSlot()
	[
		FPSText.ToSharedRef()
	];

	OverlayTextVerticalBox->AddSlot()
	[
		DrawCallsText.ToSharedRef()
	];

	for (const auto& TextItem : TextItems)
	{
		OverlayTextVerticalBox->AddSlot()
		[
			SNew(STextBlock)
			.Text(TextItem.Text)
			.TextStyle(FAppStyle::Get(), TextItem.Style)
		];
	}
}

void SDataprepEditorViewport::UpdatePerfStats()
{
	FNumberFormattingOptions FormatOptions;
	FormatOptions.MinimumFractionalDigits = 0;
	FormatOptions.MaximumFractionalDigits = 0;

	extern ENGINE_API float GAverageFPS;

	DrawCallsAccumulator += GNumDrawCallsRHI[0];
	++CurrentDrawCalls;

	if (CurrentDrawCalls >= DrawCallsUpdateInterval)
	{
		AverageDrawCalls = DrawCallsAccumulator / CurrentDrawCalls;
		CurrentDrawCalls = 0;
		DrawCallsAccumulator = 0;
	}

	FPSText->SetText(FText::Format(LOCTEXT( "FPS_F", "FPS:  {0}"), FText::AsNumber(GAverageFPS, &FormatOptions)));
	DrawCallsText->SetText(FText::Format(LOCTEXT( "DrawCalls_F", "Num Draw Calls:  {0}"), AverageDrawCalls));
}

void SDataprepEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FDataprepEditorViewportCommands& Commands = FDataprepEditorViewportCommands::Get();

	TSharedRef<FDataprepEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::SetShowGrid ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowGridChecked ) );

	CommandList->MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::ToggleShowBounds ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FEditorViewportClient::IsSetShowBoundsChecked ) );

	CommandList->MapAction(
		Commands.SetShowNaniteFallback,
		FExecuteAction::CreateSP(this, &SDataprepEditorViewport::ToggleShowNaniteFallback),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDataprepEditorViewport::IsSetShowNaniteFallbackChecked),
		FIsActionButtonVisible::CreateSP(this, &SDataprepEditorViewport::IsShowNaniteFallbackVisible));

	CommandList->MapAction(
		Commands.ApplyOriginalMaterial,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetRenderingMaterial, ERenderingMaterialType::OriginalRenderingMaterial ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsRenderingMaterialApplied, ERenderingMaterialType::OriginalRenderingMaterial ) );

	CommandList->MapAction(
		Commands.ApplyBackFaceMaterial,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetRenderingMaterial, ERenderingMaterialType::BackFaceRenderingMaterial ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsRenderingMaterialApplied, ERenderingMaterialType::BackFaceRenderingMaterial ) );

	CommandList->MapAction(
		Commands.ApplyXRayMaterial,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetRenderingMaterial, ERenderingMaterialType::XRayRenderingMaterial ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsRenderingMaterialApplied, ERenderingMaterialType::XRayRenderingMaterial ) );

	CommandList->MapAction(
		Commands.ApplyPerMeshMaterial,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetRenderingMaterial, ERenderingMaterialType::PerMeshRenderingMaterial ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsRenderingMaterialApplied, ERenderingMaterialType::PerMeshRenderingMaterial ) );

	CommandList->MapAction(
		Commands.ApplyReflectionMaterial,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetRenderingMaterial, ERenderingMaterialType::ReflectionRenderingMaterial ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsRenderingMaterialApplied, ERenderingMaterialType::ReflectionRenderingMaterial ) );

	CommandList->MapAction(
		Commands.ApplyOutlineSelection,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetSelectionMode, ESelectionModeType::OutlineSelectionMode ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsSetSelectionModeApplied, ESelectionModeType::OutlineSelectionMode ) );

	CommandList->MapAction(
		Commands.ApplyXRaySelection,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::SetSelectionMode, ESelectionModeType::XRaySelectionMode ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsSetSelectionModeApplied, ESelectionModeType::XRaySelectionMode ) );

	CommandList->MapAction(
		Commands.ApplyWireframeMode,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::ToggleWireframeRenderingMode ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsWireframeRenderingModeOn ) );

#ifdef VIEWPORT_EXPERIMENTAL
	CommandList->MapAction(
		Commands.ShowOOBs,
		FExecuteAction::CreateSP( this, &SDataprepEditorViewport::ToggleShowOrientedBox ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDataprepEditorViewport::IsShowOrientedBoxOn ) );
#endif
}

EVisibility SDataprepEditorViewport::OnGetViewportContentVisibility() const
{
	return EVisibility::Visible/*IsVisible() ? EVisibility::Visible : EVisibility::Collapsed*/;
}

void SDataprepEditorViewport::OnFocusViewportToSelection()
{
	if(SelectedPreviewComponents.Num() == 0)
	{
		EditorViewportClient->FocusViewportOnBox( SceneBounds );
		return;
	}

	FBox SelectionBounds;
	SelectionBounds.Init();

	// Compute bounding box of scene to determine camera position and scaling to apply
	for( UStaticMeshComponent* SelectedComponent : SelectedPreviewComponents )
	{
		if(SelectedComponent != nullptr)
		{
			const UStaticMesh* StaticMesh = SelectedComponent->GetStaticMesh();
			const FTransform& ComponentToWorldTransform = SelectedComponent->GetComponentTransform();
			SelectionBounds += StaticMesh->GetExtendedBounds().GetBox().TransformBy( ComponentToWorldTransform );
		}
	}

	FVector Center = SelectionBounds.GetCenter();
	FVector Extents = SelectionBounds.GetExtent() * 1.1f;
	EditorViewportClient->FocusViewportOnBox( FBox(Center - Extents, Center + Extents) );

	SceneViewport->Invalidate();
}

void SDataprepEditorViewport::InitializeDefaultMaterials()
{
	auto CreateMaterialFunc = [](FString MaterialName) -> UMaterial*
	{
		FSoftObjectPath MaterialSoftPath = FSoftObjectPath(FString("/DataprepEditor/") + MaterialName + "." + MaterialName);
		if(UMaterial* BaseMaterial = Cast< UMaterial >( MaterialSoftPath.TryLoad() ))
		{
			// Temporary solution to fix UE-170536: Duplicate material asset
			// TODO: Proper solution is to use material instances for the get go. Will be done in future release.
			UMaterial* Material = DuplicateObject< UMaterial >(BaseMaterial, nullptr);
			Material->ClearFlags(RF_AllFlags);
			Material->SetFlags(RF_Transient);

			return Material;
		}

		check(false);
		return nullptr;
	};

	if(!PreviewMaterial.IsValid())
	{
		TransparentMaterial.Reset();

		PreviewMaterial = TStrongObjectPtr<UMaterial>( CreateMaterialFunc("PreviewMaterial") );
		check( PreviewMaterial.IsValid() );

		DataprepCorePrivateUtils::CompileMaterial(PreviewMaterial.Get());
	}

	if(!TransparentMaterial.IsValid())
	{
		TransparentMaterial = TStrongObjectPtr<UMaterialInstanceConstant>( NewObject<UMaterialInstanceConstant>( GetTransientPackage(), NAME_None, EObjectFlags::RF_Transient) );
		check( TransparentMaterial.IsValid() );

		TransparentMaterial->Parent = PreviewMaterial.Get();

		TransparentMaterial->BasePropertyOverrides.bOverride_BlendMode = true;
		TransparentMaterial->BasePropertyOverrides.BlendMode = BLEND_Translucent;

		TransparentMaterial->SetScalarParameterValueEditorOnly( TEXT("Transparency"), 0.75f );
		TransparentMaterial->SetVectorParameterValueEditorOnly( TEXT( "DiffuseColor" ), FLinearColor::Gray );

		DataprepCorePrivateUtils::CompileMaterial(TransparentMaterial.Get());
	}

	if(!XRayMaterial.IsValid())
	{
		XRayMaterial = TStrongObjectPtr<UMaterial>( CreateMaterialFunc("xray_material") );
		check( XRayMaterial.IsValid() );

		DataprepCorePrivateUtils::CompileMaterial(XRayMaterial.Get());
	}

	if(!BackFaceMaterial.IsValid())
	{
		BackFaceMaterial = TStrongObjectPtr<UMaterial>( CreateMaterialFunc("BackFaceMaterial") );

		DataprepCorePrivateUtils::CompileMaterial(BackFaceMaterial.Get());
	}

	if(!PerMeshMaterial.IsValid())
	{
		PerMeshMaterialInstances.Empty();

		PerMeshMaterial = TStrongObjectPtr<UMaterial>( CreateMaterialFunc("PerMeshMaterial") );
		check( PerMeshMaterial.IsValid() );

		DataprepCorePrivateUtils::CompileMaterial(PerMeshMaterial.Get());
	}

	if(PerMeshMaterialInstances.Num() == 0)
	{
		PerMeshMaterialInstances.AddDefaulted( sizeof(PerMeshColor) );
	}

	for(int32 Index = 0; Index < PerMeshMaterialInstances.Num(); ++Index)
	{
		TStrongObjectPtr<UMaterialInstanceConstant>& PerMeshMaterialInstance = PerMeshMaterialInstances[Index];

		if(!PerMeshMaterialInstance.IsValid())
		{
			PerMeshMaterialInstance = TStrongObjectPtr<UMaterialInstanceConstant>( NewObject<UMaterialInstanceConstant>( GetTransientPackage(), NAME_None, EObjectFlags::RF_Transient) );
			check( PerMeshMaterialInstance.IsValid() );

			PerMeshMaterialInstance->Parent = PerMeshMaterial.Get();

			check( PerMeshMaterialInstance.IsValid() );

			PerMeshMaterialInstance->SetVectorParameterValueEditorOnly( TEXT( "Color" ), FLinearColor( PerMeshColor[Index] ) );

			DataprepCorePrivateUtils::CompileMaterial(PerMeshMaterialInstance.Get());
		}
	}

	if(!ReflectionMaterial.IsValid())
	{
		ReflectionMaterial = TStrongObjectPtr<UMaterial>( CreateMaterialFunc("ReflectionMaterial") );
		check( ReflectionMaterial.IsValid() );

		DataprepCorePrivateUtils::CompileMaterial(ReflectionMaterial.Get());
	}
}

void SDataprepEditorViewport::ReleaseDefaultMaterials()
{
	PreviewMaterial.Reset();
	XRayMaterial.Reset();
	BackFaceMaterial.Reset();
	ReflectionMaterial.Reset();
	TransparentMaterial.Reset();
	PerMeshMaterialInstances.Empty();
	PerMeshMaterial.Reset();
}

void SDataprepEditorViewport::SetSelection(UStaticMeshComponent* Component)
{
	SelectedPreviewComponents.Empty( 1 );
	SelectedPreviewComponents.Add( Component );
	UpdateSelection();
}

void SDataprepEditorViewport::AddToSelection(UStaticMeshComponent* Component)
{
	int32 PrevSelectedCount = SelectedPreviewComponents.Num();

	SelectedPreviewComponents.Add( Component );

	if(PrevSelectedCount != SelectedPreviewComponents.Num())
	{
		UpdateSelection();
	}
}

void SDataprepEditorViewport::RemoveFromSelection(UStaticMeshComponent* Component)
{
	int32 PrevSelectedCount = SelectedPreviewComponents.Num();

	SelectedPreviewComponents.Remove( Component );

	if(PrevSelectedCount != SelectedPreviewComponents.Num())
	{
		UpdateSelection();
	}
}


void SDataprepEditorViewport::ClearSelection(bool bNotify)
{
	if(SelectedPreviewComponents.Num() > 0)
	{
		SelectedPreviewComponents.Empty();
		ApplyRenderingMaterial();

		if(bNotify)
		{
			TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin();

			if(DataprepEditorPtr.IsValid())
			{
				DataprepEditorPtr->SetWorldObjectsSelection( TSet<TWeakObjectPtr<UObject>>(), FDataprepEditor::EWorldSelectionFrom::Viewport );
			}
		}
	}
}

void SDataprepEditorViewport::SelectActors(const TArray< AActor* >& SelectedActors)
{
	// Deselect all if array of selected actors is empty
	if(SelectedActors.Num() == 0)
	{
		ClearSelection();
		return;
	}

	TArray<UStaticMeshComponent*> NewSelectedPreviewComponents;
	NewSelectedPreviewComponents.Reserve( SelectedActors.Num() );

	for(const AActor* SelectedActor : SelectedActors)
	{
		TArray<UStaticMeshComponent*> Components;
		SelectedActor->GetComponents( Components );
		for( UStaticMeshComponent* SelectedComponent : Components )
		{
			// If a mesh is displayable, it should have at least one material
			if(SelectedComponent->GetStaticMesh())
			{
				// If a mesh is displayable, it should have at least one material
				if(UStaticMeshComponent** PreviewComponentPtr = MeshComponentsMapping.Find(SelectedComponent))
				{
					NewSelectedPreviewComponents.Add( *PreviewComponentPtr );
				}
			}
		}
	}

	if(NewSelectedPreviewComponents.Num() == 0)
	{
		ClearSelection();
		return;
	}

	SelectedPreviewComponents.Empty( NewSelectedPreviewComponents.Num() );

	SelectedPreviewComponents.Append( MoveTemp( NewSelectedPreviewComponents ) );

	UpdateSelection(false);
}

void SDataprepEditorViewport::SetActorVisibility(AActor* SceneActor, bool bInVisibility)
{
	TArray<UStaticMeshComponent*> SceneComponents;
	SceneActor->GetComponents(SceneComponents);
	for (UStaticMeshComponent* SceneComponent : SceneComponents)
	{
		UStaticMeshComponent** PreviewComponent = MeshComponentsMapping.Find(SceneComponent);
		if (PreviewComponent)
		{
			(*PreviewComponent)->SetVisibility(bInVisibility);
		}
	}
}

void SDataprepEditorViewport::UpdateSelection(bool bNotify)
{
	TSharedPtr<FDataprepEditor> DataprepEditorPtr = DataprepEditor.Pin();

	if(SelectedPreviewComponents.Num() == 0)
	{
		if(DataprepEditorPtr.IsValid())
		{
			DataprepEditorPtr->SetWorldObjectsSelection( TSet<TWeakObjectPtr<UObject>>(), FDataprepEditor::EWorldSelectionFrom::Viewport );
		}

		ClearSelection();
		return;
	}

	// Apply materials. Only selected ones will be affected
	ApplyRenderingMaterial();

	// UpdateScene Dataprep editor with new selection
	TSet<TWeakObjectPtr<UObject>> SelectedActors;
	SelectedActors.Empty(SelectedPreviewComponents.Num());

	for(UStaticMeshComponent* SelectedComponent : SelectedPreviewComponents)
	{
		UStaticMeshComponent* SceneMeshComponent = MeshComponentsReverseMapping[SelectedComponent];
		SelectedActors.Emplace(SceneMeshComponent->GetOwner());
	}

	if(DataprepEditorPtr.IsValid() && bNotify)
	{
		DataprepEditorPtr->SetWorldObjectsSelection( MoveTemp(SelectedActors), FDataprepEditor::EWorldSelectionFrom::Viewport );
	}

	SceneViewport->Invalidate();
}

bool SDataprepEditorViewport::IsComponentSelected(const UPrimitiveComponent* InPrimitiveComponent)
{
	UPrimitiveComponent* PrimitiveComponent = const_cast<UPrimitiveComponent*>(InPrimitiveComponent);
	return SelectedPreviewComponents.Contains( Cast<UCustomStaticMeshComponent>(PrimitiveComponent) ) && CurrentSelectionMode == ESelectionModeType::OutlineSelectionMode;
}

void SDataprepEditorViewport::SetRenderingMaterial(ERenderingMaterialType InRenderingMaterialType)
{
	if(RenderingMaterialType != InRenderingMaterialType)
	{
		RenderingMaterialType = InRenderingMaterialType;
		ApplyRenderingMaterial();
	}
}

void SDataprepEditorViewport::ToggleWireframeRenderingMode()
{
	bWireframeRenderingMode = !bWireframeRenderingMode;

	for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponent : PreviewMeshComponents )
	{
		if(UCustomStaticMeshComponent* CustomComponent = Cast<UCustomStaticMeshComponent>(PreviewMeshComponent))
		{
			CustomComponent->bForceWireframe = bWireframeRenderingMode;
			PreviewMeshComponent->MarkRenderStateDirty();
		}
	}

	SceneViewport->Invalidate();
}

void SDataprepEditorViewport::SetSelectionMode(ESelectionModeType InSelectionMode)
{
	if(CurrentSelectionMode != InSelectionMode)
	{
		CurrentSelectionMode = InSelectionMode;

		ApplyRenderingMaterial();
	}
}

UMaterialInterface* SDataprepEditorViewport::GetRenderingMaterial(UStaticMeshComponent* PreviewMeshComponent)
{
	switch(RenderingMaterialType)
	{
		case ERenderingMaterialType::XRayRenderingMaterial:
		{
			return XRayMaterial.Get();
		}

		case ERenderingMaterialType::BackFaceRenderingMaterial:
		{
			return BackFaceMaterial.Get();
		}

		case ERenderingMaterialType::PerMeshRenderingMaterial:
		{
			if(UCustomStaticMeshComponent* CustomComponent = Cast<UCustomStaticMeshComponent>(PreviewMeshComponent))
			{
				return PerMeshMaterialInstances[CustomComponent->MeshColorIndex].Get();
			}
		}

		case ERenderingMaterialType::ReflectionRenderingMaterial:
		{
			return ReflectionMaterial.Get();
		}

		case ERenderingMaterialType::OriginalRenderingMaterial:
		default:
		{
			break;
		}
	}

	return nullptr;
}

void SDataprepEditorViewport::ApplyRenderingMaterial()
{
	auto ApplyMaterial = [&](UStaticMeshComponent* PreviewMeshComponent)
	{
		UMaterialInterface* RenderingMaterial = GetRenderingMaterial( PreviewMeshComponent );

		UStaticMeshComponent* SceneMeshComponent = MeshComponentsReverseMapping[PreviewMeshComponent];

		UStaticMesh* StaticMesh = SceneMeshComponent->GetStaticMesh();

		for(int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
		{
			UMaterialInterface* MaterialInterface = SceneMeshComponent->GetMaterial(Index);

			if(MaterialInterface == nullptr)
			{
				MaterialInterface = StaticMesh->GetStaticMaterials()[Index].MaterialInterface;
			}

			PreviewMeshComponent->SetMaterial( Index, RenderingMaterial ? RenderingMaterial : MaterialInterface );
		}

		PreviewMeshComponent->MarkRenderStateDirty();
	};

	if(SelectedPreviewComponents.Num() > 0)
	{
		switch(CurrentSelectionMode)
		{
			case ESelectionModeType::XRaySelectionMode:
			{
				// Apply transparent material on all mesh components
				for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents )
				{
					if(UStaticMeshComponent* PreviewMeshComponent = PreviewMeshComponentPtr.Get())
					{
						UStaticMesh* StaticMesh = PreviewMeshComponent->GetStaticMesh();

						for(int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
						{
							PreviewMeshComponent->SetMaterial( Index, TransparentMaterial.Get() );
						}

						PreviewMeshComponent->MarkRenderStateDirty();
					}
				}

				// Apply rendering material only on selected mesh components
				for( UStaticMeshComponent* PreviewMeshComponent : SelectedPreviewComponents )
				{
					ApplyMaterial( PreviewMeshComponent );
				}
			}
			break;

			default:
			case ESelectionModeType::OutlineSelectionMode:
			{
				for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents )
				{
					if(UStaticMeshComponent* PreviewMeshComponent = PreviewMeshComponentPtr.Get())
					{
						ApplyMaterial( PreviewMeshComponent );
					}
				}
			}
			break;
		}
	}
	else
	{
		for( const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents )
		{
			if(UStaticMeshComponent* PreviewMeshComponent = PreviewMeshComponentPtr.Get())
			{
				ApplyMaterial( PreviewMeshComponent );
			}
		}
	}

	SceneViewport->Invalidate();
}

void SDataprepEditorViewport::LoadDefaultSettings()
{
	// Find index of Dataprep's viewport's settings
	const TCHAR* DataprepViewportSettingProfileName = TEXT("DataprepViewportSetting");

	UAssetViewerSettings* DefaultSettings = UAssetViewerSettings::Get();
	check(DefaultSettings);

	for(int32 Index = 0; Index < DefaultSettings->Profiles.Num(); ++Index)
	{
		if(DefaultSettings->Profiles[Index].ProfileName == DataprepViewportSettingProfileName)
		{
			AssetViewerProfileIndex = Index;
			break;
		}
	}

	// No profile found, create one
	if(AssetViewerProfileIndex == INDEX_NONE)
	{
		FPreviewSceneProfile Profile = DefaultSettings->Profiles[0];

		Profile.bSharedProfile = false;
		Profile.ProfileName = DataprepViewportSettingProfileName;
		AssetViewerProfileIndex = DefaultSettings->Profiles.Num();

		DefaultSettings->Profiles.Add( Profile );
		const bool bWarnIfFail = false;
		DefaultSettings->Save(bWarnIfFail);
	}

	// Update the profile with the settings for the project
	FPreviewSceneProfile& DataprepViewportSettingProfile = 	DefaultSettings->Profiles[AssetViewerProfileIndex];

	// Read default settings, tessellation and import, for Datasmith file producer
	const FString DataprepEditorIni = FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s%s/%s.ini"), *FPaths::GeneratedConfigDir(), ANSI_TO_TCHAR(FPlatformProperties::PlatformName()), TEXT("DataprepEditor")));

	const TCHAR* ViewportSectionName = TEXT("ViewportSettings");
	if(GConfig->DoesSectionExist( ViewportSectionName, DataprepEditorIni ))
	{
		FString EnvironmentCubeMapPath = GConfig->GetStr( ViewportSectionName, TEXT("EnvironmentCubeMap"), DataprepEditorIni);

		if(EnvironmentCubeMapPath != DataprepViewportSettingProfile.EnvironmentCubeMapPath)
		{
			// Check that the Cube map does exist
			FSoftObjectPath EnvironmentCubeMap( EnvironmentCubeMapPath );
			UObject* LoadedObject = EnvironmentCubeMap.TryLoad();

			while (UObjectRedirector* Redirector = Cast<UObjectRedirector>( LoadedObject ))
			{
				LoadedObject = Redirector->DestinationObject;
			}

			// Good to go, update the profile's related parameters
			if( UTextureCube* LoadedTextureCube = Cast<UTextureCube>( LoadedObject ) )
			{
				DataprepViewportSettingProfile.EnvironmentCubeMapPath = EnvironmentCubeMapPath;
				DataprepViewportSettingProfile.EnvironmentCubeMap = LoadedTextureCube;
			}
		}
	}
}

void SDataprepEditorViewport::SetShowNaniteFallback(bool bShow)
{
	bShowNaniteFallbackMenuChecked = bShow;

	for (const TWeakObjectPtr< UStaticMeshComponent >& PreviewMeshComponentPtr : PreviewMeshComponents)
	{
		if (UCustomStaticMeshComponent* PreviewMeshComponent = Cast<UCustomStaticMeshComponent>(PreviewMeshComponentPtr.Get()))
		{
			const UStaticMesh* StaticMesh = PreviewMeshComponent->GetStaticMesh();
			if (StaticMesh && StaticMesh->IsNaniteEnabled())
			{
				PreviewMeshComponent->bDisplayNaniteFallbackMesh = bShowNaniteFallbackMenuChecked;
				PreviewMeshComponent->MarkRenderStateDirty();
			}
		}
	}

	UpdateOverlayText();
}

//
// FDataprepEditorViewportClient Class
//

FDataprepEditorViewportClient::FDataprepEditorViewportClient(const TSharedRef<SEditorViewport>& InDataprepEditorViewport, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene)
	: FEditorViewportClient( nullptr, &InPreviewScene.Get(), InDataprepEditorViewport )
	, AdvancedPreviewScene( nullptr )
	, DataprepEditorViewport( nullptr )
{
	EngineShowFlags.SetSelectionOutline( true );

	if (EditorViewportWidget.IsValid())
	{
		DataprepEditorViewport = static_cast<SDataprepEditorViewport*>( EditorViewportWidget.Pin().Get() );
	}

	AdvancedPreviewScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	check(AdvancedPreviewScene);

	AdvancedPreviewScene->SetProfileIndex( SDataprepEditorViewport::AssetViewerProfileIndex );
}

bool FDataprepEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = false;
	
	// #ueent_todo: Put code for specific handling

	return bHandled ? true : FEditorViewportClient::InputKey(EventArgs);
}

void FDataprepEditorViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	if(DataprepEditorViewport != nullptr)
	{
		if(HitProxy != nullptr)
		{
			if( HitProxy->IsA( HActor::StaticGetType() ) )
			{
				// A static mesh component has been selected
				if( UStaticMeshComponent* Component = const_cast<UStaticMeshComponent*>(Cast<UStaticMeshComponent>( ((HActor*)HitProxy)->PrimComponent )))
				{
					// A static mesh component part of the ones to preview has been selected
					if( DataprepEditorViewport->IsAPreviewComponent(Component) )
					{
						// Applies the selection logic
						if(Key == EKeys::LeftMouseButton)
						{
							if( DataprepEditorViewport->IsSelected(Component) )
							{
								if( IsCtrlPressed() || IsShiftPressed() )
								{
									DataprepEditorViewport->RemoveFromSelection(Component);
								}
							}
							else
							{
								if( IsCtrlPressed() || IsShiftPressed() )
								{
									DataprepEditorViewport->AddToSelection(Component);
								}
								else
								{
									DataprepEditorViewport->SetSelection(Component);
								}
							}

							return;
						}
					}
				}
			}
		}
		// No geometry picked, de-select all
		else if(Key == EKeys::LeftMouseButton)
		{
			DataprepEditorViewport->ClearSelection(true);
			return;
		}
	}

	// Nothing to be done, delegate to base class
	FEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY );
}

void FDataprepEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	DataprepEditorViewport->UpdatePerfStats();
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
}

void FDataprepEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

#ifdef VIEWPORT_EXPERIMENTAL
	if(DataprepEditorViewport->IsShowOrientedBoxOn())
	{
		for(TWeakObjectPtr< UStaticMeshComponent >& ComponentPtr : DataprepEditorViewport->PreviewMeshComponents)
		{
			if(UCustomStaticMeshComponent* MeshComponent = Cast<UCustomStaticMeshComponent>(ComponentPtr.Get()))
			{
				FPrototypeOrientedBox& Box = MeshComponent->MeshProperties;
				FTransform Transform = MeshComponent->GetComponentToWorld();

				FVector Positions[8];

				Positions[0] = Transform.TransformPosition(Box.Center + (Box.HalfExtents.X * Box.LocalXAxis) + (Box.HalfExtents.Y * Box.LocalYAxis) + (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[1] = Transform.TransformPosition(Box.Center - (Box.HalfExtents.X * Box.LocalXAxis) + (Box.HalfExtents.Y * Box.LocalYAxis) + (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[2] = Transform.TransformPosition(Box.Center - (Box.HalfExtents.X * Box.LocalXAxis) - (Box.HalfExtents.Y * Box.LocalYAxis) + (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[3] = Transform.TransformPosition(Box.Center + (Box.HalfExtents.X * Box.LocalXAxis) - (Box.HalfExtents.Y * Box.LocalYAxis) + (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[4] = Transform.TransformPosition(Box.Center + (Box.HalfExtents.X * Box.LocalXAxis) + (Box.HalfExtents.Y * Box.LocalYAxis) - (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[5] = Transform.TransformPosition(Box.Center - (Box.HalfExtents.X * Box.LocalXAxis) + (Box.HalfExtents.Y * Box.LocalYAxis) - (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[6] = Transform.TransformPosition(Box.Center - (Box.HalfExtents.X * Box.LocalXAxis) - (Box.HalfExtents.Y * Box.LocalYAxis) - (Box.HalfExtents.Z * Box.LocalZAxis));
				Positions[7] = Transform.TransformPosition(Box.Center + (Box.HalfExtents.X * Box.LocalXAxis) - (Box.HalfExtents.Y * Box.LocalYAxis) - (Box.HalfExtents.Z * Box.LocalZAxis));

				int32 Indices[24] = { 0, 1, 1, 2, 2, 3, 3, 0, 4, 5, 5, 6, 6, 7, 7, 4, 0, 4, 1, 5, 2, 6, 3, 7 };

				for(int32 Index = 0; Index < 24; Index += 2)
				{
					PDI->DrawLine( Positions[Indices[Index + 0]], Positions[Indices[Index + 1]], MeshComponent->bShouldBeInstanced ? FColor( 255, 0, 0 ) : FColor( 255, 255, 0 ), SDPG_World );
				}

				FVector TransformedCenter = Transform.TransformPosition(Box.Center);
				PDI->DrawLine( TransformedCenter, Transform.TransformPosition(Box.Center + /*Box.Moments.X*/10.0f * Box.LocalXAxis), FColor( 0, 0, 255 ), SDPG_World );
				PDI->DrawLine( TransformedCenter, Transform.TransformPosition(Box.Center + /*Box.Moments.Y*/10.0f * Box.LocalYAxis), FColor( 0, 0, 255 ), SDPG_World );
				PDI->DrawLine( TransformedCenter, Transform.TransformPosition(Box.Center + /*Box.Moments.Z*/10.0f * Box.LocalZAxis), FColor( 0, 0, 255 ), SDPG_World );
			}
		}
	}
#endif
}

void FDataprepEditorViewportClient::Draw(FViewport* InViewport, FCanvas* InCanvas)
{
	FEditorViewportClient::Draw(InViewport, InCanvas);
}

///////////////////////////////////////////////////////////
// SDataprepEditorViewportToolbar

void SDataprepEditorViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	// Create default widgets in toolbar: View, 
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> SDataprepEditorViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(FDataprepEditorViewportCommands::Get().SetShowGrid);
		ShowMenuBuilder.AddMenuEntry(FDataprepEditorViewportCommands::Get().SetShowBounds);
		ShowMenuBuilder.AddMenuEntry(FDataprepEditorViewportCommands::Get().SetShowNaniteFallback);
	}

	// #ueent_remark: Look at SAnimViewportToolBar::GenerateShowMenu in SAnimViewportToolBar.cpp for adding ShowFlagFilter to the Show menu

	return ShowMenuBuilder.MakeWidget();
}

void SDataprepEditorViewportToolbar::ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const
{
	const FMargin ToolbarSlotPadding(2.0f, 2.0f);
	const FMargin ToolbarButtonPadding(2.0f, 0.0f);
	static const FName DefaultForegroundName("DefaultForeground");

	if (!MainBoxPtr.IsValid())
	{
		return;
	}

	MainBoxPtr->AddSlot()
	.AutoWidth()
	.Padding(ToolbarSlotPadding)
	[
		SNew(SEditorViewportToolbarMenu)
		.Label( LOCTEXT("DataprepEditor_Rendering", "Rendering") )
		.ToolTipText(LOCTEXT("DataprepEditor_RenderingTooltip", "Rendering Options. Use this enable/disable the rendering of types of meshes."))
		.ParentToolBar( ParentToolBarPtr )
		.Cursor(EMouseCursor::Default)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("RenderingMenuButton")))
		.OnGetMenuContent( this, &SDataprepEditorViewportToolbar::GenerateRenderingMenu)
	];

#ifdef VIEWPORT_EXPERIMENTAL
	MainBoxPtr->AddSlot()
	.AutoWidth()
	.Padding(ToolbarSlotPadding)
	[
		SNew(SEditorViewportToolbarMenu)
		.Label( LOCTEXT("DataprepEditor_Experimental", "Experimental") )
		.ToolTipText(LOCTEXT("DataprepEditor_ExperimentalTooltip", "Experimental viewing modes or actions."))
		.ParentToolBar( ParentToolBarPtr )
		.Cursor(EMouseCursor::Default)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ExperimentalMenuButton")))
		.OnGetMenuContent( this, &SDataprepEditorViewportToolbar::GenerateExperimentalMenu)
	];
#endif
}

bool SDataprepEditorViewportToolbar::IsViewModeSupported(EViewModeIndex ViewModeIndex) const
{
	// #ueent_todo: Eliminate view mode we do not want
	return true;
}

TSharedRef<SWidget> SDataprepEditorViewportToolbar::GenerateRenderingMenu() const
{
	TSharedPtr<FExtender> MenuExtender = GetInfoProvider().GetExtenders();
	TSharedPtr<SEditorViewport> Viewport = GetInfoProvider().GetViewportWidget();
	TSharedPtr<FUICommandList> CommandList = Viewport->GetCommandList();

	const FDataprepEditorViewportCommands& Commands = FDataprepEditorViewportCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList, MenuExtender);

	MenuBuilder.PushCommandList( CommandList.ToSharedRef() );
	MenuBuilder.PushExtender( MenuExtender.ToSharedRef() );
	{
		MenuBuilder.BeginSection("DataprepEditorViewportRenderingMenu", LOCTEXT("DataprepEditor_RenderingMaterial", "Materials"));
		MenuBuilder.AddMenuEntry(Commands.ApplyOriginalMaterial);
		MenuBuilder.AddMenuEntry(Commands.ApplyBackFaceMaterial);
#ifdef VIEWPORT_EXPERIMENTAL
		MenuBuilder.AddMenuEntry(Commands.ApplyXRayMaterial);
#endif
		MenuBuilder.AddMenuEntry(Commands.ApplyPerMeshMaterial);
		MenuBuilder.AddMenuEntry(Commands.ApplyReflectionMaterial);
		MenuBuilder.EndSection();
		MenuBuilder.BeginSection("DataprepEditorViewportRenderingMenu", LOCTEXT("DataprepEditor_SelectionMode", "Selection"));
		MenuBuilder.AddMenuEntry(Commands.ApplyOutlineSelection);
		MenuBuilder.AddMenuEntry(Commands.ApplyXRaySelection);
		MenuBuilder.EndSection();
		MenuBuilder.BeginSection("DataprepEditorViewportRenderingMenu", LOCTEXT("DataprepEditor_RenderingMode", "Modes"));
		MenuBuilder.AddMenuEntry(Commands.ApplyWireframeMode);
		MenuBuilder.EndSection();
	}

	MenuBuilder.PopCommandList();
	MenuBuilder.PopExtender();

	return MenuBuilder.MakeWidget();
}

AActor* SDataprepEditorViewport::CreateActor(USceneComponent* ParentComponent, const TCHAR* ActorName)
{
	UWorld* PreviewSceneWorld = PreviewScene->GetWorld();

	AActor* Actor = PreviewSceneWorld->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity);

	if (Actor->GetRootComponent() == nullptr)
	{
		USceneComponent* RootComponent = NewObject< USceneComponent >(Actor, USceneComponent::StaticClass(), ActorName, RF_NoFlags);
		Actor->SetRootComponent(RootComponent);
	}

	if (ParentComponent)
	{
		Actor->GetRootComponent()->AttachToComponent(ParentComponent, FAttachmentTransformRules::KeepRelativeTransform);
	}

	Actor->RegisterAllComponents();

	return Actor;
}

#ifdef VIEWPORT_EXPERIMENTAL
TSharedRef<SWidget> SDataprepEditorViewportToolbar::GenerateExperimentalMenu() const
{
	TSharedPtr<FExtender> MenuExtender = GetInfoProvider().GetExtenders();
	TSharedPtr<SEditorViewport> Viewport = GetInfoProvider().GetViewportWidget();
	TSharedPtr<FUICommandList> CommandList = Viewport->GetCommandList();

	const FDataprepEditorViewportCommands& Commands = FDataprepEditorViewportCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList, MenuExtender);

	MenuBuilder.PushCommandList( CommandList.ToSharedRef() );
	MenuBuilder.PushExtender( MenuExtender.ToSharedRef() );
	{
		MenuBuilder.BeginSection("DataprepEditorViewportExperimentalMenu", LOCTEXT("DataprepEditor_Experimental_Viewing", "Viewing"));
		MenuBuilder.AddMenuEntry(Commands.ShowOOBs);
		MenuBuilder.EndSection();
		MenuBuilder.BeginSection("DataprepEditorViewportExperimentalMenu", LOCTEXT("DataprepEditor_Experimental_Actions", "Actions"));
		MenuBuilder.EndSection();
	}

	MenuBuilder.PopCommandList();
	MenuBuilder.PopExtender();

	return MenuBuilder.MakeWidget();
}

void SDataprepEditorViewport::ToggleShowOrientedBox()
{
	bShowOrientedBox = !bShowOrientedBox;

	SceneViewport->Invalidate();
}
#endif

//////////////////////////////////////////////////////////////////////////
// FDataprepEditorViewportCommands

void FDataprepEditorViewportCommands::RegisterCommands()
{
	// Show menu
	UI_COMMAND(SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::G));
	UI_COMMAND(SetShowBounds, "Bounds", "Toggles display of the bounds of the selected component.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::B));
	UI_COMMAND(SetShowNaniteFallback, "Nanite Fallback", "Toggles the display of the Nanite fallback mesh.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::N));

	// Rendering Material
	UI_COMMAND(ApplyOriginalMaterial, "None", "Display all meshes with original materials.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ApplyBackFaceMaterial, "BackFace", "Display front face and back face of triangles with different colors.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ApplyXRayMaterial, "XRay", "Use XRay material to render meshes.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ApplyPerMeshMaterial, "MultiColored", "Assign a different color for each rendered mesh.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ApplyReflectionMaterial, "ReflectionLines", "Use reflective material to show lines of reflection.", EUserInterfaceActionType::RadioButton, FInputChord());

	// Selection Mode
	UI_COMMAND(ApplyOutlineSelection, "Outline", "Outline selected meshes with a colored contour.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ApplyXRaySelection, "XRay", "Use XRay material on non selected meshes.", EUserInterfaceActionType::RadioButton, FInputChord());

	// Rendering Mode
	UI_COMMAND(ApplyWireframeMode, "Wireframe", "Display all meshes in wireframe.", EUserInterfaceActionType::ToggleButton, FInputChord());

#ifdef VIEWPORT_EXPERIMENTAL
	// Experimental
	UI_COMMAND(ShowOOBs, "OrientedBox", "Display object oriented bounding boxes.", EUserInterfaceActionType::ToggleButton, FInputChord());
#endif
}

// Copied from UStaticMeshComponent::CreateSceneProxy
FPrimitiveSceneProxy* UCustomStaticMeshComponent::CreateSceneProxy()
{
	UStaticMesh* Mesh = GetStaticMesh();

	if (Mesh == nullptr || Mesh->GetRenderData() == nullptr)
	{
		return nullptr;
	}

	// Do not overwrite proxy if Nanite supported and there is built Nanite data for the static mesh
	// #ueent_todo: Track changes to Nanite::FSceneProxy code in case wireframe display is supported
	if (ShouldCreateNaniteProxy() && !bDisplayNaniteFallbackMesh)
	{
		return UStaticMeshComponent::CreateSceneProxy();
	}

	FSceneInterface* Scene = GetScene();
	const EShaderPlatform ShaderPlatform = Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform;
	if (Mesh->HasValidNaniteData() && Mesh->HasNaniteFallbackMesh(ShaderPlatform))
	{
		UE_LOGF(LogStaticMesh, Warning, "Unable to create a proxy for StaticMeshComponent [%ls] because it doesn't have a fallback mesh.", *GetFullName());
		return nullptr;
	}

	const FStaticMeshLODResourcesArray& LODResources = Mesh->GetRenderData()->LODResources;
	if (LODResources.Num() == 0	|| LODResources[FMath::Clamp<int32>(Mesh->GetDefaultMinLOD(), 0, LODResources.Num()-1)].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (LOD problems)", *GetFullName());
		return nullptr;
	}
	LLM_SCOPE(ELLMTag::StaticMesh);

	FPrimitiveSceneProxy* Proxy = ::new FStaticMeshSceneProxyExt(this, false);
#if STATICMESH_ENABLE_DEBUG_RENDERING
	SendRenderDebugPhysics(Proxy);
#endif

	return Proxy;
}

namespace DataprepEditor3DPreviewUtils
{
	void BuildStaticMeshes(TSet<UStaticMesh*>& StaticMeshes, TArray<UStaticMesh*>& BuiltMeshes)
	{
		ViewportDebug::FTimeLogger LapTimeLogger( TEXT("Building static meshes") );

		BuiltMeshes.Empty( StaticMeshes.Num() );

		TArray< TArray<FMeshBuildSettings> > StaticMeshesSettings;
		StaticMeshesSettings.Reserve( StaticMeshes.Num() );

		for(UStaticMesh* StaticMesh : StaticMeshes)
		{
			if(StaticMesh && (!StaticMesh->GetRenderData() || !StaticMesh->GetRenderData()->IsInitialized()))
			{
				BuiltMeshes.Add( StaticMesh );
			}
		}

		if(BuiltMeshes.Num() > 0)
		{
			FScopedSlowTask SlowTask( BuiltMeshes.Num(), LOCTEXT("BuildStaticMeshes_Title", "Building static meshes ...") );
			SlowTask.MakeDialog(false);

			auto ProgressFunction = [&](UStaticMesh* StaticMesh)
			{
				SlowTask.EnterProgressFrame(1.f, FText::FromString(FString::Printf(TEXT("Building Static Mesh %s ..."), *StaticMesh->GetName())));
				return true;
			};

			// Start with the biggest mesh first to help balancing tasks on threads
			BuiltMeshes.Sort(
				[](const UStaticMesh& Lhs, const UStaticMesh& Rhs) 
			{ 
				int32 LhsVerticesNum = Lhs.IsMeshDescriptionValid(0) ? Lhs.GetMeshDescription(0)->Vertices().Num() : 0;
				int32 RhsVerticesNum = Rhs.IsMeshDescriptionValid(0) ? Rhs.GetMeshDescription(0)->Vertices().Num() : 0;

				return LhsVerticesNum > RhsVerticesNum;
			}
			);

			//Cache the BuildSettings and update them before building the meshes.
			for (UStaticMesh* StaticMesh : BuiltMeshes)
			{
				int32 NumSourceModels = StaticMesh->GetNumSourceModels();
				TArray<FMeshBuildSettings> BuildSettings;
				BuildSettings.Reserve(NumSourceModels);

				for (int32 LodIndex = 0; LodIndex < NumSourceModels; LodIndex++)
				{
					FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LodIndex);

					BuildSettings.Add(SourceModel.BuildSettings);

					SourceModel.BuildSettings.bGenerateLightmapUVs = false;
					SourceModel.BuildSettings.bRecomputeNormals = false;
					SourceModel.BuildSettings.bRecomputeTangents = false;
					SourceModel.BuildSettings.bBuildReversedIndexBuffer = false;
				}

				StaticMeshesSettings.Add(MoveTemp(BuildSettings));				
			}

			// Disable warnings from LogStaticMesh. Not useful
			ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
			LogStaticMesh.SetVerbosity( ELogVerbosity::Error );

			UStaticMesh::BatchBuild(BuiltMeshes, true, ProgressFunction);

			// Restore LogStaticMesh verbosity
			LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );

			for(int32 Index = 0; Index < BuiltMeshes.Num(); ++Index)
			{
				UStaticMesh* StaticMesh = BuiltMeshes[Index];
				TArray<FMeshBuildSettings>& PrevBuildSettings = StaticMeshesSettings[Index];

				int32 NumSourceModels = StaticMesh->GetNumSourceModels();

				for(int32 SourceModelIndex = 0; SourceModelIndex < NumSourceModels; ++SourceModelIndex)
				{
					StaticMesh->GetSourceModel(SourceModelIndex).BuildSettings = PrevBuildSettings[SourceModelIndex];
				}

				for ( FStaticMeshLODResources& LODResources : StaticMesh->GetRenderData()->LODResources )
				{
					LODResources.bHasColorVertexData = true;
				}
			}
		}
	}

	void FindMeshComponents(const AActor * InActor, TArray<UStaticMeshComponent*>& MeshComponents, bool bRecursive )
	{
		if(InActor == nullptr)
		{
			return;
		}

		TArray<UStaticMeshComponent*> StaticMeshComponents;
		InActor->GetComponents( StaticMeshComponents );
		for(UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			MeshComponents.Add( StaticMeshComponent );
		}

		if(bRecursive)
		{
			TArray<AActor*> Children;
			InActor->GetAttachedActors( Children );
			for(AActor* ChildActor : Children)
			{
				FindMeshComponents( ChildActor, MeshComponents, bRecursive );
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
