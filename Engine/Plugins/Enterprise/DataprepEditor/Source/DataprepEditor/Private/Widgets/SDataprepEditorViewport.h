// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef VIEWPORT_EXPERIMENTAL
#include "ViewportExperimental.h"
#endif

#include "Components/StaticMeshComponent.h"
#include "EditorViewportClient.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"

#include "SDataprepEditorViewport.generated.h"

class AActor;
class FAdvancedPreviewScene;
class FDataprepEditor;
class SDataprepEditorViewport;
class SVerticalBox;
class STextBlock;
class SViewportToolBar;
class UMaterialInterface;
class UMaterialInstanceConstant;
class UStaticMesh;
class UStaticMeshComponent;

/**
 * Custom class deriving from UStaticMeshComponent to allow the display of individual meshes in wireframe
 * @note: This technique was inspired from USkinnedMeshComponent
 */
UCLASS()
class DATAPREPEDITOR_API UCustomStaticMeshComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

#ifdef VIEWPORT_EXPERIMENTAL
	FPrototypeOrientedBox MeshProperties;
	bool bShouldBeInstanced;
#endif

protected:
	bool bForceWireframe;
	int32 MeshColorIndex;

	friend class FStaticMeshSceneProxyExt;
	friend class SDataprepEditorViewport;
};

class FDataprepEditorViewportCommands : public TCommands<FDataprepEditorViewportCommands>
{
public:
	FDataprepEditorViewportCommands()
		: TCommands<FDataprepEditorViewportCommands>(
			TEXT("DataprepEditorViewport"), // Context name for fast lookup
			NSLOCTEXT( "DataprepEditorViewportCommands", "DataprepEditorViewportText", "Dataprep Editor Viewport"), // Localized context name for displaying
			NAME_None, // Parent
			FAppStyle::GetAppStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowNaniteFallback;

	TSharedPtr<FUICommandInfo> ApplyOriginalMaterial;
	TSharedPtr<FUICommandInfo> ApplyBackFaceMaterial;
	TSharedPtr<FUICommandInfo> ApplyXRayMaterial;
	TSharedPtr<FUICommandInfo> ApplyPerMeshMaterial;
	TSharedPtr<FUICommandInfo> ApplyReflectionMaterial;

	TSharedPtr<FUICommandInfo> ApplyOutlineSelection;
	TSharedPtr<FUICommandInfo> ApplyXRaySelection;

	TSharedPtr<FUICommandInfo> ApplyWireframeMode;

#ifdef VIEWPORT_EXPERIMENTAL
	TSharedPtr<FUICommandInfo> ShowOOBs;
#endif
};

// In-viewport toolbar widget used in the Dataprep editor
class SDataprepEditorViewportToolbar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SDataprepEditorViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider);

	// SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override;
	virtual void ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const override;
	// End of SCommonEditorViewportToolbarBase

	// SViewportToolBar interface
	virtual bool IsViewModeSupported(EViewModeIndex ViewModeIndex) const override; 
	// End of SViewportToolBar

private:
	/** Generates the toolbar rendering menu content */
	TSharedRef<SWidget> GenerateRenderingMenu() const;

#ifdef VIEWPORT_EXPERIMENTAL
	/** Generates the toolbar experimental menu content */
	TSharedRef<SWidget> GenerateExperimentalMenu() const;
#endif
};

class FDataprepEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FDataprepEditorViewportClient>
{
public:
	FDataprepEditorViewportClient(const TSharedRef<SEditorViewport>& InDataprepEditorViewport, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene);
	~FDataprepEditorViewportClient() {}

	// FEditorViewportClient interface
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual void DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas ) override;
	// End of FEditorViewportClient interface

	/** FViewElementDrawer interface */
	virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	// End of FViewElementDrawer interface

private:

	/** Stored pointer to the preview scene in which the preview mesh component are shown */
	FAdvancedPreviewScene* AdvancedPreviewScene;

	/** Stored pointer to the viewport in which the preview mesh component are shown */
	SDataprepEditorViewport* DataprepEditorViewport;
};

class SDataprepEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS( SDataprepEditorViewport ){}
		SLATE_ARGUMENT( UWorld*, WorldToPreview)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FDataprepEditor> InDataprepEditor);

	SDataprepEditorViewport();
	~SDataprepEditorViewport();

	/** UpdateScene the viewport with the current content of the editor's preview world */
	void UpdateScene();

	/** Move the camera so that we can see the full scene */
	void FocusViewportOnScene();

	/** Delete all the rendering data created by the viewport for display */
	void ClearScene();

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	void SelectActors( const TArray< AActor* >& SelectedActors );

	/** Change visibility of preview mesh components */
	void SetActorVisibility(AActor* SceneActor, bool bInVisibility);

	/** Load the settings affecting the viewport, i.e. environment map */
	static void LoadDefaultSettings();

	/** Release the materials used for the different rendering options in the viewport */
	static void ReleaseDefaultMaterials();

	int32 GetDrawCallsAverage() const
	{
		return AverageDrawCalls;
	}

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;

private:
	enum class ERenderingMaterialType : uint8
	{
		OriginalRenderingMaterial,
		BackFaceRenderingMaterial,
		XRayRenderingMaterial,
		UVStretchRenderingMaterial,
		PerMeshRenderingMaterial,
		ReflectionRenderingMaterial,
		MaxRenderingMaterial
	};

	enum class ESelectionModeType : uint8
	{
		OutlineSelectionMode,
		XRaySelectionMode,
		MaxSelectionMode
	};

	enum class ERenderingMode : uint8
	{
		NormalRenderingMode,
		WireframeRenderingMode,
		MaxRenderingMode
	};

	/** Struct defining the text and its style of each item in the overlay widget */
	struct FOverlayTextItem
	{
		explicit FOverlayTextItem(const FText& InText, const FName& InStyle = "TextBlock.ShadowedText")
			: Text(InText), Style(InStyle)
		{}

		FText Text;
		FName Style;
	};

	/** Returns true if the given mesh component is part of the mesh components to preview */
	bool IsAPreviewComponent( const UStaticMeshComponent* Component ) const { return MeshComponentsReverseMapping.Contains( Component ); }

	/** Clear the selection and if required notify */
	void ClearSelection(bool bNotify = false);

	/** Set the selection to this preview mesh component and calls UpdateSelection */
	void SetSelection( UStaticMeshComponent* Component );

	/** Add this preview mesh component to the selection and calls UpdateSelection */
	void AddToSelection( UStaticMeshComponent* Component );

	/** Remove this preview mesh component from the selection and calls UpdateSelection */
	void RemoveFromSelection( UStaticMeshComponent* Component );

	/** Return if preview mesh component is part of the selection */
	bool IsSelected( UStaticMeshComponent* Component ) const { return SelectedPreviewComponents.Contains( Component ); }

	/**
	 * Applies the selection material on all selected mesh components.
	 * If none is selected, applies current rendering material
	 */
	void UpdateSelection(bool bNotify = true);

	/** Initialize the materials used for the different rendering options */
	void InitializeDefaultMaterials();

	/** Set the material to apply on the preview mesh components */
	void SetRenderingMaterial(ERenderingMaterialType InRenderingMaterial);

	/** Returns true if the rendering material is currently used */
	bool IsRenderingMaterialApplied(ERenderingMaterialType InRenderingMaterialType) const
	{
		return RenderingMaterialType == InRenderingMaterialType;
	}

	/**
	 * Modifies materials used on all the preview static mesh components based on value of RenderingMaterialType 
	 */
	void ApplyRenderingMaterial();

	/**
	 * Returns a pointer to the material interface to used on a preview static mesh component based on value of RenderingMaterialType
	 * @param PreviewMeshComponent	The preview static mesh component
	 * @remark Returns null pointer if RenderingMaterialType is equal to ERenderingMaterialType::OriginalRenderingMaterial
	 */
	UMaterialInterface* GetRenderingMaterial( UStaticMeshComponent* PreviewMeshComponent );

	void ToggleWireframeRenderingMode();

	bool IsWireframeRenderingModeOn() { return bWireframeRenderingMode; }

	void UpdateOverlayText();

	void UpdatePerfStats();

#ifdef VIEWPORT_EXPERIMENTAL
	void ToggleShowOrientedBox();

	bool IsShowOrientedBoxOn() { return bShowOrientedBox; }
#endif

	/** Set the selection mode */
	void SetSelectionMode(ESelectionModeType InSelectionMode);

	/** Check if a given selection mode is active */
	bool IsSetSelectionModeApplied(ESelectionModeType InSelectionMode) { return InSelectionMode == CurrentSelectionMode; }

	/** Returns true if the given primitive component is part of the selection set */
	bool IsComponentSelected(const UPrimitiveComponent* PrimitiveComponent);

	/** Sets the show Nanite flag */
	void SetShowNaniteFallback(bool bShow);

	/** Callback for toggling the show Nanite flag. */
	void ToggleShowNaniteFallback() { SetShowNaniteFallback(!bShowNaniteFallbackMenuChecked); }

	/** Callback for checking the Nanite show flag. */
	bool IsSetShowNaniteFallbackChecked() const { return bShowNaniteFallbackMenuChecked; }

	/** Callback for checking whether the Nanite show entry should be displayed. */
	bool IsShowNaniteFallbackVisible() const { return bCanShowNaniteFallbackMenu; }
	
	AActor* CreateActor(USceneComponent* ParentComponent, const TCHAR* ActorName);

private:
	/** The scene for this viewport */
	TSharedPtr< FAdvancedPreviewScene > PreviewScene;

	/** Editor viewport client */
	TSharedPtr<class FDataprepEditorViewportClient> EditorViewportClient;

	TSharedPtr<FExtender> Extender;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	/** World containing the data to preview */
	UWorld* WorldToPreview;

	/** Root component of the viewport world's root actor. */
	TWeakObjectPtr<USceneComponent> RootActorComponent;

	/** Actor to hold onto all 'Movable' preview mesh components */
	TWeakObjectPtr<AActor> MovableActor;

	/** Actor to hold onto all 'Static' preview mesh components */
	TWeakObjectPtr<AActor> StaticActor;

	/** Array of static mesh components added to the preview scene for display. */
	TArray< TWeakObjectPtr< UStaticMeshComponent > > PreviewMeshComponents;

	/** Mapping between scene's static mesh component and preview's one */
	TMap< UStaticMeshComponent*, UStaticMeshComponent* > MeshComponentsMapping;

	/** Mapping between preview's static mesh component and scene's one */
	TMap< UStaticMeshComponent*, UStaticMeshComponent* > MeshComponentsReverseMapping;

	/** Array of selected static mesh components */
	TSet<UStaticMeshComponent*> SelectedPreviewComponents;

	/** Scale factor applied to better navigate through the scene */
	float SceneUniformScale;

	/** Bounding box of the uniformly scaled scene */
	FBox SceneBounds;

	ERenderingMaterialType RenderingMaterialType;

	ESelectionModeType CurrentSelectionMode;

	/** Indicates if wireframe mode is on or off */
	bool bWireframeRenderingMode;

	/** Material used to create utility material instances */
	static TStrongObjectPtr<UMaterial> PreviewMaterial;

	/** Stylized XRay Material */
	static TStrongObjectPtr<UMaterial> XRayMaterial;

	/** Material used to display front facing triangle in green and back facing one in red */
	static TStrongObjectPtr<UMaterial> BackFaceMaterial;

	/** Transparent material instance used to display non-selected meshes */
	static TStrongObjectPtr<UMaterialInstanceConstant> TransparentMaterial;

	/** Fully reflective material used to display surface discontinuity */
	static TStrongObjectPtr<UMaterial> ReflectionMaterial;

	/** Materials used to display each mesh component in a different color */
	static TStrongObjectPtr<UMaterial> PerMeshMaterial;
	static TArray<TStrongObjectPtr<UMaterialInstanceConstant>> PerMeshMaterialInstances;

	TWeakPtr<FDataprepEditor> DataprepEditor;

	/** Pointer to the vertical box into which the overlay text items are added */
	TSharedPtr<SVerticalBox> OverlayTextVerticalBox;
	TSharedPtr<STextBlock> FPSText;
	TSharedPtr<STextBlock> DrawCallsText;

	// Draw calls smoothing
	const int32 DrawCallsUpdateInterval = 20; // Number of frames to average
	int32 DrawCallsAccumulator = 0;
	int32 CurrentDrawCalls = 0;
	int32 AverageDrawCalls = 0;

	/** Index of the profile to use in the preview scene */
	static int32 AssetViewerProfileIndex;

#ifdef VIEWPORT_EXPERIMENTAL
	bool bShowOrientedBox;
#endif

	bool bCanShowNaniteFallbackMenu = false;
	bool bShowNaniteFallbackMenuChecked = false;

	friend FDataprepEditorViewportClient;
};
