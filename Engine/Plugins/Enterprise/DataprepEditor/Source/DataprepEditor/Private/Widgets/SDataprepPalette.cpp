// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDataprepPalette.h"

// Dataprep includes
#include "AssetRegistry/AssetData.h"
#include "DataprepOperation.h"
#include "DataprepEditorUtils.h"
#include "DataprepEditorStyle.h"
#include "Misc/PackageName.h"
#include "SchemaActions/DataprepMenuActionCollectorUtils.h"
#include "SchemaActions/DataprepAllMenuActionCollector.h"
#include "SchemaActions/DataprepDragDropOp.h"
#include "SchemaActions/DataprepFilterMenuActionCollector.h"
#include "SchemaActions/DataprepOperationMenuActionCollector.h"

// Engine includes
#include "AssetRegistry/AssetRegistryModule.h"
#include "Brushes/SlateColorBrush.h"
#include "EditorFontGlyphs.h"
#include "EditorWidgetsModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SExpanderArrow.h"

#define LOCTEXT_NAMESPACE "SDataprepPalette"

// Custom expander to specify our desired padding
class SDataprepPaletteExpanderArrow: public SExpanderArrow
{
	SLATE_BEGIN_ARGS( SDataprepPaletteExpanderArrow )
		: _StyleSet(&FCoreStyle::Get())
		, _IndentAmount(10)
		, _BaseIndentLevel(0)
		, _ShouldDrawWires(false)
	{ }
		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		/** How many Slate Units to indent for every level of the tree. */
		SLATE_ATTRIBUTE(float, IndentAmount)
		/** The level that the root of the tree should start (e.g. 2 will shift the whole tree over by `IndentAmount*2`) */
		SLATE_ATTRIBUTE(int32, BaseIndentLevel)
		/** Whether to draw the wires that visually reinforce the tree hierarchy. */
		SLATE_ATTRIBUTE(bool, ShouldDrawWires)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData)
	{
		OwnerRowPtr = ActionMenuData.TableRow;
		if (!ActionMenuData.RowAction.IsValid())
		{
			OwnerRowPtr = ActionMenuData.TableRow;
			StyleSet = InArgs._StyleSet;
			SetIndentAmount(InArgs._IndentAmount);
			SetBaseIndentLevel(InArgs._BaseIndentLevel);
			ShouldDrawWires.Assign(*this, InArgs._ShouldDrawWires);

			ChildSlot
			.Padding( FMargin( 3.0f, 12.0f, 4.0f, 9.0f ) )
			[
				SAssignNew(ExpanderArrow, SButton)
				.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ClickMethod( EButtonClickMethod::MouseDown )
				.OnClicked( this, &SDataprepPaletteExpanderArrow::OnArrowClicked )
				.ContentPadding(0.f)
				.ForegroundColor( FSlateColor::UseForeground() )
				.IsFocusable( false )
				[
					SNew(SImage)
					.Image( this, &SDataprepPaletteExpanderArrow::GetExpanderImage )
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			];
		}
		else
		{
			ChildSlot
			[
				SNullWidget::NullWidget
			];
		}
	}
};

void SDataprepPalette::Construct(const FArguments& InArgs)
{
	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked< FEditorWidgetsModule >( "EditorWidgets" );
	TSharedRef< SWidget > AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator( EAssetDiscoveryIndicatorScaleMode::Scale_Vertical );

	// Setting the categories text and string
	AllCategory = LOCTEXT("All Category", "All");
	SelectorsCategory = FDataprepFilterMenuActionCollector::FilterCategory;
	OperationsCategory = FDataprepOperationMenuActionCollector::OperationCategory;

	SAssignNew(FilterBox, SSearchBox)
		.OnTextChanged(this, &SDataprepPalette::OnFilterTextChanged);

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// Path and history
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 0, 0, 0, 0 )
		[
			SNew( SWrapBox )
			.UseAllottedSize( true )
			.InnerSlotPadding( FVector2D( 5, 2 ) )

			+ SWrapBox::Slot()
			.FillLineWhenSizeLessThan( 600 )
			.FillEmptySpace( true )
			.Padding( 3, 3, 3, 0 )
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( FMargin( 2 ) )
				[
					SNew( SHorizontalBox )

					// Add New
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign( VAlign_Center )
					.HAlign( HAlign_Left )
					[
						SNew( SComboButton )
						.ComboButtonStyle( FAppStyle::Get(), "ToolbarComboButton" )
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(FMargin(6, 2))
						.OnGetMenuContent_Lambda( [this]{ return ConstructAddActionMenu(); } )
						.HasDownArrow(false)
						.ButtonContent()
						[
							SNew( SHorizontalBox )

							// New Icon
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "NormalText.Important")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
								.Text(FEditorFontGlyphs::File)
							]

							// New Text
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(4, 0, 0, 0)
							[
								SNew( STextBlock )
								.TextStyle( FAppStyle::Get(), "NormalText.Important" )
								.Text( LOCTEXT( "AddNewButton", "Add New" ) )
							]

							// Down Arrow
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(4, 0, 0, 0)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "NormalText.Important")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
								.Text(FEditorFontGlyphs::Caret_Down)
							]
						]
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2, 0, 2, 0)
				[
					FilterBox.ToSharedRef()
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,0,0,0)
		[
			SNew(SBox)
			.HeightOverride(2.0f)
			[
				SNew(SImage)
				.Image(new FSlateColorBrush(FLinearColor( FColor( 34, 34, 34) ) ) )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			[
				SNew(SVerticalBox)
				// Content list
				+ SVerticalBox::Slot()
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnGetFilterText(this, &SDataprepPalette::GetFilterText)
						.OnActionDragged( this, &SDataprepPalette::OnActionDragged )
						.OnCreateCustomRowExpander( this, &SDataprepPalette::OnCreateCustomRowExpander )
						.OnCreateWidgetForAction( this, &SDataprepPalette::OnCreateWidgetForAction )
						.OnCollectAllActions( this, &SDataprepPalette::CollectAllActions )
						.OnContextMenuOpening(this, &SDataprepPalette::OnContextMenuOpening)
						.AutoExpandActionMenu( true )
						.OnGetSectionTitle( this, &SDataprepPalette::OnGetSectionTitle )
					]
			
					+ SOverlay::Slot()
					.HAlign( HAlign_Fill )
					.VAlign( VAlign_Bottom )
					.Padding(FMargin(24, 0, 24, 0))
					[
						// Asset discovery indicator
						AssetDiscoveryIndicator
					]
				]
			]
		]

	];

	// Register with the Asset Registry to be informed when it is done loading up files and when a file changed (Added/Removed/Renamed).
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked< FAssetRegistryModule >( TEXT("AssetRegistry") );
	AssetRegistryModule.Get().OnFilesLoaded().AddSP( this, &SDataprepPalette::RefreshActionsList, true );
	AssetRegistryModule.Get().OnAssetAdded().AddSP( this, &SDataprepPalette::AddAssetFromAssetRegistry );
	AssetRegistryModule.Get().OnAssetRemoved().AddSP( this, &SDataprepPalette::RemoveAssetFromRegistry );
	AssetRegistryModule.Get().OnAssetRenamed().AddSP( this, &SDataprepPalette::RenameAssetFromRegistry );
}

TSharedRef<SWidget> SDataprepPalette::CreateBackground(const TAttribute<FSlateColor>& InColorAndOpacity)
{
	return SNew(SOverlay)
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(InColorAndOpacity)
			.Image(FDataprepEditorStyle::GetBrush( "DataprepEditor.Node.Body" ))
		];
}

FText SDataprepPalette::OnGetSectionTitle(int32 InSection) 
{
	FText SectionTitle;

	switch (InSection) 
	{
		case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Filter:
			SectionTitle = LOCTEXT("FilterTitle", "Filter");
			break;
		case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::SelectionTransform:
			SectionTitle = LOCTEXT("SelectionTransformTitle", "Selection Transform");
			break;
		case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Operation:
			SectionTitle = LOCTEXT("OperationTitle", "Operation");
			break;
	};

	check(!SectionTitle.IsEmpty());

	return SectionTitle;
}

FSlateColor SDataprepPalette::OnGetWidgetColor(FLinearColor InDefaultColor, FIsSelected InIsActionSelectedDelegate)
{
	if (InIsActionSelectedDelegate.IsBound() && InIsActionSelectedDelegate.Execute())
	{
		return InDefaultColor + FColor(90, 90, 90);
	}
	return InDefaultColor;
}

TSharedRef<SWidget> SDataprepPalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	FText Category;

	FLinearColor OutlineColor;
	FLinearColor BodyColor = FColor(91, 91, 91);

	if ( TSharedPtr<FDataprepSchemaAction> DataprepSchemaAction = StaticCastSharedPtr<FDataprepSchemaAction>( InCreateData->Action ) )
	{
		Category = FText::Format(LOCTEXT("DataprepSchemaActionCategory", "({0})"), DataprepSchemaAction->ActionCategory);

		switch ( DataprepSchemaAction->GetSectionID() )
		{
			case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Filter:
				OutlineColor = FDataprepEditorStyle::GetColor("DataprepActionStep.Filter.OutlineColor");
				break;
			case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::SelectionTransform:
				OutlineColor = FDataprepEditorStyle::GetColor("DataprepActionStep.SelectionTransform.OutlineColor");
				break;
			case DataprepMenuActionCollectorUtils::EDataprepMenuActionCategory::Operation:
				OutlineColor = FDataprepEditorStyle::GetColor("DataprepActionStep.Operation.OutlineColor");
				break;
		}
	}

	TAttribute<FSlateColor> OutlineColorAttribute = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepPalette::OnGetWidgetColor, OutlineColor, InCreateData->IsRowSelectedDelegate));
	TAttribute<FSlateColor> BodyColorAttribute = TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SDataprepPalette::OnGetWidgetColor, BodyColor, InCreateData->IsRowSelectedDelegate));

	return SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.Padding(FMargin(1.f, 2.f, 2.f, 2.f))
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(OutlineColorAttribute)
				]

				+ SOverlay::Slot()
				.Padding(FMargin(6.f, 2.f, 1.f, 2.f))
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(BodyColorAttribute)
				]

				+ SOverlay::Slot()
				.Padding( FMargin(11.f, 2.f, 11.f, 2.f) )
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew( SVerticalBox )
					.Clipping(EWidgetClipping::ClipToBounds)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						.ToolTipText(InCreateData->Action->GetTooltipDescription())
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.f, 5.0f, 0.0f, 5.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
							.ColorAndOpacity(FLinearColor::White)
							.Text(InCreateData->Action->GetMenuDescription())
							.HighlightText(InCreateData->HighlightText)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.f, 5.0f, 0.0f, 5.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(Category)
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
							.ColorAndOpacity(FLinearColor::Gray)
						]
					]
				]
			]
		]
	];
}

FText SDataprepPalette::GetFilterText() const
{
	return FilterBox->GetText();
}

void SDataprepPalette::OnFilterTextChanged(const FText& InFilterText)
{
	GraphActionMenu->GenerateFilteredItems(false);
}

TSharedRef<SWidget> SDataprepPalette::ConstructAddActionMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/true);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("DataprepPaletteLabel", "Dataprep Palette"));
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("CreateNewFilterLabel", "Create New Filter"), LOCTEXT("CreateNewFilterTooltip", "Create new user-defined filter"), FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				if (FDataprepEditorUtils::CreateUserDefinedFilter())
				{
					RefreshActionsList(true);
				}
			}))
		);
		MenuBuilder.AddMenuEntry(LOCTEXT("CreateNewOperatorLabel", "Create New Operator"), LOCTEXT("CreateNewOperatorTooltip", "Create new user-defined operator"), FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				if (FDataprepEditorUtils::CreateUserDefinedOperation())
				{
					RefreshActionsList(true);
				}
			}))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SDataprepPalette::OnContextMenuOpening()
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions( SelectedActions );
	if ( SelectedActions.Num() != 1 )
	{
		return TSharedPtr<SWidget>();
	}

	TSharedPtr<FDataprepSchemaAction> DataprepSchemaAction = StaticCastSharedPtr<FDataprepSchemaAction>( SelectedActions[0] );
	
	if ( !DataprepSchemaAction || DataprepSchemaAction->GeneratedClassObjectPath.IsEmpty() )
	{
		return TSharedPtr<SWidget>();
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, nullptr );

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenInBP", "Open in Blueprint Editor"),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([ObjectPath = DataprepSchemaAction->GeneratedClassObjectPath]()
			{
				if ( UObject* Obj = StaticLoadObject( UObject::StaticClass(), nullptr, *ObjectPath ) )
				{
					FKismetEditorUtilities::BringKismetToFocusAttentionOnObject( Obj );
				}
			}))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDataprepPalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FDataprepAllMenuActionCollector ActionCollector;
	for ( TSharedPtr< FDataprepSchemaAction > Action : ActionCollector.CollectActions() )
	{
		OutAllActions.AddAction(StaticCastSharedPtr< FEdGraphSchemaAction >(Action));
	}
}

FReply SDataprepPalette::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
{
	if (InActions.Num() > 0 && InActions[0].IsValid() )
	{
		if ( InActions[0]->GetTypeId() == FDataprepSchemaAction::StaticGetTypeId() )
		{
			TSharedPtr<FDataprepSchemaAction> InAction = StaticCastSharedPtr<FDataprepSchemaAction>( InActions[0] );
			return FReply::Handled().BeginDragDrop( FDataprepDragDropOp::New( InAction.ToSharedRef() ) );
		}
	}

	return FReply::Unhandled();
}

TSharedRef<SExpanderArrow> SDataprepPalette::OnCreateCustomRowExpander(const FCustomExpanderData& InCustomExpanderData) const
{
	return SNew(SDataprepPaletteExpanderArrow, InCustomExpanderData);
}

void SDataprepPalette::AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData)
{
	RefreshAssetInRegistry( InAddedAssetData );
}

void SDataprepPalette::RemoveAssetFromRegistry(const FAssetData& InRemovedAssetData)
{
	RefreshAssetInRegistry( InRemovedAssetData );
}

void SDataprepPalette::RenameAssetFromRegistry(const FAssetData& InRenamedAssetData, const FString& InNewName)
{
	RefreshAssetInRegistry( InRenamedAssetData );
}

void SDataprepPalette::RefreshAssetInRegistry(const FAssetData& InAssetData)
{
	// Grab the asset generated class, it will be checked for being a dataprep operation
	FAssetDataTagMapSharedView::FFindTagResult GeneratedClassPathPtr = InAssetData.TagsAndValues.FindTag( TEXT("GeneratedClass") );
	if (GeneratedClassPathPtr.IsSet())
	{
		const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath( GeneratedClassPathPtr.GetValue() ));

		TArray<FTopLevelAssetPath> OutAncestorClassNames;

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked< FAssetRegistryModule >( TEXT("AssetRegistry") );
		AssetRegistryModule.Get().GetAncestorClassNames(ClassObjectPath, OutAncestorClassNames);
		
		bool bIsTrackedClass = false;
		for ( FTopLevelAssetPath Ancestor : OutAncestorClassNames )
		{
			if ( Ancestor == UDataprepOperation::StaticClass()->GetClassPathName() )
			{
				bIsTrackedClass = true;
				break;
			}
		}

		if ( bIsTrackedClass )
		{
			RefreshActionsList( true );
		}
	}
}

#undef LOCTEXT_NAMESPACE
