// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPalette.h"

struct FGraphActionListBuilderBase;
struct FCustomExpanderData;
struct FCreateWidgetForActionData;

class SDataprepPalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS(SDataprepPalette)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	// SGraphPalette Interface
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual FReply OnActionDragged(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent) override;
	// End of SGraphPalette Interface

private:
	/** CallBack from the GraphActionMenu for when the custom row expender is created */
	TSharedRef<class SExpanderArrow> OnCreateCustomRowExpander(const FCustomExpanderData& InCustomExpanderData) const;

	/** Callback from the Asset Registry when a new asset is added. */
	void AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData);

	/** Callback from the Asset Registry when an asset is removed. */
	void RemoveAssetFromRegistry(const FAssetData& InRemoveddAssetData);

	/** Callback from the Asset Registry when an asset is renamed. */
	void RenameAssetFromRegistry(const FAssetData& InRenamedAssetData, const FString& InNewName);

	void RefreshAssetInRegistry(const FAssetData& InAssetData);

	FText GetFilterText() const;

	TSharedRef<SWidget> CreateBackground(const TAttribute<FSlateColor>& ColorAndOpacity);

	void OnFilterTextChanged(const FText& InFilterText);

	FText OnGetSectionTitle(int32 InSection);

	FSlateColor OnGetWidgetColor(FLinearColor InDefaultColor, FIsSelected InIsActionSelectedDelegate);

	TSharedRef<SWidget> ConstructAddActionMenu();
	TSharedPtr<SWidget> OnContextMenuOpening();
	TSharedPtr<SSearchBox> FilterBox;

	// The options for the category selection
	FText AllCategory;
	FText SelectorsCategory;
	FText OperationsCategory;
};

