// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDataprepPreviewRow.h"

#include "DataprepEditorStyle.h"
#include "PreviewSystem/DataprepPreviewSystem.h"

#include "EditorFontGlyphs.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SDataprepPreviewRow::Construct(const FArguments& InArgs, const TSharedPtr<FDataprepPreviewProcessingResult>& InPreviewData)
{
	PreviewData = InPreviewData;

	FSlateFontInfo AwsomeFontStyle = FAppStyle::Get().GetFontStyle("FontAwesome.11");
	AwsomeFontStyle.Size =9 ;

	ChildSlot
	[
		SNew( SHorizontalBox )
		+ SHorizontalBox::Slot()
		.VAlign( VAlign_Center )
		.HAlign( HAlign_Left )
		.MaxWidth(15.f)
		[
				SNew( STextBlock )
				.Font( AwsomeFontStyle )
				.ColorAndOpacity( FDataprepEditorStyle::GetColor("Graph.ActionStepNode.PreviewColor"))
				.Text( this, &SDataprepPreviewRow::GetIcon )
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew( STextBlock )
			.Text( this, &SDataprepPreviewRow::GetLabel )
			.HighlightText( InArgs._HighlightText )
			.ColorAndOpacity( this, &SDataprepPreviewRow::GetTextColor )
		]
	];
}

FText SDataprepPreviewRow::GetIcon() const
{
	if ( FDataprepPreviewProcessingResult* Result = PreviewData.Get() )
	{
		switch ( Result->Status )
		{
		case EDataprepPreviewStatus::BeingProcessed:
			return FEditorFontGlyphs::Refresh;
		case EDataprepPreviewStatus::Pass:
			return FEditorFontGlyphs::Check;
		case EDataprepPreviewStatus::Failed:
			return FText::GetEmpty();
		case EDataprepPreviewStatus::NotSupported :
			return FEditorFontGlyphs::Question;
		default:
			break;
		}
	}

	return FEditorFontGlyphs::Bug;
}

FText SDataprepPreviewRow::GetLabel() const
{
	if ( FDataprepPreviewProcessingResult* Result = PreviewData.Get() )
	{
		 return Result->GetFetchedDataAsText();
	}

	return {};
}

FSlateColor SDataprepPreviewRow::GetTextColor() const
{
	if ( FDataprepPreviewProcessingResult* Result = PreviewData.Get() )
	{
		if ( Result->Status == EDataprepPreviewStatus::Pass )
		{
			return FSlateColor::UseForeground();
		}
	}

	return FSlateColor::UseSubduedForeground();
}
