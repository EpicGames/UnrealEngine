// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FDataprepPreviewProcessingResult; 

/**
 * The widget to display the result of the filter preview
 */
class SDataprepPreviewRow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataprepPreviewRow) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedPtr<FDataprepPreviewProcessingResult>& InPreviewData);

private:
	FText GetIcon() const;

	FText GetLabel() const;

	FSlateColor GetTextColor() const;

	TSharedPtr<FDataprepPreviewProcessingResult> PreviewData;
};
