// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataprepProducersWidget.h"

#include "DataprepContentProducer.h"
#include "DataprepEditorStyle.h"
#include "DataprepEditorUtils.h"
#include "DataprepWidgets.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorFontGlyphs.h"
#include "SPrimaryButton.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "DataprepProducersWidget"

FContentProducerEntry::FContentProducerEntry(int32 InProducerIndex, UDataprepAssetProducers* InAssetProducersPtr)
	: ProducerIndex( InProducerIndex )
	, bIsEnabled( false )
	, bIsSuperseded( false )
	, AssetProducersPtr( InAssetProducersPtr )
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get() )
	{
		if( const UDataprepContentProducer* Producer = AssetProducers->GetProducer( ProducerIndex ) )
		{
			bIsEnabled = AssetProducers->IsProducerEnabled( ProducerIndex );
			bIsSuperseded = AssetProducers->IsProducerSuperseded( ProducerIndex );
			Label = Producer->GetLabel().ToString();
		}
	}
}


void FContentProducerEntry::ToggleProducer()
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get() )
	{
		const FScopedTransaction Transaction( LOCTEXT("Producers_ToggleProducer", "Toggle Producer") );
		AssetProducers->EnableProducer(ProducerIndex, !bIsEnabled);

		// #ueent_todo: Cache previous value to report failed enabling/disabling
		bIsEnabled = AssetProducers->IsProducerEnabled(ProducerIndex);
	}
}

void FContentProducerEntry::RemoveProducer()
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get() )
	{
		AssetProducers->RemoveProducer( ProducerIndex );
	}
}


/** Construct function for this widget */
void SDataprepProducersTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<FContentProducerEntry>& InNode, TSharedRef< FDetailColumnSizeData > InColumnSizeData)
{
	Node = InNode;
	STableRow::Construct(STableRow::FArguments(), OwnerTableView);

	FContentProducerEntryPtr ProducerStackEntry = Node.Pin();

	if (!ProducerStackEntry.IsValid())
	{
		SetRowContent( SNullWidget::NullWidget );
	}
	else
	{
		SetRowContent( GetInputMainWidget( InColumnSizeData ) );
	}
}

TSharedRef<SWidget> SDataprepProducersTableRow::GetInputMainWidget( TSharedRef< FDetailColumnSizeData > ColumnSizeData )
{
	FContentProducerEntryPtr ProducerStackEntry = Node.Pin();

	if ( !ProducerStackEntry.IsValid() )
	{
		return SNullWidget::NullWidget;
	}

	auto DeleteEntry = [ProducerStackEntry]()
	{
		ProducerStackEntry->RemoveProducer();
		return FReply::Handled();
	};

	TSharedPtr<SWidget> Widget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBrush"))
		[
			SNew(SHorizontalBox)
			// Input entry label
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew( SDataprepDetailsView )
				.Object( ProducerStackEntry->GetProducer() )
				.ColumnSizeData( ColumnSizeData )
				.ResizableColumn(false)
			]
			// Delete button
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			.Padding(FMargin(0.0f, 7.0f, 0.0f, 0.0f))
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("DataprepProducersWidget_DeleteToolTip", "Delete this producer"))
				.IsFocusable(false)
				.OnClicked_Lambda(DeleteEntry)
				.VAlign(VAlign_Top)
				.Content()
				[
					SNew(STextBlock)
					.Font(FDataprepEditorUtils::GetGlyphFont())
					.ColorAndOpacity(FLinearColor::White)
					.Text(FEditorFontGlyphs::Trash)
				]
			]
		];

	return Widget.ToSharedRef();
}

void SDataprepProducersTreeView::Construct(const FArguments& InArgs, UDataprepAssetProducers* InAssetProducersPtr, TSharedRef< FDetailColumnSizeData > InColumnSizeData )
{
	ColumnSizeData = InColumnSizeData;
	AssetProducersPtr = InAssetProducersPtr;

	check( AssetProducersPtr.IsValid() );

	BuildProducerEntries();

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&RootNodes)
		.OnGenerateRow(this, &SDataprepProducersTreeView::OnGenerateRow)
		.OnGetChildren(this, &SDataprepProducersTreeView::OnGetChildren)
	);
}

int32 SDataprepProducersTreeView::GetDisplayIndexOfNode(FContentProducerEntryRef InNode)
{
	return LinearizedItems.Find(InNode);
}

void SDataprepProducersTreeView::Refresh()
{
	BuildProducerEntries();
	RequestTreeRefresh();
}

void SDataprepProducersTreeView::OnExpansionChanged(FContentProducerEntryRef InItem, bool bInIsExpanded)
{
}

TSharedRef<ITableRow> SDataprepProducersTreeView::OnGenerateRow(FContentProducerEntryRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDataprepProducersTableRow, OwnerTable, InDisplayNode, ColumnSizeData.ToSharedRef());
}

void SDataprepProducersTreeView::OnGetChildren(FContentProducerEntryRef InParent, TArray<FContentProducerEntryRef>& OutChildren) const
{
	OutChildren.Reset();
}

void SDataprepProducersTreeView::BuildProducerEntries()
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get() )
	{
		int32 ProducersCount = AssetProducers->GetProducersCount();

		RootNodes.Empty( ProducersCount );

		for( int32 Index = 0; Index < ProducersCount; ++Index )
		{
			TSharedRef<FContentProducerEntry> ProducerStackEntry = MakeShareable( new FContentProducerEntry( Index, AssetProducers ) );
			RootNodes.Add( ProducerStackEntry );
		}
	}
}

SDataprepProducersWidget::~SDataprepProducersWidget()
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get())
	{
		AssetProducers->GetOnChanged().RemoveAll(this);
	}
}

void SDataprepProducersWidget::Construct( const FArguments & InArgs, UDataprepAssetProducers* InAssetProducersPtr )
{
	DataprepImportProducersDelegate = InArgs._DataprepImportProducersDelegate;
	DataprepImportProducersEnabledDelegate = InArgs._DataprepImportProducersEnabledDelegate;

	AssetProducersPtr = InAssetProducersPtr;
	AssetProducersPtr->GetOnChanged().AddSP( this, &SDataprepProducersWidget::OnDataprepProducersChanged );

	TreeView = SNew( SDataprepProducersTreeView, InAssetProducersPtr, InArgs._ColumnSizeData.ToSharedRef() );

	AddNewMenu = SNew(SComboButton)
	.ComboButtonStyle( FAppStyle::Get(), "ToolbarComboButton" )
	.ForegroundColor( FLinearColor::White )
	.ToolTipText( LOCTEXT("AddNewToolTip", "Add a new producer.") )
	.OnGetMenuContent( this, &SDataprepProducersWidget::CreateAddProducerMenuWidget )
	.HasDownArrow( false )
	.ButtonContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( FMargin(0, 1) )
		.HAlign( HAlign_Center )
		[
			SNew(STextBlock)
			.Font( FDataprepEditorUtils::GetGlyphFont() )
			.ColorAndOpacity( FLinearColor::White )
			.Text( FEditorFontGlyphs::Plus_Circle )
		]
	];

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SScrollBox)
			.ExternalScrollbar(ScrollBar)
			+ SScrollBox::Slot()
			[
				// Section for producers
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					TreeView.ToSharedRef()
				]
			]
			+ SScrollBox::Slot()
			.HAlign( HAlign_Center )
			.Padding(20)
			[
				SNew(SPrimaryButton)
				.Text( LOCTEXT( "ImportButton", "Import" ) )
				.ToolTipText( LOCTEXT( "ImportButtonTooltip", "Load inputs' data into the Dataprep Editor"  ) )
				.OnClicked(FOnClicked::CreateLambda([this]()
				{
					DataprepImportProducersDelegate.ExecuteIfBound();
					return FReply::Handled();
				}))
				.IsEnabled_Lambda([this]() 
				{ 
					if (AssetProducersPtr->GetProducersCount() > 0 && DataprepImportProducersDelegate.IsBound() && DataprepImportProducersEnabledDelegate.IsBound())
					{
						return DataprepImportProducersEnabledDelegate.IsBound();
					}
					return false;
				})
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SBox )
			.WidthOverride( FOptionalSize( 16 ) )
			[
				ScrollBar
			]
		]
	];
}

void SDataprepProducersWidget::OnDataprepProducersChanged(FDataprepAssetChangeType ChangeType, int32 Index)
{
	TreeView->Refresh();
}

TSharedRef<SWidget> SDataprepProducersWidget::CreateAddProducerMenuWidget()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;

	TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("AddNewProducer", LOCTEXT("DataprepProducersWidget_AddImports", "Add Producer"));
	{
		FUIAction MenuAction;

		// Find content producers the user could use for their data preparation
		for( TObjectIterator< UClass > It ; It ; ++It )
		{
			UClass* CurrentClass = (*It);

			if ( !CurrentClass->HasAnyClassFlags( CLASS_Abstract ) )
			{
				if( CurrentClass->IsChildOf( UDataprepContentProducer::StaticClass() ) )
				{
					UDataprepContentProducer* DefaultProducer = CurrentClass->GetDefaultObject<UDataprepContentProducer>();
					if (!ensure(DefaultProducer) || !DefaultProducer->IsActive())
					{
						continue;
					}

					MenuAction.ExecuteAction.BindSP(this, &SDataprepProducersWidget::OnAddProducer, CurrentClass);

					MenuBuilder.AddMenuEntry(
						DefaultProducer->GetLabel(),
						DefaultProducer->GetDescription(),
						FSlateIcon( FDataprepEditorStyle::GetStyleSetName(), TEXT("DataprepEditor.Producer") ),
						MenuAction,
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDataprepProducersWidget::OnAddProducer( UClass* ProducerClass )
{
	if( UDataprepAssetProducers* AssetProducers = AssetProducersPtr.Get() )
	{
		const FScopedTransaction Transaction( LOCTEXT("Producers_AddProducer", "Add Producer") );
		AssetProducers->AddProducer(ProducerClass);
	}

	TreeView->Refresh();
}

TSharedRef<SWidget> FDataprepAssetProducersDetails::CreateWidget(UDataprepAssetProducers* Producers)
{
	return SNew(SDataprepProducersWidget, Producers );
}

void FDataprepAssetProducersDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized( Objects );
	check( Objects.Num() > 0 );

	UDataprepAssetProducers* Producers = Cast< UDataprepAssetProducers >(Objects[0].Get());
	check( Producers );

	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );

	FName CategoryName = CategoryNames.Num() > 0 ? CategoryNames[0] : FName( TEXT("DatasmithDirProducer") );
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory( CategoryName, FText::GetEmpty(), ECategoryPriority::Important );

	FDetailWidgetRow& CustomRow = CategoryBuilder.AddCustomRow( FText::GetEmpty() )
	.NameContent()
	[
		SNullWidget::NullWidget
	]
	.ValueContent()
	[
		CreateWidget( Producers )
	];
}

#undef LOCTEXT_NAMESPACE
