// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DataprepAssetProducers.h"

#include "IDetailCustomization.h"
#include "Widgets/Views/STreeView.h"

class FUICommandList;
class UDataprepContentProducer;
class UDataprepAssetProducers;
class FDetailColumnSizeData;

class FContentProducerEntry
{
public:
	FContentProducerEntry(int32 InProducerIndex, UDataprepAssetProducers* InAssetProducersPtr);

	bool HasValidData()
	{
		return AssetProducersPtr.IsValid() && AssetProducersPtr->GetProducer( ProducerIndex ) != nullptr;
	}

	UDataprepContentProducer* GetProducer()
	{
		return AssetProducersPtr.IsValid() ? AssetProducersPtr->GetProducer( ProducerIndex ) : nullptr;
	}

	bool WillBeRun() { return bIsEnabled && !bIsSuperseded; }

	void ToggleProducer();

	void RemoveProducer();

	FString Label;
	int32 ProducerIndex;
	bool bIsEnabled;
	bool bIsSuperseded;
	TWeakObjectPtr<UDataprepAssetProducers> AssetProducersPtr;
};

typedef TSharedRef<FContentProducerEntry> FContentProducerEntryRef;
typedef TSharedPtr<FContentProducerEntry> FContentProducerEntryPtr;

/** Represents a row in the VariantManager's tree views and list views */
class SDataprepProducersTableRow : public STableRow<FContentProducerEntryRef>
{
public:
	SLATE_BEGIN_ARGS(SDataprepProducersTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<FContentProducerEntry>& InNode, TSharedRef< FDetailColumnSizeData > InColumnSizeData);

	TSharedRef<SWidget> GetInputMainWidget( TSharedRef< FDetailColumnSizeData > ColumnSizeData );

	TSharedPtr<FContentProducerEntry> GetDisplayNode() const
	{
		return Node.Pin();
	}

private:
	mutable TWeakPtr<FContentProducerEntry> Node;
};

class SDataprepProducersTreeView : public STreeView<FContentProducerEntryRef>
{
public:

	SLATE_BEGIN_ARGS(SDataprepProducersTreeView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDataprepAssetProducers* InAssetProducersPtr, TSharedRef< FDetailColumnSizeData > InColumnSizeData );

	int32 GetDisplayIndexOfNode(FContentProducerEntryRef InNode);

	// Caches the nodes the VariantManagerNodeTree is using and refreshes the display
	void Refresh();

protected:
	void OnExpansionChanged(FContentProducerEntryRef InItem, bool bIsExpanded);
	TSharedRef<ITableRow> OnGenerateRow(FContentProducerEntryRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetChildren(FContentProducerEntryRef InParent, TArray<FContentProducerEntryRef>& OutChildren) const;

private:
	void BuildProducerEntries();

private:
	TWeakObjectPtr<UDataprepAssetProducers> AssetProducersPtr;
	TArray<FContentProducerEntryRef> RootNodes;
	TSharedPtr< FDetailColumnSizeData > ColumnSizeData;
};

/** Delegates for producers import */
DECLARE_DELEGATE(FDataprepImportProducers);
DECLARE_DELEGATE_RetVal(bool, FDataprepImportProducersEnabled);

class SDataprepProducersWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDataprepProducersWidget) {}
		SLATE_ARGUMENT(TSharedPtr< FDetailColumnSizeData >, ColumnSizeData)
		SLATE_ARGUMENT(FDataprepImportProducers, DataprepImportProducersDelegate) 
		SLATE_ARGUMENT(FDataprepImportProducersEnabled, DataprepImportProducersEnabledDelegate)
	SLATE_END_ARGS()

	~SDataprepProducersWidget();

	void Construct(const FArguments& InArgs, UDataprepAssetProducers* InAssetProducersPtr);

	TSharedRef<SWidget> CreateAddProducerMenuWidget();

	void Refresh() { TreeView->Refresh(); }

	TSharedPtr<SWidget> GetAddNewMenu() { return AddNewMenu; }

protected:
	void OnAddProducer( UClass* ProducerClass );

private:
	void OnDataprepProducersChanged(FDataprepAssetChangeType ChangeType, int32 Index);

private:
	TSharedPtr<SWidget> AddNewMenu;
	TWeakObjectPtr<UDataprepAssetProducers> AssetProducersPtr;
	TSharedPtr<SDataprepProducersTreeView> TreeView;
	FDataprepImportProducers DataprepImportProducersDelegate;
	FDataprepImportProducersEnabled DataprepImportProducersEnabledDelegate;
};

// Customization of the details of the Datasmith Scene for the data prep editor.
class DATAPREPEDITOR_API FDataprepAssetProducersDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDataprepAssetProducersDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	TSharedRef<SWidget> CreateWidget(UDataprepAssetProducers* Producers);
};
