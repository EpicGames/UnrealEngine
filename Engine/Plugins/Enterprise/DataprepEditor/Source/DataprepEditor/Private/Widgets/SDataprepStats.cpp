// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataprepStats.h"
#include "DataprepStats.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "DataprepStats"

namespace DataprepStats
{
	const FName ColumnNameOnImport(TEXT("OnImport"));
	const FName ColumnNameOnImportCount(TEXT("OnImportCount"));
	const FName ColumnNameAfterExecuteCount(TEXT("AfterExecuteCount"));

	class STableRow : public SMultiColumnTableRow<FDataprepStatPtr>
	{
	public:
		SDataprepStats::FStatListEntryPtr ItemPtr;

		SLATE_BEGIN_ARGS(STableRow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTable, SDataprepStats::FStatListEntryPtr InItemPtr) 
		{
			ItemPtr = InItemPtr;
			SMultiColumnTableRow<FDataprepStatPtr>::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}

		TSharedPtr<STextBlock> CreateTextForCount(const TAttribute<int32>* InAttribute)
		{
			return SNew(STextBlock)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([=]()
				{
					FText CellText = FText::FromString("-");

					if (InAttribute->IsSet())
					{
						CellText = FText::AsNumber(InAttribute->Get());
					}
					return CellText;
				})));
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
		{
			TSharedPtr<SBox> CellText;
			const FLinearColor DividerColor(0.16f, 0.16f, 0.16f);
			const float MarginHoriz = 12.0f;
			const float MarginVert  = 4.0f;

			if (ItemPtr->StatPtr.IsValid())
			{
				// This is item row
				if (InColumnName == DataprepStats::ColumnNameOnImport)
				{
					CellText = SNew(SBox)
					.Padding(FMargin(MarginHoriz, MarginVert))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ItemPtr->StatPtr->StatName)
					];
				}
				else if (InColumnName == DataprepStats::ColumnNameOnImportCount || InColumnName == DataprepStats::ColumnNameAfterExecuteCount)
				{
					TSharedPtr<STextBlock> CountText;

					if (InColumnName == DataprepStats::ColumnNameAfterExecuteCount)
					{
						CountText = CreateTextForCount(&ItemPtr->StatPtr->PostExecuteCount);
						CountText->SetColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.0f));
					}
					else
					{
						CountText = CreateTextForCount(&ItemPtr->StatPtr->PreExecuteCount);
					}
					
					CellText = SNew(SBox)
					.Padding(FMargin(MarginHoriz, MarginVert))
					.VAlign(VAlign_Center)
					[
						CountText.ToSharedRef()
					];
				}
			}
			else
			{
				// This is category row
				if (InColumnName == DataprepStats::ColumnNameOnImport)
				{
					CellText = SNew(SBox)
					.Padding(FMargin(4, 12, 4, 8))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(ItemPtr->StatCategory))
						.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
					];
				}
				else
				{
					CellText = SNew(SBox);
				}
			}

			return SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(ItemPtr->BgColor)
				]
				+SOverlay::Slot()
				[
					CellText.ToSharedRef()
				];
		}
	};
}

void SDataprepStats::AddStat(const FName& InStatName, const FName& InStatCategory, const FDataprepStat& InStat)
{
	int32 CategoryItemIndex = INDEX_NONE;
	for (int32 ItemIndex = 0; ItemIndex < ListEntries.Num(); ++ItemIndex)
	{
		if (ListEntries[ItemIndex]->StatCategory == InStatCategory)
		{
			CategoryItemIndex = ItemIndex;
			break;
		}
	}

	// Alternate row colors
	const FLinearColor BgColor1 = FLinearColor::Transparent;
	const FLinearColor BgColor2 = FLinearColor(0.068f, 0.068f, 0.068f);

	if (CategoryItemIndex == INDEX_NONE)
	{
		// Insert category
		FStatListEntryPtr CategoryItemPtr = MakeShared<FStatListEntry>();
		CategoryItemPtr->StatName = NAME_None;
		CategoryItemPtr->StatCategory = InStatCategory;
		CategoryItemPtr->StatPtr = TSharedPtr<FDataprepStat>();
		CategoryItemPtr->BgColor = BgColor1;
		CategoryItemIndex = ListEntries.Add(CategoryItemPtr);
		StatCountsMap.Add(InStatCategory, 0);
	}

	const int32 ItemCount = ++StatCountsMap[InStatCategory];
	FStatListEntryPtr StatItemPtr = MakeShared<FStatListEntry>();
	StatItemPtr->StatCategory = InStatCategory;
	StatItemPtr->StatName = InStatName;
	StatItemPtr->StatPtr = MakeShared<FDataprepStat>(InStat);
	StatItemPtr->BgColor = (ItemCount % 2) == 0 ?  BgColor1 : BgColor2;

	ListEntries.Insert(StatItemPtr, CategoryItemIndex + ItemCount);

	StatsListView->RebuildList();
}

FDataprepStatPtr SDataprepStats::GetStat(const FName& InStatName)
{
	for (int StatIndex = 0; StatIndex < ListEntries.Num(); ++StatIndex)
	{
		if (ListEntries[StatIndex]->StatName == InStatName)
		{
			return ListEntries[StatIndex]->StatPtr;
		}
	}
	return FDataprepStatPtr();
}

void SDataprepStats::SetStats(const TSharedPtr<FDataprepStats>& InStats, bool bIsPreExecute)
{
	auto SetSingleStat = [this, InStats, bIsPreExecute](const FName InStatName)
	{
		FDataprepStats* Stats = InStats.Get();
		FDataprepStatPtr StatPtr = GetStat(InStatName);
		if (StatPtr.IsValid())
		{
			TAttribute<int32>& Attr = bIsPreExecute ? StatPtr->PreExecuteCount : StatPtr->PostExecuteCount;
			if (Stats != nullptr)
			{
				Attr.Set(Stats->Get(InStatName));
			}
			else
			{
				Attr = TAttribute<int32>();
			}
		}
	};

	SetSingleStat(FDataprepStats::StatNameTriangles);
	SetSingleStat(FDataprepStats::StatNameVertices);
	SetSingleStat(FDataprepStats::StatNameNaniteTriangles);
	SetSingleStat(FDataprepStats::StatNameNaniteVertices);
	SetSingleStat(FDataprepStats::StatNameTextures);
	SetSingleStat(FDataprepStats::StatNameTextureSize);
	SetSingleStat(FDataprepStats::StatNameMeshes);
	SetSingleStat(FDataprepStats::StatNameSkeletalMeshes);
	SetSingleStat(FDataprepStats::StatNameLights);
	SetSingleStat(FDataprepStats::StatNameMaterials);
	SetSingleStat(FDataprepStats::StatNameActors);
	SetSingleStat(FDataprepStats::StatNameActorComponents);
}

void SDataprepStats::ClearStats(bool bInClearPreExecuteStats, bool bInClearPostExecuteStats)
{
	for (const FStatListEntryPtr& EntryPtr : ListEntries) 
	{
		if (EntryPtr->StatName == NAME_None)
		{
			continue; // This is category
		}

		if (bInClearPreExecuteStats)
		{
			EntryPtr->StatPtr->PreExecuteCount = TAttribute<int32>();
		}
		if (bInClearPostExecuteStats)
		{
			EntryPtr->StatPtr->PostExecuteCount = TAttribute<int32>();
		}
	}
}

void SDataprepStats::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBorder)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(StatsListView, FDataprepStatListView)
					.ListItemsSource(&ListEntries)

					.OnGenerateRow_Lambda([this](FStatListEntryPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
					{
						return SNew(DataprepStats::STableRow, InOwnerTable, InItem);
					})

					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(DataprepStats::ColumnNameOnImport)
							.DefaultLabel(FText())
							.FillWidth(0.33f)
						+ SHeaderRow::Column(DataprepStats::ColumnNameOnImportCount)
							.DefaultLabel(LOCTEXT("ColumnPreCountLabel", "On Import"))
							.FillWidth(0.33f)
						+ SHeaderRow::Column(DataprepStats::ColumnNameAfterExecuteCount)
							.DefaultLabel(LOCTEXT("ColumnPostCountLabel", "After Execute"))
							.FillWidth(0.33f)
					)
			]
		]
	];

	StatsListView->SetSelectionMode(ESelectionMode::None);
}

#undef LOCTEXT_NAMESPACE
