// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

struct FDataprepStats;

struct FDataprepStat
{
	FDataprepStat(const FText& InName, 
				  const TAttribute<int32>& InPreExecuteCount = TAttribute<int32>(),
				  const TAttribute<int32>& InPostExecuteCount = TAttribute<int32>())
	{
		StatName = InName;
	}

	FText StatName;

	TAttribute<int32> PreExecuteCount;
	TAttribute<int32> PostExecuteCount;
};

typedef TSharedPtr<FDataprepStat> FDataprepStatPtr;

class SDataprepStats : public SCompoundWidget
{
public:
	struct FStatListEntry
	{
		FName StatName;
		FName StatCategory;
		FLinearColor BgColor;
		FDataprepStatPtr StatPtr;
	};

	typedef TSharedPtr<FStatListEntry> FStatListEntryPtr;
	typedef SListView<FStatListEntryPtr> FDataprepStatListView;

	SLATE_BEGIN_ARGS(SDataprepStats) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void AddStat(const FName& InStatName, const FName& InStatCategory, const FDataprepStat& InStat);
	
	FDataprepStatPtr GetStat(const FName& InStatName);

	void SetStats(const TSharedPtr<FDataprepStats>& InStats, bool bIsPreExecute);

	void ClearStats(bool bInClearPreExecuteStats, bool bInClearPostExecuteStats);

private:
	TArray<FStatListEntryPtr> ListEntries;

	// Counts by category name
	TMap<FName, int32> StatCountsMap;

	TSharedPtr<FDataprepStatListView> StatsListView;
};
