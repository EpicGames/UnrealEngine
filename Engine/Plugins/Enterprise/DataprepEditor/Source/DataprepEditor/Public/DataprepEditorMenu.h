// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepEditorMenu.generated.h"

UCLASS()
class DATAPREPEDITOR_API UDataprepEditorContextMenuContext : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray< TObjectPtr<UObject> > SelectedObjects;

	UPROPERTY()
	TObjectPtr<class UDataprepAssetInterface> DataprepAsset;
};
