// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetTypeCategories.h"
#include "Developer/AssetTools/Public/AssetTypeCategories.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()
#include "Toolkits/AssetEditorToolkit.h"

class SWidget;
class UDataprepAssetProducers;

extern const FName DataprepEditorAppIdentifier;

#define DATAPREPEDITOR_MODULE_NAME TEXT("DataprepEditor")

/**
* Data preparation editor module interface
*/
class IDataprepEditorModule : public IModuleInterface,	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/**
	 * Singleton-like access to IDatasmithContentEditorModule
	 *
	 * @return Returns DatasmithContent singleton instance, loading the module on demand if needed
	 */
	static inline IDataprepEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDataprepEditorModule>(DATAPREPEDITOR_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATAPREPEDITOR_MODULE_NAME);
	}

	virtual TSharedRef<SWidget> CreateDataprepProducersWidget(UDataprepAssetProducers* AssetProducers) = 0;
	virtual TSharedRef<SWidget> CreateDataprepDetailsView(UObject* ObjectToDetail) = 0;

};
