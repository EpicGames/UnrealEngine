// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDataprepAssetLibrary.h"

#include "DataprepAsset.h"
#include "DataprepContentProducer.h"
#include "DataprepCoreUtils.h"
#include "DataprepEditorScriptingLog.h"

#include "Trace/Trace.inl"

namespace EditorDataprepAssetLibraryUtils
{
	void LogInvalidDataprepAssetInterface()
	{
		UE_LOGF(LogDataprepEditorScripting, Error, "The dataprep asset interface was null.")
	}

	void LogInvalidDataprepAsset()
	{
		UE_LOGF(LogDataprepEditorScripting, Error, "The dataprep asset was null.")
	}

	void LogInvalidDataprepAction()
	{
		UE_LOGF(LogDataprepEditorScripting, Error, "The dataprep action was null.");
	}

	const UDataprepAssetProducers* GetProducers(const UDataprepAssetInterface* DataprepAssetInterface)
	{
		if ( DataprepAssetInterface )
		{
			const UDataprepAssetProducers* Producers = DataprepAssetInterface->GetProducers();
			check( Producers );
			return Producers;
		}
		
		return nullptr;
	}

	UDataprepAssetProducers* GetProducers(UDataprepAssetInterface* DataprepAssetInterface)
	{
		return const_cast<UDataprepAssetProducers*>( GetProducers(const_cast<const UDataprepAssetInterface*>( DataprepAssetInterface ) ) );
	}

	UDataprepContentProducer* AddProducer(UDataprepAssetInterface* DataprepAssetInterface, const TSubclassOf<UDataprepContentProducer>& ProducerClass, bool bIsAutomated)
	{
		if ( !DataprepAssetInterface )
		{
			EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
			return nullptr;
		}

		UClass* Class = ProducerClass.Get();
		if ( !Class || bool(Class->ClassFlags & CLASS_Abstract) )
		{
			UE_LOGF(LogDataprepEditorScripting, Error, "Cannot add a producer because the class is not abstract or null");
			return nullptr;
		}

		if ( UDataprepAssetProducers* Producers = EditorDataprepAssetLibraryUtils::GetProducers( DataprepAssetInterface ) )
		{
			if ( bIsAutomated )
			{
				return Producers->AddProducerAutomated(Class);
			}
			else
			{
				return Producers->AddProducer(Class);
			}
		}

		return nullptr;
	}
}


bool UEditorDataprepAssetLibrary::ExecuteDataprep(UDataprepAssetInterface* DataprepAssetInterface, EDataprepReportMethod LogReportingMethod, EDataprepReportMethod ProgressReportingMethod)
{
	if( DataprepAssetInterface )
	{
		if ( nullptr == DataprepAssetInterface->GetConsumer() )
		{
			UE_LOGF( LogDataprepEditorScripting, Error, "Cannot execute recipe: no Dataprep consumer found." )
			return false;
		}
		
		TSharedPtr<IDataprepLogger> Logger;
		TSharedPtr<IDataprepProgressReporter> Reporter;

		switch (LogReportingMethod)
		{
		case EDataprepReportMethod::StandardLog:
		case EDataprepReportMethod::SameFeedbackAsEditor:
			Logger = MakeShared<FDataprepCoreUtils::FDataprepLogger>();
			break;
		case EDataprepReportMethod::NoFeedback:
		default:
			break;
		}

		switch (ProgressReportingMethod)
		{
		case EDataprepReportMethod::StandardLog:
			Reporter = MakeShared<FDataprepCoreUtils::FDataprepProgressTextReporter>();
			break;
		case EDataprepReportMethod::SameFeedbackAsEditor:
			Reporter = MakeShared<FDataprepCoreUtils::FDataprepProgressUIReporter>();
			break;
		case EDataprepReportMethod::NoFeedback:
			break;
		default:
			break;
		}

		return FDataprepCoreUtils::ExecuteDataprep( DataprepAssetInterface, Logger, Reporter );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
	}

	return false;
}

int32 UEditorDataprepAssetLibrary::GetProducersCount(const UDataprepAssetInterface* DataprepAssetInterface)
{
	if ( !DataprepAssetInterface )
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
	}
	else if ( const UDataprepAssetProducers* Producers = EditorDataprepAssetLibraryUtils::GetProducers( DataprepAssetInterface ) )
	{
		return Producers->GetProducersCount();
	}

	return INDEX_NONE;
}

UDataprepContentProducer* UEditorDataprepAssetLibrary::GetProducer(UDataprepAssetInterface* DataprepAssetInterface, int32 Index)
{
	if ( !DataprepAssetInterface )
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
	}
	else if (  UDataprepAssetProducers* Producers = EditorDataprepAssetLibraryUtils::GetProducers( DataprepAssetInterface ) )
	{
		return Producers->GetProducer( Index );
	}

	return nullptr;
}

void UEditorDataprepAssetLibrary::RemoveProducer(UDataprepAssetInterface* DataprepAssetInterface, int32 Index)
{
	if ( !DataprepAssetInterface )
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
	}
	else if(UDataprepAssetProducers * Producers = EditorDataprepAssetLibraryUtils::GetProducers(DataprepAssetInterface))
	{
		Producers->RemoveProducer( Index );
	}
}

UDataprepContentProducer* UEditorDataprepAssetLibrary::AddProducer(UDataprepAssetInterface* DataprepAssetInterface, TSubclassOf<UDataprepContentProducer> ProducerClass)
{
	constexpr bool bIsAutomated = false;
	return EditorDataprepAssetLibraryUtils::AddProducer( DataprepAssetInterface, ProducerClass, bIsAutomated );
}

UDataprepContentProducer* UEditorDataprepAssetLibrary::AddProducerAutomated(UDataprepAssetInterface* DataprepAssetInterface, TSubclassOf<UDataprepContentProducer> ProducerClass)
{
	constexpr bool bIsAutomated = true;
	return EditorDataprepAssetLibraryUtils::AddProducer( DataprepAssetInterface, ProducerClass, bIsAutomated );
}

int32 UEditorDataprepAssetLibrary::GetActionCount(UDataprepAsset* DataprepAsset)
{
	if ( DataprepAsset )
	{
		return DataprepAsset->GetActionCount();
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}

	return INDEX_NONE;
}

void UEditorDataprepAssetLibrary::RemoveAction(UDataprepAsset* DataprepAsset, int32 Index)
{
	if ( DataprepAsset )
	{
		DataprepAsset->RemoveAction( Index );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}
}

UDataprepActionAsset* UEditorDataprepAssetLibrary::AddAction(UDataprepAsset* DataprepAsset)
{
	// Todo clean this for the new graph and move the implementation to the dataprep asset
	UDataprepActionAsset* NewDataprepAction = nullptr;

	if ( DataprepAsset )
	{
		NewDataprepAction = DataprepAsset->GetAction( DataprepAsset->AddAction() );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}

	return NewDataprepAction;
}

UDataprepActionAsset* UEditorDataprepAssetLibrary::AddActionByDuplication(UDataprepAsset* DataprepAsset, UDataprepActionAsset* ActionToDuplicate)
{
	UDataprepActionAsset* NewDataprepAction = nullptr;

	if ( !DataprepAsset )
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}
	else if ( !ActionToDuplicate )
	{
		UE_LOGF(LogDataprepEditorScripting, Error, "The action to duplicate is null.")
	}
	else
	{
		uint32 NewActionIndex = DataprepAsset->AddAction( ActionToDuplicate );
		NewDataprepAction = DataprepAsset->GetAction( NewActionIndex );
	}

	return NewDataprepAction;
}

void UEditorDataprepAssetLibrary::SwapActions(UDataprepAsset* DataprepAsset, int32 FirstActionIndex, int32 SecondActionIndex)
{
	if ( DataprepAsset )
	{
		DataprepAsset->SwapActions( FirstActionIndex, SecondActionIndex );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}
}

UDataprepActionAsset* UEditorDataprepAssetLibrary::GetAction(UDataprepAsset* DataprepAsset, int32 Index)
{
	if ( DataprepAsset )
	{
		return DataprepAsset->GetAction( Index );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAsset();
	}

	return nullptr;
}

UDataprepContentConsumer* UEditorDataprepAssetLibrary::GetConsumer(UDataprepAssetInterface* DataprepAssetInterface)
{
	if ( DataprepAssetInterface )
	{
		return DataprepAssetInterface->GetConsumer();
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAssetInterface();
	}

	return nullptr;
}

int32 UEditorDataprepAssetLibrary::GetStepsCount(UDataprepActionAsset* DataprepAction)
{
	if (DataprepAction)
	{
		return DataprepAction->GetStepsCount();
	}

	EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	return INDEX_NONE;
}

UDataprepParameterizableObject* UEditorDataprepAssetLibrary::AddStep(UDataprepActionAsset* DataprepAction, TSubclassOf<UDataprepParameterizableObject> StepType)
{
	if (DataprepAction)
	{
		int32 NewActionIndex = DataprepAction->AddStep(StepType);
		if (NewActionIndex != INDEX_NONE)
		{
			return DataprepAction->GetStep(NewActionIndex)->GetStepObject();
		}
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}

	return nullptr;
}

UDataprepParameterizableObject* UEditorDataprepAssetLibrary::AddStepByDuplication(UDataprepActionAsset* DataprepAction, UDataprepParameterizableObject* StepObject)
{
	if (DataprepAction)
	{
		int32 NewActionIndex = DataprepAction->AddStep( StepObject );
		if ( NewActionIndex != INDEX_NONE )
		{
			return DataprepAction->GetStep( NewActionIndex )->GetStepObject();
		}
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}

	return nullptr;
}

void UEditorDataprepAssetLibrary::RemoveStep(UDataprepActionAsset* DataprepAction, int32 Index)
{
	if ( DataprepAction )
	{
		int32 NewActionIndex = DataprepAction->RemoveStep( Index );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}
}

void UEditorDataprepAssetLibrary::MoveStep(UDataprepActionAsset* DataprepAction, int32 StepIndex, int32 DestinationIndex)
{
	if ( DataprepAction )
	{
		int32 NewActionIndex = DataprepAction->MoveStep( StepIndex, DestinationIndex );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}
}

void UEditorDataprepAssetLibrary::SwapSteps(UDataprepActionAsset* DataprepAction, int32 FirstIndex, int32 SecondIndex)
{
	if ( DataprepAction )
	{
		int32 NewActionIndex = DataprepAction->SwapSteps( FirstIndex, SecondIndex );
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}
}

UDataprepParameterizableObject* UEditorDataprepAssetLibrary::GetStepObject(UDataprepActionAsset* DataprepAction, int32 Index)
{
	if ( DataprepAction )
	{
		if ( UDataprepActionStep* Step = DataprepAction->GetStep(Index).Get() )
		{
			return Step->GetStepObject();
		}
	}
	else
	{
		EditorDataprepAssetLibraryUtils::LogInvalidDataprepAction();
	}

	return nullptr;
}
