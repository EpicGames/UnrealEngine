// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"

#include "EditorDataprepAssetLibrary.generated.h"

class IDataprepLogger;
class IDataprepProgressReporter;
class UDataprepActionAsset;
class UDataprepAsset;
class UDataprepAssetInterface;
class UDataprepContentConsumer;
class UDataprepContentProducer;

UENUM(BlueprintType)
enum class EDataprepReportMethod : uint8
{
	// Report the feedback into the output log only
	StandardLog,

	// Report the feedback the same way that the dataprep asset editor does
	SameFeedbackAsEditor,

	// Don't report the feedback
	NoFeedback,
};


/**
 * Utility class to do most expose most of the common functionalities of the dataprep editor plugin (Visual Dataprep).
 *
 *
 * A Dataprep asset is composed of tree main parts: a array of producers, a recipe and a consumer.
 *
 * The producers are the objects that manage the logistic of importing the data into the dataprep context
 * For example, a DatasmithFileProducer is an object that can import the data from a file using the datasmith importer
 *
 * The recipe is a series of DataprepActions. Each action is generally compose of a filter(s) and an operation(s) We refer those as the steps of action.
 * The typical action consist in filtering the dataprep context to get a subset objects and passing those to some operations.
 * Each action receive the full context from the scene and the asset loaded into the dataprep environment.
 * 
 * The consumer is the object that receive the dataprep environment and turns it into something useful and not transient.
 * Currently, the only type of consumer supported is the DatasmithConsumer. It take the dataprep environment and import it into the engine in similar fashion to a standard datasmith import.
 *
 *
 * More on the dataprep action asset:
 * Each step of dataprep action are a descendant of the type UDataprepParameterizableObject.
 * When setting the value of variables on those objects prefer using the SetEditorProperty utility function as the parameterization of the dataprep asset depends on certain editor calls to stay in sync with the recipe.
 *
 */
UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "Dataprep Core Blueprint Library"))
class DATAPREPEDITORSCRIPTINGUTILITIES_API UEditorDataprepAssetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Runs the Dataprep asset's producers, execute its recipe and finally runs the consumer to output the results.
	 * @param	DataprepAssetInterface		Dataprep asset to run.
	 * @param	LogReportingMethod		Chose the way the log from the producers, operations and consumer will be reported (this will only affect the log from dataprep).
	 * @param	ProgressReportingMethod		The way that the progress updates will be reported.
	 * @return	True if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset")
	static bool ExecuteDataprep(UDataprepAssetInterface* DataprepAssetInterface, EDataprepReportMethod LogReportingMethod, EDataprepReportMethod ProgressReportingMethod);

	/**
	 * Get number of the producer of a dataprep asset
	 * @param DataprepAssetInterface The dataprep asset from which to get the number of producer
	 * @return The number of producers of a dataprep asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Producers")
	static int32 GetProducersCount(const UDataprepAssetInterface* DataprepAssetInterface);

	/**
	 * Get a producer from a dataprep asset.
	 * @param DataprepAssetInterface The dataprep asset from which the producer will be retrieved
	 * @param Index The index of the producer in the dataprep asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Producers")
	static UDataprepContentProducer* GetProducer(UDataprepAssetInterface* DataprepAssetInterface, int32 Index);

	/**
	 * Remove a producer from a dataprep asset
	 * @param DataprepAssetInterface The dataprep asset from which the producer will be removed
	 * @param Index Index of the producer to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Producers")
	static void RemoveProducer(UDataprepAssetInterface* DataprepAssetInterface, int32 Index);

	/**
	 * Add a producer to a dataprep asset (The producer will act as if was call from the dataprep editor, use the automated version if you don't want any ui)
	 * @param DataprepAssetInterface The dataprep asset on which the producer will be added.
	 * @param ProducerClass The type of producer to add
	 * @return The created producer
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Producers", meta = (DeterminesOutputType = "ProducerClass"))
	static UDataprepContentProducer* AddProducer(UDataprepAssetInterface* DataprepAssetInterface,TSubclassOf<UDataprepContentProducer> ProducerClass);

	/**
	 * Add a producer to a dataprep asset
	 * @param DataprepAssetInterface The dataprep asset on which the producer will be added.
	 * @param ProducerClass The type of producer to add
	 * @return The created producer
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Producers", meta = (DeterminesOutputType = "ProducerClass"))
	static UDataprepContentProducer* AddProducerAutomated(UDataprepAssetInterface* DataprepAssetInterface, TSubclassOf<UDataprepContentProducer> ProducerClass);

	/**
	 * Get number of actions of a dataprep asset
	 * @param DataprepAsset The dataprep asset from which to get the number of action
	 * @return The number of actions of a dataprep asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static int32 GetActionCount(UDataprepAsset* DataprepAsset);

	/**
	 * Remove an action from a dataprep asset
	 * @param DataprepAsset The dataprep asset from which the action will be removed
	 * @param Index Index of the action to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static void RemoveAction(UDataprepAsset* DataprepAsset, int32 Index);

	/**
	 * Add an action to a dataprep asset
	 * @param DataprepAsset The dataprep asset to which the action will added
	 * @return the new action
	 * @note the action is added at the end of the action list
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepActionAsset* AddAction(UDataprepAsset* DataprepAsset);

	/**
	 * Add an action to a dataprep asset
	 * @param DataprepAsset The dataprep asset to which the action will added
	 * @param ActionToDuplicate The action that will be duplicated
	 * @return the new action
	 * @note the action is added at the end of the action list
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepActionAsset* AddActionByDuplication(UDataprepAsset* DataprepAsset, UDataprepActionAsset* ActionToDuplicate);

	/**
	 * Swap the actions of a dataprep asset
	 * @param DataprepAsset The dataprep asset on which the actions will swapped
	 * @param FirstActionIndex The index of the first action
	 * @param SecondActionIndex The index of the second action
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static void SwapActions(UDataprepAsset* DataprepAsset, int32 FirstActionIndex, int32 SecondActionIndex);

	/**
	 * Get an action from a dataprep asset.
	 * @param DataprepAsset The dataprep asset from which the action will be retrieved
	 * @param Index The index of the action in the dataprep asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepActionAsset* GetAction(UDataprepAsset* DataprepAsset, int32 Index);

	/**
	 * Access the consumer of a dataprep asset
	 * @param DataprepAssetInterface The dataprep asset from which the consumer retrieved
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Consumer")
	static UDataprepContentConsumer* GetConsumer(UDataprepAssetInterface* DataprepAssetInterface);

	/**
	 * Get the number of steps for a dataprep action
	 * @param DataprepAction The dataprep action from which we will count the number steps
	 * @return The number of steps the dataprep action
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static int32 GetStepsCount(UDataprepActionAsset* DataprepAction);

	/**
	 * Add a step to a dataprep action
	 * @param DataprepAction The dataprep action on which the step will be added
	 * @param StepType The type of the step we want to add. It can be a fetcher (for the filters) or a operation.z
	 * @return The object of the new step
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepParameterizableObject* AddStep(UDataprepActionAsset* DataprepAction, TSubclassOf<UDataprepParameterizableObject> StepType);

	/**
	 * Add a step to a dataprep action by duplicating the step object
	 * @param DataprepAction The dataprep action on which the step will be added
	 * @param StepObject The step that will be duplicated into the action
	 * @return The object of the new step
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepParameterizableObject* AddStepByDuplication(UDataprepActionAsset* DataprepAction, UDataprepParameterizableObject* StepObject);

	/**
	 * Remove a step from the action
	 * @param DataprepAction The dataprep action from which we will remove the step
	 * @param Index the index of the step to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static void RemoveStep(UDataprepActionAsset* DataprepAction, int32 Index);

	/**
	 * Move a step of the dataprep action
	 * @param DataprepAction The dataprep action on which a step will be moved
	 * @param StepIndex The index of the step to move
	 * @param DestinationIndex The index where the step will be moved
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static void MoveStep(UDataprepActionAsset* DataprepAction, int32 StepIndex, int32 DestinationIndex);

	/**
	 * Swap the steps of a dataprep action
	 * @param DataprepAction The dataprep action on which the step will be swapped
	 * @param FirstIndex The index of the first step
	 * @param SecondIndex The index of the seconds step
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static void SwapSteps(UDataprepActionAsset* DataprepAction, int32 FirstIndex, int32 SecondIndex);

	/**
	 * Return the object of a step from the dataprep action
	 * @param DataprepAction The dataprep action on which the step object will retrieve
	 * @param Index The index of the step
	 * @return The retrieved step object (Generally a dataprep operation or filter)
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep Asset | Actions")
	static UDataprepParameterizableObject* GetStepObject(UDataprepActionAsset* DataprepAction, int32 Index);
};
