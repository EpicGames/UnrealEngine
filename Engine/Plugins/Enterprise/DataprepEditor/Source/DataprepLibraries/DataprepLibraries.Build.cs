// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataprepLibraries : ModuleRules
	{
		public DataprepLibraries(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"BlueprintGraph",
					"Core",
					"CoreUObject",
					"DataprepCore",
					"DataprepEditor",
					"DatasmithContent",
					"DerivedDataCache",
					"EditorFramework",
					"EditorScriptingUtilities",
					"Engine",
					"GraphEditor",
					"InputCore",
					"KismetCompiler",
					"LevelSequence",
					"MeshDescription",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"ToolMenus",
					"UnrealEd",
					"PhysicsCore",
					"StaticMeshEditor"
				}
			);

			PrivateIncludePaths.AddRange(
				new string[] 
				{
					"DataprepCore/Private/Shared",
				}
			);
		}
	}
}
