// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepEditingOperations.h"

#include "Components/StaticMeshComponent.h"
#include "DataprepCoreUtils.h"
#include "DataprepOperationsLibrary.h"
#include "DataprepOperationsLibraryUtil.h"

#include "ActorEditorUtils.h"
#include "ActorFactories/ActorFactory.h"
#include "Engine/Blueprint.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "DerivedDataCacheInterface.h"
#include "Editor.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMeshSourceData.h"
#include "Engine/Texture.h"
#include "GameFramework/WorldSettings.h"
#include "IMeshMergeUtilities.h"
#include "LevelSequence.h"
#include "Materials/MaterialInstance.h"
#include "MeshMergeModule.h"
#include "ObjectTools.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"
#include "ScopedTransaction.h"

// UI related section
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DatasmithEditingOperations"

#ifdef LOG_TIME
namespace DataprepEditingOperationTime
{
	typedef TFunction<void(FText)> FLogFunc;

	class FTimeLogger
	{
	public:
		FTimeLogger(const FString& InText, FLogFunc&& InLogFunc)
			: StartTime( FPlatformTime::Cycles64() )
			, Text( InText )
			, LogFunc(MoveTemp(InLogFunc))
		{
			UE_LOGF( LogDataprep, Log, "%ls ...", *Text );
		}

		~FTimeLogger()
		{
			// Log time spent to import incoming file in minutes and seconds
			double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

			int ElapsedMin = int(ElapsedSeconds / 60.0);
			ElapsedSeconds -= 60.0 * (double)ElapsedMin;
			FText Msg = FText::Format( LOCTEXT("DataprepOperation_LogTime", "{0} took {1} min {2} s."), FText::FromString( Text ), ElapsedMin, FText::FromString( FString::Printf( TEXT("%.3f"), ElapsedSeconds ) ) );
			LogFunc( Msg );
		}

	private:
		uint64 StartTime;
		FString Text;
		FLogFunc LogFunc;
	};
}
#endif

namespace DatasmithEditingOperationsUtils
{
	void FindActorsToMerge(const TArray<AActor*>& ChildrenActors, TArray<AActor*>& ActorsToMerge);
	void FindActorsToCollapseOrDelete(const TArray<AActor*>& ActorsToVisit, TArray<AActor*>& ActorsToCollapse, TArray<UObject*>& ActorsToDelete );
	void GetRootActors(UWorld* World, TArray<AActor*>& RootActors);
	void GetComponentsToMerge(UWorld*& World, const TArray<UObject*>& InObjects, TArray<UStaticMeshComponent*>& ComponentsToMerge);
	TArray<UObject*> CollectObjectsToDeleteAfterMeshMerging(TArray<UStaticMeshComponent*>& InMergedComponents);

	int32 GetActorDepth(AActor* Actor)
	{
		return Actor ? 1 + GetActorDepth(Actor->GetAttachParentActor()) : 0;
	}

	class FMergingData
	{
	public:
		FMergingData(const TArray<UPrimitiveComponent*>& PrimitiveComponents);

		bool Equals(const FMergingData& Other);

		TMap< FString, TArray< FTransform > > Data;
	};
}

void UDataprepDeleteObjectsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger( TEXT("RemoveObjects"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Implementation based on DatasmithImporterImpl::DeleteActorsMissingFromScene, UEditorLevelLibrary::DestroyActor
	struct FActorAndDepth
	{
		AActor* Actor;
		int32 Depth;
	};

	TArray<FActorAndDepth> ActorsToDelete;
	ActorsToDelete.Reserve(InContext.Objects.Num());

	TArray<UObject*> ObjectsToDelete;
	ObjectsToDelete.Reserve(InContext.Objects.Num());

	for (UObject* Object : InContext.Objects)
	{
		if ( !ensure(Object) || !IsValid(Object) )
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >( Object ))
		{
			ActorsToDelete.Add(FActorAndDepth{Actor, DatasmithEditingOperationsUtils::GetActorDepth(Actor)});
		}
		else if(FDataprepCoreUtils::IsAsset(Object))
		{
			ObjectsToDelete.Add(Object);
		}
	}

	// Sort actors by decreasing depth (in order to delete children first)
	ActorsToDelete.Sort([](const FActorAndDepth& Lhs, const FActorAndDepth& Rhs){ return Lhs.Depth > Rhs.Depth; });

	bool bSelectionAffected = false;
	for (const FActorAndDepth& ActorInfo : ActorsToDelete)
	{
		AActor* Actor = ActorInfo.Actor;

		if(Actor && Actor->GetRootComponent())
		{
			// Reattach our children to our parent
			TArray< USceneComponent* > AttachChildren = Actor->GetRootComponent()->GetAttachChildren(); // Make a copy because the array in RootComponent will get modified during the process
			USceneComponent* AttachParent = Actor->GetRootComponent()->GetAttachParent();

			for ( USceneComponent* ChildComponent : AttachChildren )
			{
				if(ChildComponent)
				{
					// skip component with invalid or condemned owner
					AActor* Owner = ChildComponent->GetOwner();
					if ( Owner == Actor || !IsValid(Owner) || InContext.Objects.Contains(Owner) /* Slow!!! */)
					{
						continue;
					}

					ChildComponent->AttachToComponent( AttachParent, FAttachmentTransformRules::KeepWorldTransform );
				}
			}
		}

		ObjectsToDelete.Add( Actor );
	}

	DeleteObjects( ObjectsToDelete );
}

void UDataprepMergeActorsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	TArray<UStaticMeshComponent*> ComponentsToMerge;
	UWorld* CurrentWorld = nullptr;

	DatasmithEditingOperationsUtils::GetComponentsToMerge(CurrentWorld, InContext.Objects, ComponentsToMerge);

	// Nothing to do if there is only one component to merge
	if( ComponentsToMerge.Num() < 2)
	{
		UE_LOGF( LogDataprep, Log, "No static mesh actors to merge" );
		return;
	}

#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger( TEXT("MergeActors"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	if(!MergeStaticMeshActors(CurrentWorld, ComponentsToMerge, NewActorLabel.IsEmpty() ? TEXT("Merged") : *NewActorLabel ))
	{
		return;
	}

	// Position the merged actor at the right location
	if(MergedActor->GetRootComponent() == nullptr)
	{
		USceneComponent* RootComponent = NewObject< USceneComponent >( MergedActor, USceneComponent::StaticClass(), *MergedActor->GetActorLabel(), RF_Transactional );

		MergedActor->AddInstanceComponent( RootComponent );
		MergedActor->SetRootComponent( RootComponent );
	}

	MergedActor->GetRootComponent()->SetWorldLocation( MergedMeshWorldLocation );

	AActor* Owner = ComponentsToMerge[0]->GetOwner<AActor>();
	ensure( Owner );

	if( AActor* OwnerParent = Owner ? Owner->GetAttachParentActor() : nullptr )
	{
		// Keep the merged actor in the hierarchy, taking the parent of the first component
		// In the future, the merged actor could be attached to the common ancestor instead of the first parent in the list
		MergedActor->GetRootComponent()->AttachToComponent( OwnerParent->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform );
	}

	// Collect all objects to be deleted
	TArray<UObject*> ObjectsToDelete = DatasmithEditingOperationsUtils::CollectObjectsToDeleteAfterMeshMerging(ComponentsToMerge);

	if( ObjectsToDelete.Num() > 0 )
	{
		DeleteObjects( ObjectsToDelete );
	}
}

bool UDataprepMergeActorsOperation::MergeStaticMeshActors(UWorld* World, const TArray<UStaticMeshComponent*>& ComponentsToMerge, const FString& RootName, bool bCreateActor)
{
	TSet<UStaticMesh*> StaticMeshes;
	TArray<UPrimitiveComponent*> PrimitiveComponentsToMerge; // because of MergeComponentsToStaticMesh
	for(UStaticMeshComponent* StaticMeshComponent : ComponentsToMerge)
	{
		PrimitiveComponentsToMerge.Add(StaticMeshComponent);

		if(StaticMeshComponent->GetStaticMesh()->GetRenderData() == nullptr)
		{
			StaticMeshes.Add(StaticMeshComponent->GetStaticMesh());
		}
	}

	DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder( StaticMeshes );

	//
	// See MeshMergingTool.cpp
	//
	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	FMeshMergingSettings MergeSettings;
	MergeSettings.PivotType = bPivotPointAtZero ? EMeshMergePivotType::WorldOrigin : EMeshMergePivotType::Automatic;

	TArray<UObject*> CreatedAssets;
	const float ScreenAreaSize = TNumericLimits<float>::Max();
	FVector MMWL;
	MeshUtilities.MergeComponentsToStaticMesh( PrimitiveComponentsToMerge, World, MergeSettings, nullptr, GetTransientPackage(), FString(), CreatedAssets, MMWL, ScreenAreaSize, true);
	MergedMeshWorldLocation = MMWL;

	UStaticMesh* UtilitiesMergedMesh = nullptr;
	if (!CreatedAssets.FindItemByClass(&UtilitiesMergedMesh))
	{
		UE_LOGF(LogDataprep, Error, "MergeStaticMeshActors failed. No mesh was created.");
		return false;
	}

	// Add asset to set of assets in Dataprep action working set
	MergedMesh = Cast<UStaticMesh>( AddAsset( UtilitiesMergedMesh, NewActorLabel.IsEmpty() ? TEXT("Merged_Mesh") : *NewActorLabel ) );
	if (!MergedMesh)
	{
		UE_LOGF(LogDataprep, Error, "MergeStaticMeshActors failed. Internal error while creating the merged mesh.");
		return false;
	}

	if(bCreateActor == true)
	{
		// Place new mesh in the world
		MergedActor = Cast<AStaticMeshActor>( CreateActor( AStaticMeshActor::StaticClass(), NewActorLabel.IsEmpty() ? TEXT("Merged_Actor") : *NewActorLabel ) );
		if (!MergedActor)
		{
			UE_LOGF(LogDataprep, Error, "MergeStaticMeshActors failed. Internal error while creating the merged actor.");
			return false;
		}

		MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		MergedActor->SetActorLabel(NewActorLabel);
		World->UpdateCullDistanceVolumes(MergedActor, MergedActor->GetStaticMeshComponent());
	}

	return true;
}

void UDataprepCreateProxyMeshOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	TArray<UStaticMeshComponent*> ComponentsToMerge;
	UWorld* CurrentWorld = nullptr;

	DatasmithEditingOperationsUtils::GetComponentsToMerge(CurrentWorld, InContext.Objects, ComponentsToMerge);

	// Nothing to do if there is no static mesh components to merge
	if(ComponentsToMerge.Num() == 0)
	{
		UE_LOGF(LogDataprep, Log, "No static mesh to merge");
		return;
	}

	// Validate render data for static meshes
	TSet<UStaticMesh*> StaticMeshes;
	for(UPrimitiveComponent* PrimitiveComponent : ComponentsToMerge)
	{
		if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
		{
			if(StaticMeshComponent->GetStaticMesh()->GetRenderData() == nullptr)
			{
				StaticMeshes.Add(StaticMeshComponent->GetStaticMesh());
			}
		}
	}

	DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder( StaticMeshes );

	// Update the settings for geometry
	FMeshProxySettings ProxySettings;
	ProxySettings.bOverrideVoxelSize = false;

	const float Coefficient = 2.0f * Quality / 100.0f;

	const float MinScreenSize = Coefficient <= 1.0f ? 100.0f : 300.0f;
	const float MaxScreenSize = Coefficient <= 1.0f ? 300.0f : 1200.0f;
	ProxySettings.ScreenSize = FMath::FloorToInt( FMath::Lerp( MinScreenSize, MaxScreenSize, Coefficient <= 1.0f ? Coefficient : Coefficient - 1.0f ) + 0.5f );

	// Determine if incoming lightmap UVs are usable
	ProxySettings.bReuseMeshLightmapUVs = true;
	StaticMeshes.Empty( ComponentsToMerge.Num() );
	for(UPrimitiveComponent* PrimitiveComponent : ComponentsToMerge)
	{
		if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
		{
			if(StaticMeshComponent->GetStaticMesh())
			{
				StaticMeshes.Add( StaticMeshComponent->GetStaticMesh() );
			}
		}
	}

	for(UStaticMesh* StaticMesh : StaticMeshes)
	{
		const FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
		if(!BuildSettings.bGenerateLightmapUVs)
		{
			ProxySettings.bReuseMeshLightmapUVs = false;
			break;
		}
		else if(FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription( 0 ))
		{
			FStaticMeshAttributes Attributes(*MeshDescription);
			bool bHasValidLightmapUVs = Attributes.GetVertexInstanceUVs().IsValid() &&
				Attributes.GetVertexInstanceUVs().GetNumChannels() > BuildSettings.SrcLightmapIndex &&
				Attributes.GetVertexInstanceUVs().GetNumChannels() > BuildSettings.DstLightmapIndex;
			if(!bHasValidLightmapUVs)
			{
				ProxySettings.bReuseMeshLightmapUVs = false;
				break;
			}
		}
	}

	// Update the settings for materials
	ProxySettings.MaterialSettings.bMetallicMap = true;
	ProxySettings.MaterialSettings.bRoughnessMap = true;

	const int32 TextureSize = (Coefficient <= 0.5f) ? 512 : ((Coefficient <= 1.0f) ? 1024 : ((Coefficient <= 1.5f) ? 2048 : 4096 ));
	ProxySettings.MaterialSettings.TextureSize = FIntPoint( TextureSize, TextureSize );

	const TCHAR* ProxyBasePackageName = TEXT("TOREPLACE");

	// Generate proxy mesh and proxy material assets 
	FCreateProxyDelegate ProxyDelegate;
	ProxyDelegate.BindLambda( [&](const FGuid Guid, TArray<UObject*>& AssetsToSync )
	{
		UStaticMesh* ProxyMesh = nullptr;
		if(!AssetsToSync.FindItemByClass(&ProxyMesh))
		{
			UE_LOGF(LogDataprep, Error, "CreateProxyMesh failed. No mesh was created.");
			return;
		}

		// Add asset to set of assets in Dataprep action working set
		MergedMesh = Cast<UStaticMesh>(AddAsset(ProxyMesh, NewActorLabel.IsEmpty() ? TEXT("Proxy_Mesh") : *NewActorLabel));
		if(!MergedMesh)
		{
			UE_LOGF(LogDataprep, Error, "CreateProxyMesh failed. Internal error while creating the merged mesh.");
			return;
		}

		// Place new mesh in the world (on a new actor)
		MergedActor = Cast<AStaticMeshActor>(CreateActor(AStaticMeshActor::StaticClass(), NewActorLabel.IsEmpty() ? TEXT("Proxy_Actor") : *NewActorLabel));
		if(!MergedActor)
		{
			UE_LOGF(LogDataprep, Error, "CreateProxyMesh failed. Internal error while creating the merged actor.");
			return;
		}

		MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		MergedActor->SetActorLabel(NewActorLabel.IsEmpty() ? TEXT("Proxy_Actor") : *NewActorLabel);
		CurrentWorld->UpdateCullDistanceVolumes(MergedActor, MergedActor->GetStaticMeshComponent());

		// Add the other assets created by the merge, i.e. material, texture, etc, to the context
		TArray< TPair< UObject*, UObject* > > RedirectionMap;
		RedirectionMap.Reserve( AssetsToSync.Num() );

		for(UObject* Object : AssetsToSync)
		{
			if( Cast<UStaticMesh>(Object) != ProxyMesh)
			{
				const FString AssetName = Object->GetName().Replace( ProxyBasePackageName, NewActorLabel.IsEmpty() ? *GetDisplayOperationName().ToString() : *NewActorLabel, ESearchCase::CaseSensitive );
				UObject* AssetFromMerge = AddAsset( Object, *AssetName );

				RedirectionMap.Emplace( AssetFromMerge, Object );
			}
		}

		// Update references accordingly
		for(TPair< UObject*, UObject* >& MapEntry : RedirectionMap)
		{
			TArray<UObject*> ObjectsToReplace(&MapEntry.Value, 1);
			ObjectTools::ForceReplaceReferences( MapEntry.Key, ObjectsToReplace );
		}
	});

	FGuid JobGuid = FGuid::NewGuid();

	const IMeshMergeUtilities& MergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	MergeUtilities.CreateProxyMesh(ComponentsToMerge, ProxySettings, nullptr, GetTransientPackage(), ProxyBasePackageName, JobGuid, ProxyDelegate);

	// Position the merged actor at the right location
	if(MergedActor->GetRootComponent() == nullptr)
	{
		USceneComponent* RootComponent = NewObject< USceneComponent >(MergedActor, USceneComponent::StaticClass(), *MergedActor->GetActorLabel(), RF_Transactional);

		MergedActor->AddInstanceComponent(RootComponent);
		MergedActor->SetRootComponent(RootComponent);
	}

	AActor* Owner = ComponentsToMerge[0]->GetOwner<AActor>();
	ensure( Owner );

	if( AActor* OwnerParent = Owner ? Owner->GetAttachParentActor() : nullptr )
	{
		// Keep the merged actor in the hierarchy, taking the parent of the first component
		// In the future, the merged actor could be attached to the common ancestor instead of the first parent in the list
		MergedActor->GetRootComponent()->AttachToComponent( OwnerParent->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform );
	}

	// Collect all objects to be deleted
	TArray<UObject*> ObjectsToDelete = DatasmithEditingOperationsUtils::CollectObjectsToDeleteAfterMeshMerging(ComponentsToMerge);

	if( ObjectsToDelete.Num() > 0 )
	{
		DeleteObjects( ObjectsToDelete );
	}
}

void UDataprepDeleteUnusedAssetsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	UWorld* World = nullptr;

	TSet<UObject*> UsedAssets;
	UsedAssets.Reserve(InContext.Objects.Num());

	auto CollectAssets = [&UsedAssets](UMaterialInterface* MaterialInterface )
	{
		UsedAssets.Add(MaterialInterface);
		if(UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface))
		{
			if(MaterialInstance->Parent)
			{
				UsedAssets.Add(MaterialInstance->Parent);
			}
		}

		TArray<UTexture*> Textures;
		MaterialInterface->GetUsedTextures(Textures);
		for (UTexture* Texture : Textures)
		{
			UsedAssets.Add(Texture);
		}
	};

#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger( TEXT("CleanWorld"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	for (UObject* Object : InContext.Objects)
	{
		if ( !ensure(Object) || !IsValid(Object) )
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >( Object ))
		{
			World = Actor->GetWorld();

			TArray<UActorComponent*> Components = Actor->GetComponents().Array();
			Components.Append( Actor->GetInstanceComponents() );

			for(UActorComponent* Component : Components)
			{
				if(UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(Component))
				{
					if(UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
					{
						UsedAssets.Add(StaticMesh);

						for(FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
						{
							if(UMaterialInterface* MaterialInterface = StaticMaterial.MaterialInterface)
							{
								CollectAssets(MaterialInterface);
							}
						}
					}

					for(UMaterialInterface* MaterialInterface : MeshComponent->OverrideMaterials)
					{
						if(MaterialInterface)
						{
							CollectAssets(MaterialInterface);
						}
					}
				}
			}
		}
		else if(ULevelSequence* LevelSequence = Cast<ULevelSequence>(Object))
		{
			UsedAssets.Add(LevelSequence);
		}
	}

	TArray<UObject*> ObjectsToDelete;
	ObjectsToDelete.Reserve(InContext.Objects.Num());

	for(UObject* Object : InContext.Objects)
	{
		if(FDataprepCoreUtils::IsAsset(Object) && !UsedAssets.Contains(Object))
		{
			ObjectsToDelete.Add(Object);
		}
	}

	DeleteObjects( ObjectsToDelete );
}

void UDataprepCompactSceneGraphOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger(TEXT("CompactSceneGraph"), [&](FText Text) { this->LogInfo(Text); });
#endif

	TMap<AActor*, bool> VisibilityMap;
	for (UObject* Object : InContext.Objects)
	{
		if (!ensure(Object) || !IsValid(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast<AActor>(Object))
		{
			IsActorVisible(Actor, VisibilityMap);
		}
	}

	TArray<UObject*> ObjectsToDelete;
	ObjectsToDelete.Reserve(InContext.Objects.Num());

	for (const TPair<AActor*, bool>& ActorVisiblity : VisibilityMap)
	{
		if (!ActorVisiblity.Value)
		{
			ObjectsToDelete.Add(ActorVisiblity.Key);
		}
	}

	DeleteObjects(ObjectsToDelete);
}

void UDataprepSpawnActorsAtLocation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger(TEXT("SpawnActorsAtLocation"), [&](FText Text) { this->LogInfo(Text); });
#endif

	if (!SelectedAsset)
	{
		UE_LOGF(LogDataprep, Log, "No asset was selected");
		return;
	}

	for (UObject* Object : InContext.Objects)
	{
		if (!ensure(Object) || !IsValid(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast<AActor>(Object))
		{
			const FAssetData AssetData(SelectedAsset);

			if (!AssetData.IsValid())
			{
				continue;
			}

			UClass* AssetClass = Cast<UClass>(SelectedAsset);

			if (AssetClass == nullptr)
			{
				AssetClass = SelectedAsset->GetClass();
				check(AssetClass);
			}

			// Find a factory that can create an actor of desired type.
			// The factory will not be actually used to create the actor, but to verify it can be 
			// created and to get it's actual class.
			const TArray<UActorFactory*>& ActorFactories = GEditor->ActorFactories;
			UActorFactory* ActorFactory = nullptr;
			for (int32 FactoryIdx = 0; FactoryIdx < ActorFactories.Num(); FactoryIdx++)
			{
				// Check if the actor can be created using this factory
				FText UnusedErrorMessage;
				if (ActorFactories[FactoryIdx]->CanCreateActorFrom(AssetData, UnusedErrorMessage))
				{
					ActorFactory = ActorFactories[FactoryIdx];
					break;
				}
			}

			if (!ActorFactory)
			{
				continue;
			}

			// Create the actor.
			AActor* DefaultActor = ActorFactory->GetDefaultActor(AssetData);
			check(DefaultActor);

			AActor* NewActor = CreateActor(DefaultActor->GetClass(), Actor->GetName());
			check(NewActor);

			if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>( NewActor ))
			{
				// Assign mesh asset to the newly created actor
				UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(SelectedAsset);

				// Change properties
				UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();
				check(StaticMeshComponent);

				StaticMeshComponent->UnregisterComponent();

				StaticMeshComponent->SetStaticMesh(StaticMesh);
				if (StaticMesh->GetRenderData())
				{
					StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
				}

				// Init Component
				StaticMeshComponent->RegisterComponent();
			}

			NewActor->SetActorRelativeTransform(Actor->GetRootComponent()->GetRelativeTransform());
			NewActor->SetActorLabel(Actor->GetActorLabel() + TEXT("_SA"));

			AActor* ParentActor = Actor->GetAttachParentActor();
			if (ParentActor)
			{
				NewActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);
			}
		}
	}
}

void UDataprepSpawnActorsAtLocation::SetAsset(UObject* Asset)
{
	Modify();
	SelectedAsset = Asset;
}

TSharedRef< SWidget > FDataprepSpawnActorsAtLocationDetails::CreateWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0)
		.MaxWidth(100.0f)
		[
			SAssignNew(AssetPickerAnchor, SComboButton)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ContentPadding(FMargin(2, 2, 2, 1))
			.MenuPlacement(MenuPlacement_BelowAnchor)
			.ButtonContent()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(this, &FDataprepSpawnActorsAtLocationDetails::OnGetComboTextValue)
				.ToolTipText(this, &FDataprepSpawnActorsAtLocationDetails::GetObjectToolTip)
			]
			.OnGetMenuContent(this, &FDataprepSpawnActorsAtLocationDetails::GenerateAssetPicker)
		];
}

const FAssetData& FDataprepSpawnActorsAtLocationDetails::GetAssetData() const
{
	if (DataprepOperation->SelectedAsset)
	{
		if (FSoftObjectPath(DataprepOperation->SelectedAsset) == CachedAssetData.GetSoftObjectPath())
		{
			// This always uses the exact object pointed at
			CachedAssetData = FAssetData(DataprepOperation->SelectedAsset, true);
		}
	}
	else
	{
		if (CachedAssetData.IsValid())
		{
			CachedAssetData = FAssetData();
		}
	}

	return CachedAssetData;
}

FText FDataprepSpawnActorsAtLocationDetails::GetObjectToolTip() const
{
	FText Value = FText::GetEmpty();
	const FAssetData& CurrentAssetData = GetAssetData();
	
	if (CurrentAssetData.IsValid())
	{
		Value = FText::FromString(CurrentAssetData.GetFullName());
	}
	return Value;
}

FText FDataprepSpawnActorsAtLocationDetails::OnGetComboTextValue() const
{
	FText Value = LOCTEXT( "DefaultComboText", "Select Asset" );

	if (DataprepOperation->SelectedAsset != nullptr)
	{
		const FAssetData& CurrentAssetData = GetAssetData();
		if (CurrentAssetData.IsValid())
		{
			Value = FText::FromString(CurrentAssetData.AssetName.ToString());
		}
	}
	return Value;
}

TSharedRef<SWidget> FDataprepSpawnActorsAtLocationDetails::GenerateAssetPicker()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UStaticMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FDataprepSpawnActorsAtLocationDetails::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &FDataprepSpawnActorsAtLocationDetails::OnAssetEnterPressedInPicker);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowDragging = false;

	return
		SNew(SBox)
		.HeightOverride(300)
		.WidthOverride(300)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];
}

void FDataprepSpawnActorsAtLocationDetails::OnAssetSelectedFromPicker(const struct FAssetData& AssetData)
{
	if (AssetPickerAnchor.IsValid())
	{
		AssetPickerAnchor->SetIsOpen(false);

		// We need to explicitly delete the customized UI here or a warning will be generated from 
		// the call to UDataprepSpawnActorsAtLocation::Modify, which re-creates the UI
		AssetPickerAnchor.Reset();
	}

	{
		FScopedTransaction Transaction( LOCTEXT("SpawnActorSetAsset", "Choose Asset for Spawn Actor") );
		DataprepOperation->SetAsset(AssetData.GetAsset());
	}
}

void FDataprepSpawnActorsAtLocationDetails::OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets)
{
	if (InSelectedAssets.Num() > 0)
	{
		OnAssetSelectedFromPicker(InSelectedAssets[0]);
	}
}

void FDataprepSpawnActorsAtLocationDetails::CustomizeDetails(IDetailLayoutBuilder & DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;

	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(Objects.Num() > 0);

	DataprepOperation = Cast< UDataprepSpawnActorsAtLocation >(Objects[0].Get());
	check(DataprepOperation);

	DetailBuilder.HideCategory(FName(TEXT("Warning")));
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("SelectedAsset_Internal")), FText::GetEmpty(), ECategoryPriority::Important);

	// Hide SelectedAsset property as it is replaced with custom widget
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataprepSpawnActorsAtLocation, SelectedAsset));

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow(FText::FromString(TEXT("Selected Asset")));

	CustomAssetImportRow.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DatasmithActorOperationsLabel", "Selected Asset"))
			.ToolTipText(LOCTEXT("DatasmithMeshOperationsTooltip", "Selected Asset to spawn Actor from"))
			.Font(DetailBuilder.GetDetailFont())
		];
	
	CustomAssetImportRow.ValueContent()
		[
			CreateWidget()
		];
}

bool UDataprepCompactSceneGraphOperation::IsActorVisible(AActor* Actor, TMap<AActor*, bool>& VisibilityMap)
{
	if (!Actor)
	{
		return false;
	}

	// For scene compaction, actor visibility is defined as the actor having a MeshComponent (PrimitiveComponent could also be used)
	// or an attached child that is visible
	bool* bIsVisible = VisibilityMap.Find(Actor);
	if (bIsVisible)
	{
		return *bIsVisible;
	}

	TArray<UActorComponent*> Components = Actor->GetComponents().Array();
	for (UActorComponent* Component : Components)
	{
		if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(Component))
		{
			VisibilityMap.Add(Actor, true);
			return true;
		}
	}

	TArray<AActor*> AttachedActors;
	Actor->GetAttachedActors(AttachedActors);
	for (AActor* AttachedActor : AttachedActors)
	{
		if (IsActorVisible(AttachedActor, VisibilityMap))
		{
			VisibilityMap.Add(Actor, true);
			return true;
		}
	}

	VisibilityMap.Add(Actor, false);
	return false;
}

namespace DatasmithEditingOperationsUtils
{
	void FindActorsToMerge(const TArray<AActor*>& ChildrenActors, TArray<AActor*>& ActorsToMerge)
	{
		for(AActor* ChildActor : ChildrenActors)
		{
			TArray<AActor*> ActorsToVisit;
			ChildActor->GetAttachedActors(ActorsToVisit);

			bool bCouldBeMerged = ActorsToVisit.Num() > 0;
			for(AActor* ActorToVisit : ActorsToVisit)
			{
				TArray<AActor*> Children;
				ActorToVisit->GetAttachedActors(Children);

				if(Children.Num() > 0)
				{
					bCouldBeMerged = false;
					break;
				}

				// Check if we can find a static mesh component
				UStaticMeshComponent* Component = ActorToVisit->FindComponentByClass<UStaticMeshComponent>();
				if(Component == nullptr)
				{
					bCouldBeMerged = false;
					break;
				}
			}

			if(bCouldBeMerged)
			{
				ActorsToMerge.Add(ChildActor);
				continue;
			}

			FindActorsToMerge(ActorsToVisit, ActorsToMerge);
		}
	}

	void FindActorsToCollapseOrDelete(const TArray<AActor*>& ActorsToVisit, TArray<AActor*>& ActorsToCollapse, TArray<UObject*>& ActorsToDelete )
	{
		for(AActor* Actor : ActorsToVisit)
		{
			if(Actor->GetClass() == AActor::StaticClass())
			{
				TArray<AActor*> AttachedActors;
				Actor->GetAttachedActors(AttachedActors);

				if(AttachedActors.Num() == 0)
				{
					ActorsToDelete.Add( Actor );
					continue;
				}
				else if(AttachedActors.Num() == 1)
				{
					AActor* ChildActor = AttachedActors[0];

					TArray<AActor*> AttachedChildActors;
					ChildActor->GetAttachedActors(AttachedChildActors);

					if(AttachedChildActors.Num() == 0)
					{
						ActorsToCollapse.Add(Actor);
						continue;
					}
				}

				FindActorsToCollapseOrDelete(AttachedActors, ActorsToCollapse, ActorsToDelete);
			}
			else
			{
				TArray<AActor*> AttachedActors;
				Actor->GetAttachedActors(AttachedActors);
				FindActorsToCollapseOrDelete(AttachedActors, ActorsToCollapse, ActorsToDelete);
			}
		}
	}

	void GetRootActors(UWorld * World, TArray<AActor*>& OutRootActors)
	{
		for(ULevel* Level : World->GetLevels())
		{
			for(AActor* Actor : Level->Actors)
			{
				const bool bIsValidRootActor = IsValid(Actor) &&
					Actor->IsEditable() &&
					!Actor->IsTemplate() &&
					!FActorEditorUtils::IsABuilderBrush(Actor) &&
					!Actor->IsA(AWorldSettings::StaticClass()) &&
					Actor->GetParentActor() == nullptr &&
					Actor->GetRootComponent() != nullptr &&
					Actor->GetRootComponent()->GetAttachParent() == nullptr;

				if(bIsValidRootActor )
				{
					OutRootActors.Add( Actor );
				}
			}
		}
	}

	void GetComponentsToMerge(UWorld*& World, const TArray<UObject*>& InObjects, TArray<UStaticMeshComponent*>& ComponentsToMerge)
	{
		World = nullptr;

		for(UObject* Object : InObjects)
		{
			if(AActor* Actor = Cast<AActor>(Object))
			{
				if(IsValidChecked(Actor))
				{
					// Set current world to first world encountered
					if(World == nullptr)
					{
						World = Actor->GetWorld();
					}

					if(World != Actor->GetWorld())
					{
						UE_LOGF( LogDataprep, Log, "Actor %ls is not part of the Dataprep transient world ...", *Actor->GetActorLabel() );
						continue;
					}

					TInlineComponentArray<UStaticMeshComponent*> ComponentArray;
					Actor->GetComponents(ComponentArray);

					for(UStaticMeshComponent* MeshComponent : ComponentArray)
					{
						// Skip components which are either editor only or for visualization
						if(!MeshComponent->IsEditorOnly() && !MeshComponent->IsVisualizationComponent())
						{
							if(MeshComponent->GetStaticMesh() && MeshComponent->GetStaticMesh()->GetNumSourceModels() > 0)
							{
								ComponentsToMerge.Add(MeshComponent);
							}
						}
					}
				}
			}
			else if(UStaticMeshComponent* MeshComponent = Cast< UStaticMeshComponent >(Object))
			{
				// Skip components which are either editor only or for visualization
				if(!MeshComponent->IsEditorOnly() && !MeshComponent->IsVisualizationComponent())
				{
					if(MeshComponent->GetStaticMesh() && MeshComponent->GetStaticMesh()->GetSourceModels().Num() > 0)
					{
						if (AActor* Owner = MeshComponent->GetAttachmentRootActor())
						{
							if(World == nullptr)
							{
								World = Owner->GetWorld();
							}

							if(World != Owner->GetWorld())
							{
								UE_LOGF( LogDataprep, Log, "Actor %ls is not part of the Dataprep transient world ...", *Owner->GetActorLabel() );
								continue;
							}

							ComponentsToMerge.Add(MeshComponent);
						}
					}
				}
			}
		}
	}

	TArray<UObject*> CollectObjectsToDeleteAfterMeshMerging(TArray<UStaticMeshComponent*>& InMergedComponents)
	{
		// Sort merged components to detach children first
		InMergedComponents.Sort([](const USceneComponent &A, const USceneComponent &B) -> bool { return A.IsAttachedTo(&B); });

		TArray<UObject*> ObjectsToDelete;
		ObjectsToDelete.Reserve(InMergedComponents.Num());
		TSet<UStaticMesh*> StaticMeshes;
		StaticMeshes.Reserve(InMergedComponents.Num());

		for( UStaticMeshComponent* Component : InMergedComponents )
		{
			if (UStaticMesh* StaticMesh = Component->GetStaticMesh())
			{
				Component->SetStaticMesh(nullptr);
				StaticMeshes.Add(StaticMesh);
			}

			if( Component->GetNumChildrenComponents() > 0 )
			{
				// Reparent children
				USceneComponent* NewParentForChildren = nullptr;

				if( Component->GetOwner()->GetRootComponent() == Component )
				{
					// Promote first child to become new root
					USceneComponent* FirstChild = Component->GetChildComponent( 0 );
					Component->GetOwner()->SetRootComponent( FirstChild );
					NewParentForChildren = FirstChild;
				}

				if( USceneComponent* Parent = Component->GetAttachParent() )
				{
					Component->GetChildComponent( 0 )->AttachToComponent( Parent, FAttachmentTransformRules::KeepWorldTransform );

					if( !NewParentForChildren )
					{
						NewParentForChildren = Parent;
					}
				}

				for( int32 ChildIndex = 1; ChildIndex < Component->GetNumChildrenComponents(); ++ChildIndex )
				{
					Component->GetChildComponent( ChildIndex )->AttachToComponent( NewParentForChildren, FAttachmentTransformRules::KeepWorldTransform );
				}
			}

			Component->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);

			// Check if we can delete actor
			USceneComponent* RootComponent = Cast<USceneComponent>(Component->GetOwner()->GetRootComponent());
			check( RootComponent );

			const bool bCanRemoveActor = ( RootComponent == Component ) || ( RootComponent->GetNumChildrenComponents() == 0 && RootComponent->GetClass() == USceneComponent::StaticClass() );

			if( bCanRemoveActor )
			{
				ObjectsToDelete.Add( Component->GetOwner() );
			}
			else
			{
				ObjectsToDelete.Add( Component );
			}
		}

		// Remove from set any static mesh still used by another component
		for (TObjectIterator<UStaticMeshComponent> It; It; ++It)
		{
			if (UStaticMesh* StaticMesh = It->GetStaticMesh())
			{
				if (StaticMeshes.Contains(StaticMesh))
				{
					StaticMeshes.Remove(StaticMesh);
				}
			}
		}

		if (StaticMeshes.Num() > 0)
		{
			FDerivedDataCacheInterface& DDC = GetDerivedDataCacheRef();

			for (UStaticMesh* StaticMesh : StaticMeshes)
			{
				if (FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData())
				{
					// Mark this mesh render data as transient in order to delete any file written in the DDC
					DDC.MarkTransient(*RenderData->DerivedDataKey);
				}
			}

			ObjectsToDelete.Append(StaticMeshes.Array());
		}

		return MoveTemp( ObjectsToDelete );
	}

	FMergingData::FMergingData(const TArray<UPrimitiveComponent*>& PrimitiveComponents)
	{
		Data.Reserve(PrimitiveComponents.Num());

		for(UPrimitiveComponent* PrimitveComponent : PrimitiveComponents)
		{
			if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitveComponent))
			{
				FSoftObjectPath SoftObjectPath(StaticMeshComponent->GetStaticMesh());

				TArray< FTransform >& Transforms = Data.FindOrAdd(SoftObjectPath.ToString());
				Transforms.Add(PrimitveComponent->GetRelativeTransform());
			}
		}
	}

	bool FMergingData::Equals(const FMergingData& Other)
	{
		for(auto& OtherEntry : Other.Data)
		{
			TArray< FTransform >* Transforms = Data.Find(OtherEntry.Key);
			if(Transforms == nullptr)
			{
				return false;
			}

			TArray<bool> TransformMatched;
			TransformMatched.AddZeroed(Transforms->Num());
			for(const FTransform& OtherTransform : OtherEntry.Value)
			{
				int32 FoundTransformIndex = -1;
				for(int32 Index = 0; Index < Transforms->Num(); ++Index)
				{
					if(TransformMatched[Index] == false && (*Transforms)[Index].Equals(OtherTransform))
					{
						TransformMatched[Index] = true;
						FoundTransformIndex = Index;
						break;
					}
				}

				if(FoundTransformIndex == -1)
				{
					return false;
				}
			}
		}

		return true;
	}
}
#undef LOCTEXT_NAMESPACE
