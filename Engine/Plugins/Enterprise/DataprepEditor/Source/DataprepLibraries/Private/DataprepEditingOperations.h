// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataprepOperation.h"
#include "AssetRegistry/AssetData.h"
#include "IDetailCustomization.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "DataprepEditingOperations.generated.h"

class SWidget;

class AStaticMeshActor;
class IMeshBuilderModule;
class UStaticMesh;
class UStaticMeshComponent;
class UWorld;

UCLASS(Category = ObjectOperation, Meta = (DisplayName="Delete Objects", ToolTip = "Delete any asset or actor to process") )
class UDataprepDeleteObjectsOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

		//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ObjectOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ObjectOperation, Meta = (DisplayName="Merge", ToolTip = "Collect geometry from selected actors and merge them into single mesh.") )
class UDataprepMergeActorsOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:
	UDataprepMergeActorsOperation()
		: bDeleteMergedActors_DEPRECATED(true)
		, bDeleteMergedMeshes_DEPRECATED(true)
		, bPivotPointAtZero(false)
		, MergedMesh(nullptr)
		, MergedActor(nullptr)
	{
	}

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

	/** Settings to use for the merge operation */
	UPROPERTY(EditAnywhere, Category = MergeSettings)
	FString NewActorLabel;

	/** Settings to use for the merge operation */
	UPROPERTY()
	bool bDeleteMergedActors_DEPRECATED;

	/** Settings to use for the merge operation */
	UPROPERTY()
	bool bDeleteMergedMeshes_DEPRECATED;

	UPROPERTY()
	FMeshMergingSettings MergeSettings_DEPRECATED;

	/** Whether merged mesh should have pivot at world origin, or at first merged component otherwise */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	bool bPivotPointAtZero;

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

	bool MergeStaticMeshActors(UWorld* World, const TArray<UStaticMeshComponent*>& ComponentsToMerge, const FString& RootName, bool bCreateActor = true);

protected:
	FVector MergedMeshWorldLocation;
	UStaticMesh* MergedMesh;
	AStaticMeshActor* MergedActor;
};

UCLASS(Category = ObjectOperation, Meta = (DisplayName = "Create Proxy Mesh", ToolTip = "Collect geometry from selected actors and merge them into single mesh with reduction."))
class UDataprepCreateProxyMeshOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:
	UDataprepCreateProxyMeshOperation()
		: Quality(50.f)
		, MergedMesh(nullptr)
		, MergedActor(nullptr)
	{
	}

public:
	//~ Begin UDataprepOperation Interface
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

	/** Settings to use for the create proxy operation */
	UPROPERTY(EditAnywhere, Category = ProxySettings)
	FString NewActorLabel;

	UPROPERTY(EditAnywhere, Category = ProxySettings,  meta = (UIMin = 0, UIMax = 100))
	float Quality;

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
protected:
	UStaticMesh* MergedMesh;
	AStaticMeshActor* MergedActor;
};

UCLASS(Category = ObjectOperation, Meta = (DisplayName="Delete Unused Assets", ToolTip = "Delete assets that are not referenced by any objects") )
class UDataprepDeleteUnusedAssetsOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ObjectOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ActorOperation, Meta = (DisplayName="Compact Scene Graph", ToolTip = "Delete actors that do not have visuals, but keep those needed to preserve hierarchy") )
class UDataprepCompactSceneGraphOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

	bool IsActorVisible(AActor*, TMap<AActor*, bool>& VisibilityMap);
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Spawn Actors At Location", ToolTip = "For each actor in the input set, spawn an actor from the specified Asset at the same position and orientation than the reference"))
class UDataprepSpawnActorsAtLocation : public UDataprepEditingOperation
{
	GENERATED_BODY()

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

	void SetAsset(UObject* Asset);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ActorOperation)
	TObjectPtr<UObject> SelectedAsset;

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

class FDataprepSpawnActorsAtLocationDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() 
	{ 
		return MakeShared<FDataprepSpawnActorsAtLocationDetails>(); 
	};

	FDataprepSpawnActorsAtLocationDetails() : DataprepOperation(nullptr) {}
	
	/** Called when details should be customized */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	const FAssetData& GetAssetData() const;
	TSharedRef<SWidget> CreateWidget();
	TSharedRef<SWidget> GenerateAssetPicker();
	FText OnGetComboTextValue() const;
	FText GetObjectToolTip() const;
	void OnAssetSelectedFromPicker(const struct FAssetData& AssetData);
	void OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets);

private:
	TSharedPtr<class SMenuAnchor> AssetPickerAnchor;
	UDataprepSpawnActorsAtLocation* DataprepOperation;

	/** Cached AssetData of object selected */
	mutable FAssetData CachedAssetData;
};
