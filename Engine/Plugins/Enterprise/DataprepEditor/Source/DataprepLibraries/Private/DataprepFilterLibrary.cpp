// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepFilterLibrary.h"

#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"

namespace DataprepFilterLibraryImpl
{
	template<typename T>
	T* CastIfValid(UObject* Target)
	{
		return !IsValid(Target) ? nullptr : Cast<T>(Target);
	}

}

TArray< UObject* > UDataprepFilterLibrary::FilterByClass(const TArray< UObject* >& TargetArray, TSubclassOf< UObject > ObjectClass )
{
	return UEditorFilterLibrary::ByClass( TargetArray, ObjectClass, EEditorScriptingFilterType::Include );
}

TArray< UObject* > UDataprepFilterLibrary::FilterByName( const TArray< UObject* >& TargetArray, const FString& NameSubString, EEditorScriptingStringMatchType StringMatch )
{
	return UEditorFilterLibrary::ByIDName( TargetArray, NameSubString, StringMatch, EEditorScriptingFilterType::Include );
}

TArray< UObject* > UDataprepFilterLibrary::FilterBySize(const TArray< UObject* >& TargetArray, EDataprepSizeSource SizeSource, EDataprepSizeFilterMode FilterMode, float Threshold)
{
	TArray< UObject* > Result;

	auto ConditionnalAdd = [=, &Result](float RefValue, UObject* Object) -> void
	{
		if ( (RefValue <= Threshold && FilterMode == EDataprepSizeFilterMode::SmallerThan)
		  || (RefValue >= Threshold && FilterMode == EDataprepSizeFilterMode::BiggerThan) )
		{
			Result.Add(Object);
		}
	};

	switch (SizeSource)
	{
		case EDataprepSizeSource::BoundingBoxVolume:
		{
			auto GetAnyVolume = [](UObject* Object) -> FBox
			{
				if ( AActor* Actor = DataprepFilterLibraryImpl::CastIfValid<AActor>(Object) )
				{
					return Actor->GetComponentsBoundingBox();
				}
				else if ( UStaticMesh* Mesh = DataprepFilterLibraryImpl::CastIfValid<UStaticMesh>(Object) )
				{
					return Mesh->GetBoundingBox();
				}
				return FBox{};
			};

			for ( UObject* Object : TargetArray )
			{
				FBox BoundingVolume = GetAnyVolume(Object);
				if (BoundingVolume.IsValid)
				{
					ConditionnalAdd(BoundingVolume.GetVolume(), Object);
				}
			}

			break;
		}
	}

	return Result;
}

TArray< AActor* > UDataprepFilterLibrary::FilterByTag( const TArray< AActor* >& TargetArray, FName Tag )
{
	return UEditorFilterLibrary::ByActorTag( TargetArray, Tag, EEditorScriptingFilterType::Include );
}
