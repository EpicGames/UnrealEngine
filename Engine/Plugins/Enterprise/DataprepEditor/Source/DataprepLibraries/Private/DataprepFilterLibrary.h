// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorFilterLibrary.h"

#include "GameFramework/Actor.h"
#include "Templates/SubclassOf.h"
#include "DataprepFilterLibrary.generated.h"

UENUM()
enum class EDataprepSizeSource : uint8
{
	BoundingBoxVolume,
};

UENUM()
enum class EDataprepSizeFilterMode : uint8
{
	SmallerThan,
	BiggerThan,
};

UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "Datasmith Data Preparation Filter Library"))
class UDataprepFilterLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Filter the array based on the Object's class.
	 * @param	TargetArray		Array of Object to filter. The array will not change.
	 * @param	ObjectClass		The Class of the object.
	 * @return	The filtered list.
	 */
	UFUNCTION(BlueprintPure, Category = "Dataprep | Filter", meta = (DeterminesOutputType = "ObjectClass"))
	static TArray< UObject* > FilterByClass(const TArray< UObject* >& TargetArray, TSubclassOf< UObject > ObjectClass );

	/**
	 * Filter the array based on the Object name.
	 * @param	TargetArray		Array of Object to filter. The array will not change.
	 * @param	NameSubString	The name to filter with.
	 * @param	StringMatch		Contains the NameSubString OR matches with the wildcard *? OR exactly the same value.
	 * @return	The filtered list.
	 */
	UFUNCTION(BlueprintPure, Category = "Dataprep | Filter", meta = (DeterminesOutputType = "TargetArray"))
	static TArray< UObject* > FilterByName( const TArray< UObject* >& TargetArray, const FString& NameSubString, EEditorScriptingStringMatchType StringMatch = EEditorScriptingStringMatchType::Contains );

	/**
	 * Filter the array based on the geometry size.
	 * @param	TargetArray       Array of Actors or StaticMeshes to filter. The array will not change.
	 * @param   SizeSource        The reference dimension
	 * @param   FilterMode        How to compare the object size with the threshold
	 * @param   Threshold         Limit value
	 * @return	The filtered list.
	 */
	UFUNCTION(BlueprintPure, Category = "Dataprep | Filter", meta = (DeterminesOutputType = "TargetArray"))
	static TArray< UObject* > FilterBySize( const TArray< UObject* >& TargetArray, EDataprepSizeSource SizeSource, EDataprepSizeFilterMode FilterMode, float Threshold);

	/**
	 * Filter the array based on a tag.
	 * @param	TargetArray		Array of Actors to filter. The array will not change.
	 * @param	Tag				The tag to filter with.
	 * @return	The filtered list.
	 */
	UFUNCTION(BlueprintPure, Category = "Dataprep | Filter", meta = (DeterminesOutputType = "TargetArray"))
	static TArray< AActor* > FilterByTag( const TArray< AActor* >& TargetArray, FName Tag );
};
