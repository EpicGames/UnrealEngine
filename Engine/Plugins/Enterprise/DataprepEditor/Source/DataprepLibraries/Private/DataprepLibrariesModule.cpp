// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepLibrariesModule.h"
#include "DataprepEditorMenu.h"
#include "DataprepOperations.h"
#include "DataprepEditingOperations.h"

#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"
#include "PropertyEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "DataprepLibraries"

class FDataprepLibrariesModule : public IDataprepLibrariesModule
{
public:
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
		PropertyModule.RegisterCustomClassLayout( UDataprepSetLODGroupOperation::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic( &FDataprepSetLODGroupDetails::MakeDetails ) );
		PropertyModule.RegisterCustomClassLayout( UDataprepSpawnActorsAtLocation::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FDataprepSpawnActorsAtLocationDetails::MakeDetails ) );
		PropertyModule.RegisterCustomClassLayout(UDataprepSetOutputFolder::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FDataprepSetOutputFolderDetails::MakeDetails));
		PropertyModule.RegisterCustomClassLayout(UDataprepSetNaniteSettingsOperation::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FDataprepSetNaniteSettingsDetails::MakeDetails));
		RegisterMenus();
	}

	virtual void ShutdownModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
		PropertyModule.UnregisterCustomClassLayout( TEXT("DataprepSetLODGroupOperation") );
	}

	void RegisterMenus()
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		UToolMenu* Menu = ToolMenus->ExtendMenu("DataprepEditor.AssetContextMenu");

		if (!Menu)
		{
			return;
		}

		FToolMenuSection& Section = Menu->FindOrAddSection("AssetActions");

		Section.AddDynamicEntry("GetMaterials", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UDataprepEditorContextMenuContext* Context = InSection.FindContext<UDataprepEditorContextMenuContext>();
			if (Context)
			{
				if (Context->SelectedObjects.Num() > 0)
				{
					TSet< UMaterialInterface* > Materials;

					// Only keep materials
					for (UObject* Asset : Context->SelectedObjects)
					{
						if (UMaterialInterface* Material = Cast<UMaterialInterface>(Asset))
						{
							Materials.Add(Material);
						}
					}

					if (Materials.Num() == 0)
					{
						return;
					}

					InSection.AddMenuEntry(
						"CreateSubstitutionTable",
						LOCTEXT("CreateSubstitutionTableLabel", "Create Substitution Table"),
						LOCTEXT("CreateSubstitutionTableTooltip", "Create substitution table from selected materials"),
						FSlateIcon(),
						FExecuteAction::CreateLambda([Materials]()
						{
							FString NewNameSuggestion = FString(TEXT("MaterialSubstitutionDataTable"));
							FString PackageNameSuggestion = FString(TEXT("/Game/")) + NewNameSuggestion;
							FString Name;
							FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
							AssetToolsModule.Get().CreateUniqueAssetName(PackageNameSuggestion, TEXT(""), PackageNameSuggestion, Name);

							TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
								SNew(SDlgPickAssetPath)
								.Title(LOCTEXT("CreateSubstitutionTablePickName", "Choose New DataTable Location"))
								.DefaultAssetPath(FText::FromString(PackageNameSuggestion));

							FString DataTableName;
							FString PackageName;

							if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
							{
								// Get the full name of where we want to create the asset.
								PackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
								DataTableName = FPackageName::GetLongPackageAssetName(PackageName);

								// Check if the user inputed a valid asset name, if they did not, give it the generated default name
								if (DataTableName.IsEmpty())
								{
									// Use the defaults that were already generated.
									PackageName = PackageNameSuggestion;
									DataTableName = *Name;
								}
							}

							UPackage* Package = CreatePackage(*PackageName);
							check(Package);

							// Create DataTable object
							UDataTable* DataTable = NewObject<UDataTable>(Package, *DataTableName, RF_Public | RF_Standalone);
							DataTable->RowStruct = FMaterialSubstitutionDataTable::StaticStruct();

							for (UMaterialInterface* Material : Materials)
							{
								FMaterialSubstitutionDataTable RowData;
								RowData.SearchString = Material->GetName();
								RowData.StringMatch = EEditorScriptingStringMatchType::ExactMatch;
								RowData.MaterialReplacement = nullptr;
								DataTable->AddRow(Material->GetFName(), RowData);
							}

							FAssetRegistryModule::AssetCreated( DataTable );

							FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
							ContentBrowserModule.Get().SyncBrowserToAssets(TArray<UObject*>({ DataTable }), true);

							Package->MarkPackageDirty();
						}));
				}
			}
		}));
	}
};

IMPLEMENT_MODULE( FDataprepLibrariesModule, DataprepLibraries )

#undef LOCTEXT_NAMESPACE
