// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepOperations.h"


#include "Components/StaticMeshComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshResources.h"

#include "Materials/MaterialInterface.h"
#include "Misc/FileHelper.h"

// UI related section
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "DatasmithMeshOperations"

#ifdef LOG_TIME
namespace DataprepOperationTime
{
	typedef TFunction<void(FText)> FLogFunc;

	class FTimeLogger
	{
	public:
		FTimeLogger(const FString& InText, FLogFunc&& InLogFunc)
		: StartTime( FPlatformTime::Cycles64() )
		, Text( InText )
		, LogFunc(MoveTemp(InLogFunc))
		{
			UE_LOGF( LogDataprep, Log, "%ls ...", *Text );
		}

		~FTimeLogger()
		{
			// Log time spent to import incoming file in minutes and seconds
			double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

			int ElapsedMin = int(ElapsedSeconds / 60.0);
			ElapsedSeconds -= 60.0 * (double)ElapsedMin;
			FText Msg = FText::Format( LOCTEXT("DataprepOperation_LogTime", "{0} took {1} min {2} s."), FText::FromString( Text ), ElapsedMin, FText::FromString( FString::Printf( TEXT("%.3f"), ElapsedSeconds ) ) );
			LogFunc( Msg );
		}

	private:
		uint64 StartTime;
		FString Text;
		FLogFunc LogFunc;
	};
}
#endif


void UDataprepSetLODsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	if(ReductionSettings.Num() > MAX_STATIC_MESH_LODS)
	{
		FText Message = FText::Format( LOCTEXT( "DatasmithMeshOperations_SetLODs_Max", "Limiting number of reduction settings to max allowed, {0}" ), MAX_STATIC_MESH_LODS );
		LogWarning( Message );
	}

	// Limit size of array to MAX_STATIC_MESH_LODS
	const int32 LODCount = FMath::Min( ReductionSettings.Num(), MAX_STATIC_MESH_LODS );
	if( LODCount == 0 )
	{
		FText OutReason = FText( LOCTEXT( "DatasmithMeshOperations_SetLODs", "No reduction settings. Aborting operation..." ) );
		LogInfo( OutReason );
		return;
	}

	// Fill up mesh reduction struct
	FStaticMeshReductionOptions ReductionOptions;

	ReductionOptions.bAutoComputeLODScreenSize = bAutoComputeLODScreenSize;
	ReductionOptions.ReductionSettings.SetNum( LODCount );

	for(int32 Index = 0; Index < LODCount; ++Index)
	{
		ReductionOptions.ReductionSettings[Index].PercentTriangles = FMath::Clamp( ReductionSettings[Index].PercentTriangles, 0.f, 1.f );
		ReductionOptions.ReductionSettings[Index].ScreenSize = FMath::Clamp( ReductionSettings[Index].ScreenSize, 0.f, 1.f );
	}

#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetLods"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetLods( InContext.Objects, ReductionOptions, ModifiedStaticMeshes );

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

UDataprepSetLODGroupOperation::UDataprepSetLODGroupOperation()
{
	TArray<FName> LODGroupNames;
	UStaticMesh::GetLODGroups( LODGroupNames );

	GroupName = LODGroupNames[0];
}

void UDataprepSetLODGroupOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetLODGroup"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetLODGroup( InContext.Objects, GroupName, ModifiedStaticMeshes );

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepSetSimpleCollisionOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetSimpleCollision"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetSimpleCollision( InContext.Objects, ShapeType, ModifiedStaticMeshes );

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepSetConvexDecompositionCollisionOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetConvexDecompositionCollision"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetConvexDecompositionCollision( InContext.Objects, HullCount, MaxHullVerts, HullPrecision, ModifiedStaticMeshes );

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepSetMobilityOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetMobility"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::SetMobility( InContext.Objects, MobilityType );
}

void UDataprepSetMaterialOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	if(Material == nullptr)
	{
		FText OutReason = FText( LOCTEXT( "DatasmithMeshOperations_SetMaterial", "No material specified. Aborting operation..." ) );
		LogInfo( OutReason );
		return;
	}

#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetMaterial"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::SetMaterial( InContext.Objects, Material );
}

void UDataprepSubstituteMaterialOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	if(MaterialSubstitute == nullptr)
	{
		FText OutReason = FText( LOCTEXT( "DatasmithDirProducer_SubstituteMaterial", "No material specified. Aborting operation..." ) );
		LogInfo( OutReason );
		return;
	}

#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SubstituteMaterial"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::SubstituteMaterial( InContext.Objects, MaterialSearch, StringMatch, MaterialSubstitute );
}

void UDataprepSubstituteMaterialByTableOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	if(MaterialDataTable == nullptr)
	{
		FText OutReason = FText( LOCTEXT( "DatasmithDirProducer_SubstituteMaterialByTable", "No data table specified. Aborting operation..." ) );
		LogInfo( OutReason );
		return;
	}

#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SubstituteMaterialsByTable"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::SubstituteMaterialsByTable( InContext.Objects, MaterialDataTable );
}

void FDataprepSetLODGroupDetails::OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type /*SelectInfo*/)
{
	int32 Index = LODGroupOptions.Find(NewValue);
	if (Index != INDEX_NONE && LodGroupPropertyHandle.IsValid() )
	{
		LodGroupPropertyHandle->SetValue( LODGroupNames[Index] );
	}
}

TSharedRef< SWidget > FDataprepSetLODGroupDetails::CreateWidget()
{
	// Build list of LODGroup names the user will choose from
	LODGroupNames.Reset();
	UStaticMesh::GetLODGroups( LODGroupNames );

	LODGroupOptions.Reset();
	TArray<FText> LODGroupDisplayNames;
	UStaticMesh::GetLODGroupsDisplayNames( LODGroupDisplayNames );
	for (int32 GroupIndex = 0; GroupIndex < LODGroupDisplayNames.Num(); ++GroupIndex)
	{
		LODGroupOptions.Add( MakeShareable( new FString( LODGroupDisplayNames[GroupIndex].ToString() ) ) );
	}

	// Set displayed value to what is used by the SetLODGroup operation
	int32 SelectedIndex = LODGroupNames.Find( DataprepOperation->GroupName );
	if(SelectedIndex == INDEX_NONE)
	{
		SelectedIndex = 0;
		DataprepOperation->GroupName = LODGroupNames[SelectedIndex];
	}
	

	// Create widget
	return	SNew( STextComboBox )
		.OptionsSource( &LODGroupOptions )
		.InitiallySelectedItem(LODGroupOptions[SelectedIndex])
		.OnSelectionChanged( this, &FDataprepSetLODGroupDetails::OnLODGroupChanged );
}

void FDataprepSetLODGroupDetails::CustomizeDetails(IDetailLayoutBuilder & DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized( Objects );
	check( Objects.Num() > 0 );

	DataprepOperation = Cast< UDataprepSetLODGroupOperation >(Objects[0].Get());
	check( DataprepOperation );

	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );

	FName CategoryName = CategoryNames.Num() > 0 ? CategoryNames[0] : FName( TEXT("SetLOGGroup_Internal") );
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory( CategoryName, FText::GetEmpty(), ECategoryPriority::Important );

	LodGroupPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDataprepSetLODGroupOperation, GroupName));

	// Hide GroupName property as it is replaced with custom widget
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataprepSetLODGroupOperation, GroupName));

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow( FText::FromString( TEXT( "LODGroup" ) ) );

	CustomAssetImportRow.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("DatasmithMeshOperationsLabel", "LODGroupName"))
		.ToolTipText(LOCTEXT("DatasmithMeshOperationsTooltip", "List of predefined LODGroup"))
		.Font( DetailBuilder.GetDetailFont() )
	];

	CustomAssetImportRow.ValueContent()
	[
		CreateWidget()
	];
}

void UDataprepSetMeshOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	if(StaticMesh == nullptr)
	{
		FText OutReason = FText( LOCTEXT( "DatasmithMeshOperations_SetMesh", "No mesh specified. Aborting operation..." ) );
		LogInfo( OutReason );
		return;
	}

#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetMesh"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::SetMesh( InContext.Objects, StaticMesh );
}

void UDataprepAddTagsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("AddTags"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::AddTags(InContext.Objects, Tags);
}

void UDataprepSetMetadataOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("AddMetadata"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::AddMetadata(InContext.Objects, Metadata);
}

void UDataprepConsolidateObjectsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("ConsolidateObjects"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::ConsolidateObjects(InContext.Objects);
}

void UDataprepRandomizeTransformOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("RandomizeTransform"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::RandomizeTransform(InContext.Objects, TransformType, ReferenceFrame, Min, Max);
}

void UDataprepRandomizeTransformOperation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataprepRandomizeTransformOperation, Min))
	{
		Max.X = FMath::Max(Max.X, Min.X);
		Max.Y = FMath::Max(Max.Y, Min.Y);
		Max.Z = FMath::Max(Max.Z, Min.Z);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataprepRandomizeTransformOperation, Max))
	{
		Min.X = FMath::Min(Max.X, Min.X);
		Min.Y = FMath::Min(Max.Y, Min.Y);
		Min.Z = FMath::Min(Max.Z, Min.Z);
	}
}

void UDataprepFlipFacesOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("FlipFaces"), [&](FText Text) { this->LogInfo(Text); });
#endif

	TSet<UStaticMesh*> StaticMeshes;

	// Re-create static meshes
	for (UObject* Object : InContext.Objects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
			for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
			{
				UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

				if (nullptr == StaticMesh)
				{
					continue;
				}
				StaticMeshes.Add(StaticMesh);
			}
		}
	}

	// Execute operation
	UDataprepOperationsLibrary::FlipFaces(StaticMeshes);

	// Re-create meshes render data
	UStaticMesh::BatchBuild(StaticMeshes.Array());
}

void UDataprepSetOutputFolder::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("RandomizeTransform"), [&](FText Text) { this->LogInfo(Text); });
#endif

	UDataprepOperationsLibrary::SetSubOuputFolder(InContext.Objects, FolderName);
}

void FDataprepSetOutputFolderDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized( Objects );
	check( Objects.Num() > 0 );

	Operation = Cast< UDataprepSetOutputFolder >(Objects[0].Get());
	check( Operation );

	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames( CategoryNames );

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory( NAME_None, FText::GetEmpty(), ECategoryPriority::Important );

	FolderNamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDataprepSetOutputFolder, FolderName));
	FolderNamePropertyHandle->MarkHiddenByCustomization();

	FDetailWidgetRow& CustomAssetImportRow = CategoryBuilder.AddCustomRow( FText::FromString( TEXT( "Folder Name" ) ) );

	CustomAssetImportRow.NameContent()
	[
		FolderNamePropertyHandle->CreatePropertyNameWidget()
	];

	CustomAssetImportRow.ValueContent()
	[
		SAssignNew(TextBox, SEditableTextBox)
		.OnTextChanged(this, &FDataprepSetOutputFolderDetails::FolderName_TextChanged)
		.OnTextCommitted(this, &FDataprepSetOutputFolderDetails::FolderName_TextCommited)
		.Text(FText::FromString(Operation->FolderName))
	];
}

void FDataprepSetOutputFolderDetails::FolderName_TextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	if (bValidFolderName)
	{
		FolderNamePropertyHandle->SetValue(InText.ToString());
	}
	else
	{
		// New name is not valid: revert to old folder name
		TextBox->SetText(FText::FromString(Operation->FolderName));
	}

	bValidFolderName = true;
}

void FDataprepSetOutputFolderDetails::FolderName_TextChanged(const FText& Text)
{
	// Slash and Square brackets are invalid characters for a folder name
	const FString InvalidChars = INVALID_OBJECTPATH_CHARACTERS TEXT(".");

	FText ErrorMessage;
	FString FolderPath = Text.ToString();

	if (FolderPath.Len() == 0)
	{
		ErrorMessage = LOCTEXT("InvalidFolderName_PathEmpty", "A folder path must be at least one character long");
	}
	else if (FolderPath.StartsWith(TEXT("/")))
	{
		ErrorMessage = LOCTEXT("InvalidFolderName_MustBeRelative", "A folder path must be relative");
	}
	else
	{
		// See if the name contains invalid characters.
		FString Char;
		for (int32 CharIdx = 0; CharIdx < FolderPath.Len(); ++CharIdx)
		{
			Char = FolderPath.Mid(CharIdx, 1);

			if (InvalidChars.Contains(*Char))
			{
				FString ReadableInvalidChars = InvalidChars;
				ReadableInvalidChars.ReplaceInline(TEXT("\r"), TEXT(""));
				ReadableInvalidChars.ReplaceInline(TEXT("\n"), TEXT(""));
				ReadableInvalidChars.ReplaceInline(TEXT("\t"), TEXT(""));

				ErrorMessage = FText::Format(LOCTEXT("InvalidFolderName_InvalidCharacters", "A folder path may not contain any of the following characters: {0}"), FText::FromString(ReadableInvalidChars));
				break;
			}
		}
	}

	if (!ErrorMessage.IsEmpty() || !FFileHelper::IsFilenameValidForSaving(FolderPath, ErrorMessage))
	{
		TextBox->SetError(ErrorMessage);
	}
	else
	{
		// Clear error
		TextBox->SetError(FText::GetEmpty());
	}

	bValidFolderName = ErrorMessage.IsEmpty();
}

void UDataprepAddToLayerOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("AddToLayer"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	UDataprepOperationsLibrary::AddToLayer(InContext.Objects, LayerName);
}

void UDataprepSetCollisionComplexityOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger( TEXT("SetCollisionComplexity"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetCollisionComplexity( InContext.Objects, CollisionTraceFlag, ModifiedStaticMeshes );

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepSetMaxTextureSizeOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("SetMaxTextureSize"), [&](FText Text) { this->LogInfo(Text); });
#endif

	TSet<UTexture2D*> Textures;

	// Get the textures to resize
	for (UObject* Object : InContext.Objects)
	{
		if (UTexture2D* Texture = Cast< UTexture2D >(Object))
		{
			const int32 TextureWidth = Texture->GetSizeX();
			const int32 TextureHeight = Texture->GetSizeY();
			const bool bPowerOfTwo = FMath::IsPowerOfTwo(TextureWidth) && FMath::IsPowerOfTwo(TextureHeight);

			if (bPowerOfTwo || bAllowPadding)
			{
				Textures.Add(Texture);
			}
		}
	}

	// Execute operation
	UDataprepOperationsLibrary::ResizeTextures(Textures.Array(), MaxTextureSize);
}

void UDataprepSetMaxTextureSizeOperation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataprepSetMaxTextureSizeOperation, MaxTextureSize))
	{
		if (!FMath::IsPowerOfTwo(MaxTextureSize))
		{
			MaxTextureSize = FMath::RoundUpToPowerOfTwo(MaxTextureSize);
		}
	}
}

void UDataprepSetNaniteSettingsOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepOperationTime::FTimeLogger TimeLogger(TEXT("SetNaniteSettings"), [&](FText Text) { this->LogInfo(Text); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;
	UDataprepOperationsLibrary::SetNaniteSettings(InContext.Objects, bNaniteEnabled, PositionPrecision, PercentTriangles * 0.01f, ModifiedStaticMeshes);

	if (ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified(MoveTemp(ModifiedStaticMeshes));
	}
}

// Helper functions to replicate setting of PositionPrecision for Nanite settings
// in StaticMesh editor.
// Borrowed from FNaniteSettingsLayoutin StaticMeshEditorTool.cpp
namespace NaniteSettingsUtils
{
	static const int32 DisplayPositionPrecisionMin = -6;
	static const int32 DisplayPositionPrecisionMax = 13;
	/** Strings representing valid PositionPrecision. */
	static TArray< TSharedPtr< FString > > PositionPrecisionNames;

	int32 PositionPrecisionIndexToValue(int32 Index)
	{
		check(Index >= 0);

		if (Index == 0)
		{
			return MIN_int32;
		}
		else
		{
			int32 Value = DisplayPositionPrecisionMin + (Index - 1);
			Value = FMath::Min(Value, DisplayPositionPrecisionMax);
			return Value;
		}
	}

	int32 PositionPrecisionValueToIndex(int32 Value)
	{
		if (Value == MIN_int32)
		{
			return 0;
		}
		else
		{
			Value = FMath::Clamp(Value, DisplayPositionPrecisionMin, DisplayPositionPrecisionMax);
			return Value - DisplayPositionPrecisionMin + 1;
		}
	}

	FString PositionPrecisionValueToDisplayString(int32 Value)
	{
		check(Value != MIN_int32);

		if (Value <= 0)
		{
			return FString::Printf(TEXT("%dcm"), 1 << (-Value));
		}
		else
		{
			const float fValue = FMath::Exp2((double)-Value);
			return FString::Printf(TEXT("1/%dcm (%.3gcm)"), 1 << Value, fValue);
		}
	}
}

void FDataprepSetNaniteSettingsDetails::OnPositionPrecisionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type /*SelectInfo*/)
{
	using namespace NaniteSettingsUtils;

	int32 Index = PositionPrecisionNames.Find(NewValue);
	if (Index != INDEX_NONE && PositionPrecisionPropertyHandle.IsValid())
	{
		PositionPrecisionPropertyHandle->SetValue(PositionPrecisionIndexToValue(Index));
	}
}

TSharedRef< SWidget > FDataprepSetNaniteSettingsDetails::CreateWidget()
{
	using namespace NaniteSettingsUtils;

	// Build list of PositionPrecision names the user will choose from
	if (PositionPrecisionNames.Num() == 0)
	{
		PositionPrecisionNames.Reserve(20);

		PositionPrecisionNames.Add(MakeShareable(new FString(TEXT("Auto"))));

		for (int32 Value = DisplayPositionPrecisionMin; Value <= DisplayPositionPrecisionMax; ++Value)
		{
			PositionPrecisionNames.Add(MakeShareable(new FString(PositionPrecisionValueToDisplayString(Value))));
		}
	}

	// Set displayed value to what is used in this operator's PositionPrecision
	int32 SelectedIndex = PositionPrecisionValueToIndex(DataprepOperation->PositionPrecision);


	// Create widget
	return	SNew(STextComboBox)
		.OptionsSource(&PositionPrecisionNames)
		.InitiallySelectedItem(PositionPrecisionNames[SelectedIndex])
		.OnSelectionChanged(this, &FDataprepSetNaniteSettingsDetails::OnPositionPrecisionChanged);
}

void FDataprepSetNaniteSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(Objects.Num() > 0);

	DataprepOperation = Cast< UDataprepSetNaniteSettingsOperation >(Objects[0].Get());
	check(DataprepOperation);

	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames(CategoryNames);

	FName CategoryName = CategoryNames.Num() > 0 ? CategoryNames[0] : FName(TEXT("MeshOperation_Internal"));
	IDetailCategoryBuilder& ImportSettingsCategoryBuilder = DetailBuilder.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::Important);

	PositionPrecisionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDataprepSetNaniteSettingsOperation, PositionPrecision));

	// Hide PositionPrecision property as it is replaced with custom widget
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDataprepSetNaniteSettingsOperation, PositionPrecision));

	FDetailWidgetRow& CustomAssetImportRow = ImportSettingsCategoryBuilder.AddCustomRow(FText::FromString(TEXT("Position Precision")));

	CustomAssetImportRow.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DatasmithNaniteOperationsLabel", "PositionPrecision"))
			.ToolTipText(LOCTEXT("DatasmithNaniteOperationsTooltip", "List of predefined position precision"))
			.Font(DetailBuilder.GetDetailFont())
		];

	CustomAssetImportRow.ValueContent()
		[
			CreateWidget()
		];
}

#undef LOCTEXT_NAMESPACE
