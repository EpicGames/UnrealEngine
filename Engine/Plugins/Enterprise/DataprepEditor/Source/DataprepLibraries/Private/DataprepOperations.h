// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "BodySetupEnums.h"
#include "DataprepOperation.h"

#include "DataprepOperationsLibrary.h"

#include "IDetailCustomization.h"

#include "StaticMeshEditorSubsystemHelpers.h"
#include "DataprepOperations.generated.h"

class IPropertyHandle;
class SWidget;
class UMaterialInterface;
class UDataTable;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
struct FPropertyChangedEvent;

/** Local struct used by UDataprepSetLODsOperation to better control UX */
USTRUCT(BlueprintType)
struct FDataprepSetLODsReductionSettings
{
	GENERATED_BODY()

	// Percentage of triangles to keep. Ranges from 0.0 to 1.0: 1.0 = no reduction, 0.0 = no triangles.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SetLODsReductionSettings, meta=(UIMin = "0.0", UIMax = "1.0"))
	float PercentTriangles = 0.5f;

	// ScreenSize to display this LOD. Ranges from 0.0 to 1.0.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SetLODsReductionSettings, meta=(UIMin = "0.0", UIMax = "1.0"))
	float ScreenSize = 0.5f;
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set LODs", ToolTip = "For each static mesh to process, replace the existing static mesh's LODs with new ones based on the set of reduction settings") )
class UDataprepSetLODsOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetLODsOperation()
		: bAutoComputeLODScreenSize(true)
	{
	}

public:
	// If true, the screen sizes at which LODs swap are computed automatically
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (DisplayName = "Auto Screen Size", ToolTip = "If true, the screen sizes at which LODs swap are automatically computed"))
	bool bAutoComputeLODScreenSize;

	// Array of reduction settings to apply to each new LOD.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, Meta = (ToolTip = "Array of LOD reduction settings") )
	TArray<FDataprepSetLODsReductionSettings> ReductionSettings;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set LOD Group", ToolTip = "For each static mesh to process, replace the existing static mesh's LODs with new ones based on selected group") )
class UDataprepSetLODGroupOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:
	UDataprepSetLODGroupOperation();

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

private:
	// Name of the pre-defined LOD group to apply on the selected objects
	UPROPERTY(EditAnywhere, Category = SetLOGGroup_Internal, meta = (ToolTip = ""))
	FName GroupName;

	friend class FDataprepSetLODGroupDetails;
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set Simple Collision", ToolTip = "For each static mesh to process, replace the existing static mesh's collision setup with a simple one based on selected shape") )
class UDataprepSetSimpleCollisionOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetSimpleCollisionOperation()
		: ShapeType(EScriptCollisionShapeType::Box)
	{
	}

public:
	// Shape's of the collision geometry encompassing the static mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Shape's of the collision geometry encompassing the static mesh"))
		EScriptCollisionShapeType ShapeType;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set Convex Collision", ToolTip = "For each static mesh to process, replace the existing static mesh's collision setup with a convex decomposition one computed using the Hull settings") )
class UDataprepSetConvexDecompositionCollisionOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetConvexDecompositionCollisionOperation()
		: HullCount(4)
		, MaxHullVerts(16)
		, HullPrecision(100000)
	{
	}

public:
	// Maximum number of convex pieces that will be created
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Maximum number of convex pieces that will be created"))
	int32 HullCount;
	
	// Maximum number of vertices allowed for any generated convex hulls
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Maximum number of vertices allowed for any generated convex hulls"))
	int32 MaxHullVerts;
	
	// Number of voxels to use when generating collision
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Number of voxels to use when generating collision"))
	int32 HullPrecision;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ActorOperation, Meta = (DisplayName="Set Mobility", ToolTip = "For each actor to process, update its mobilty with the selected value") )
class UDataprepSetMobilityOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetMobilityOperation()
		: MobilityType(EComponentMobility::Static)
	{
	}

public:
	// Type of mobility to set on mesh actors
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ActorOperation, meta = (ToolTip = "Type of mobility to set on actors"))
	TEnumAsByte<EComponentMobility::Type> MobilityType;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set Material", ToolTip = "On each static mesh or actor to process, replace any materials used with the specified one") )
class UDataprepSetMaterialOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetMaterialOperation()
		: Material(nullptr)
	{
	}

public:
	// Material to use as a substitute
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Material to use as a substitute"))
	TObjectPtr<UMaterialInterface> Material;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ObjectOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Substitute Material", ToolTip = "On each static mesh or actor to process, replace the material matching the criteria with the specified one") )
class UDataprepSubstituteMaterialOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSubstituteMaterialOperation()
		: MaterialSearch(TEXT("*"))
		, StringMatch(EEditorScriptingStringMatchType::MatchesWildcard)
		, MaterialSubstitute(nullptr)
	{
	}

public:
	// Name of the material(s) to search for. Wildcard is supported
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Name of the material(s) to search for. Wildcard is supported"))
	FString MaterialSearch;

	// Type of matching to perform with MaterialSearch string
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Type of matching to perform with MaterialSearch string"))
	EEditorScriptingStringMatchType StringMatch;

	// Material to use as a substitute
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Material to use as a substitute"))
	TObjectPtr<UMaterialInterface> MaterialSubstitute;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ObjectOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Substitute Material By Table", ToolTip = "On each static mesh or actor to process, replace the material found in the first column of the table with the one from the second column in the same row") )
class UDataprepSubstituteMaterialByTableOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSubstituteMaterialByTableOperation()
		: MaterialDataTable(nullptr)
	{
	}

public:
	// Data table to use for the substitution
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Data table to use for the substitution"))
	TObjectPtr<UDataTable> MaterialDataTable;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ObjectOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

// Customization of the details of the Datasmith Scene for the data prep editor.
class FDataprepSetLODGroupDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDataprepSetLODGroupDetails>(); };

	FDataprepSetLODGroupDetails() : DataprepOperation(nullptr) {}

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	TSharedRef< SWidget > CreateWidget();
	void OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

private:
	UDataprepSetLODGroupOperation* DataprepOperation;

	/** LOD group options. */
	TArray< TSharedPtr< FString > > LODGroupOptions;
	TArray<FName>					LODGroupNames;

	TSharedPtr<IPropertyHandle> LodGroupPropertyHandle;
};

UCLASS(Category = ActorOperation, Meta = (DisplayName="Set Mesh", ToolTip = "On each actor to process, replace any meshes used with the specified one") )
class UDataprepSetMeshOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetMeshOperation()
	: StaticMesh(nullptr)
	{
	}

public:
	// Mesh to use as a substitute
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ActorOperation, meta = (ToolTip = "Mesh to use as a substitute"))
	TObjectPtr<UStaticMesh> StaticMesh;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Add Tags", ToolTip = "On each actor to process, add specified tags"))
class UDataprepAddTagsOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:
	// Array of reduction settings to apply to each new LOD.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ActorOperation, Meta = (ToolTip = "Array of tags to add"))
	TArray<FName> Tags;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Set Metadata", ToolTip = "On each actor to process set metadata value"))
class UDataprepSetMetadataOperation : public UDataprepOperation
{
	GENERATED_BODY()

private:
	// Table of metadata keys/values.
	UPROPERTY(EditAnywhere, Category = ActorOperation, Meta = (ToolTip = "Array of metadata values"))
	TMap<FName, FString> Metadata;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = AssetOperation, Meta = (DisplayName = "Replace Asset References", ToolTip = "Replace references to each asset with the first asset in the list"))
class UDataprepConsolidateObjectsOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::AssetOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Random Offset Transform", ToolTip = "For each actor in the input set, offset its position/rotation/scale with random vector generated from X/Y/Z Min-Max."))
class UDataprepRandomizeTransformOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepRandomizeTransformOperation()
		: TransformType(ERandomizeTransformType::Location) 
		, ReferenceFrame(ERandomizeTransformReferenceFrame::Relative)
	{
	}

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

	UPROPERTY(EditAnywhere, Category = ActorOperation, Meta = (ToolTip = "Transform component to randomize"))
	ERandomizeTransformType TransformType;

	UPROPERTY(EditAnywhere, Category = ActorOperation, Meta = (ToolTip = "Reference frame to use (relative/world)"))
	ERandomizeTransformReferenceFrame ReferenceFrame;

	UPROPERTY(EditAnywhere, Category = ActorOperation, Meta = (ToolTip = "Min random value"))
	FVector Min;

	UPROPERTY(EditAnywhere, Category = ActorOperation, Meta = (ToolTip = "Max random value"))
	FVector Max;

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

	// Track Min/Max changes and force values to be meaningful: Min<=Max
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Flip Faces", ToolTip = "On each actor to process, flip faces of each mesh"))
class UDataprepFlipFacesOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = AssetOperation, Meta = (DisplayName="Output to Folder", ToolTip = "For each asset to process, set the sub-folder to save it to.\nThe sub-folder is relative to the folder specified to the Dataprep consumer.") )
class UDataprepSetOutputFolder : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetOutputFolder()
	: FolderName(TEXT("MySubFolder"))
	{
	}

public:
	// If true, the screen sizes at which LODs swap are computed automatically
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AssetOperation, meta = (DisplayName = "Folder Name", ToolTip = "Name of the sub folder the assets to be saved to"))
	FString FolderName;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::AssetOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};
// Customization of the details of the "Output to Folder" operation.
class FDataprepSetOutputFolderDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDataprepSetOutputFolderDetails>(); };

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	void FolderName_TextChanged(const FText& Text);
	void FolderName_TextCommited(const FText& InText, ETextCommit::Type InCommitType);

private:
	UDataprepSetOutputFolder* Operation = nullptr;

	TSharedPtr<IPropertyHandle> FolderNamePropertyHandle;

	bool bValidFolderName = true;

	TSharedPtr< class SEditableTextBox > TextBox;
};

UCLASS(Category = ActorOperation, Meta = (DisplayName = "Add To Layer", ToolTip = "On each actor to process, add the actor to the layer"))
class UDataprepAddToLayerOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:
	// Name of the layer to add the actors to.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ActorOperation, Meta = (ToolTip = "Name of the layer to add the actors to"))
	FName LayerName;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::ActorOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = MeshOperation, Meta = (DisplayName="Set Collision Complexity", ToolTip = "For each static mesh to process, set its collision complexity") )
class UDataprepSetCollisionComplexityOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepSetCollisionComplexityOperation()
		: CollisionTraceFlag(ECollisionTraceFlag::CTF_UseDefault)
	{
	}

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "Collision complexity"))
	TEnumAsByte<ECollisionTraceFlag> CollisionTraceFlag;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Category = AssetOperation, Meta = (DisplayName = "Set Max Texture Size", ToolTip = "Set max size (width or height) each input texture"))
class UDataprepSetMaxTextureSizeOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = AssetOperation, BlueprintReadWrite, meta = (UIMin = "1", ClampMin = "1"), Meta = (ToolTip = ""))
	int32 MaxTextureSize;

	UPROPERTY(EditAnywhere, Category = AssetOperation, BlueprintReadWrite, meta = (ToolTip = "If true, original texture size will be enforced to power of two before resizing (if it's a non-power of two size), else only POT textures will be affected."))
	bool bAllowPadding = false;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::AssetOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

	// Track changes of MaxTextureSize and force values to be power of two
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

UCLASS(Category = MeshOperation, Meta = (DisplayName = "Set Nanite Settings", ToolTip = "For each static mesh to process, enables/disables Nanite build and if enabled update settings."))
class UDataprepSetNaniteSettingsOperation : public UDataprepOperation
{
	GENERATED_BODY()

public:
	/** If true, Nanite data will be generated */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (ToolTip = "If checked, Nanite data will be generated"))
	bool bNaniteEnabled = false;

	/** Position Precision. Step size is 2^(-PositionPrecision) cm. MIN_int32 is auto. */
	// #ueent_todo: Same display as static mesh editor. See FNaniteSettingsLayout in StaticMeshEditorTool.h
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation_Internal, meta = (/*EditConditionHides, EditCondition = "bNaniteEnabled",*/ ToolTip = "Step size is 2^(-PositionPrecision) cm. MIN_int32 is auto"))
	int32 PositionPrecision = MIN_int32;

	/** Percentage of triangles to keep from LOD0. 100.0 = no reduction, 0.0 = no triangles. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshOperation, meta = (/*EditConditionHides, EditCondition = "bNaniteEnabled",*/ ClampMin = 0, ClampMax = 100, UIMin = "0.0", UIMax = "100.0", ToolTip = "Percentage of triangles to keep from LOD0. 100.0 = no reduction, 0.0 = no triangles"))
	float PercentTriangles = 100.f;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

// Customization of the details of the DataprepSetNaniteSettingsOperation for the Dataprep editor.
class FDataprepSetNaniteSettingsDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDataprepSetNaniteSettingsDetails>(); };

	FDataprepSetNaniteSettingsDetails() : DataprepOperation(nullptr) {}

	/** Called when details should be customized */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TSharedRef< SWidget > CreateWidget();
	void OnPositionPrecisionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

private:
	UDataprepSetNaniteSettingsOperation* DataprepOperation;

	TSharedPtr<IPropertyHandle> PositionPrecisionPropertyHandle;
};
