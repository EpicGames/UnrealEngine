// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepOperationsLibrary.h"
#include "Components/StaticMeshComponent.h"
#include "DataprepOperationsLibraryUtil.h"

#include "DataprepCoreUtils.h"
#include "DataprepContentConsumer.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithAreaLightActor.h"

#include "Editor/EditorEngine.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialInstance.h"
#include "ObjectTools.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshEditorSubsystemHelpers.h"
#include "StaticMeshOperations.h"
#include "StaticMeshEditorSubsystem.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY(LogDataprep);

#define LOCTEXT_NAMESPACE "DataprepOperationsLibrary"

extern UNREALED_API UEditorEngine* GEditor;

void UDataprepOperationsLibrary::SetLods(const TArray<UObject*>& SelectedObjects, const FStaticMeshReductionOptions& ReductionOptions, TArray<UObject*>& ModifiedObjects)
{
	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(SelectedObjects);

	UStaticMeshEditorSubsystem* StaticMeshEditorSubsystem = GEditor->GetEditorSubsystem<UStaticMeshEditorSubsystem>();

	if (!StaticMeshEditorSubsystem)
	{
		return;
	}

	// Create LODs but do not commit changes
	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (StaticMesh)
		{
			DataprepOperationsLibraryUtil::FScopedStaticMeshEdit StaticMeshEdit( StaticMesh );

			StaticMeshEditorSubsystem->SetLodsWithNotification(StaticMesh, ReductionOptions, false);

			ModifiedObjects.Add( StaticMesh );
		}
	}
}

void UDataprepOperationsLibrary::SetSimpleCollision(const TArray<UObject*>& SelectedObjects, const EScriptCollisionShapeType ShapeType, TArray<UObject*>& ModifiedObjects)
{
	UStaticMeshEditorSubsystem* StaticMeshEditorSubsystem = GEditor->GetEditorSubsystem<UStaticMeshEditorSubsystem>();

	if (!StaticMeshEditorSubsystem)
	{
		return;
	}

	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(SelectedObjects);

	// Make sure all static meshes to be processed have render data for NDOP types
	bool bNeedRenderData = false;
	switch (ShapeType)
	{
		case EScriptCollisionShapeType::NDOP10_X:
		case EScriptCollisionShapeType::NDOP10_Y:
		case EScriptCollisionShapeType::NDOP10_Z:
		case EScriptCollisionShapeType::NDOP18:
		case EScriptCollisionShapeType::NDOP26:
		{
			bNeedRenderData = true;
			break;
		}
		default:
		{
			break;
		}
	}

	DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder( bNeedRenderData ? SelectedMeshes : TSet<UStaticMesh*>() );

	// Create LODs but do not commit changes
	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (StaticMesh)
		{
			DataprepOperationsLibraryUtil::FScopedStaticMeshEdit StaticMeshEdit( StaticMesh );

			// Remove existing simple collisions
			StaticMeshEditorSubsystem->RemoveCollisionsWithNotification( StaticMesh, false );

			StaticMeshEditorSubsystem->AddSimpleCollisionsWithNotification( StaticMesh, ShapeType, false );

			ModifiedObjects.Add( StaticMesh );
		}
	}
}

void UDataprepOperationsLibrary::SetConvexDecompositionCollision(const TArray<UObject*>& SelectedObjects, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, TArray<UObject*>& ModifiedObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDataprepOperationsLibrary::SetConvexDecompositionCollision)

	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(SelectedObjects);

	// Make sure all static meshes to be processed have render data
	DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder(SelectedMeshes);

	TArray<UStaticMesh*> StaticMeshes = SelectedMeshes.Array();
	StaticMeshes.RemoveAll([](UStaticMesh* StaticMesh) { return StaticMesh == nullptr; });

	// Build complex collision
	UStaticMeshEditorSubsystem* StaticMeshEditorSubsystem = GEditor->GetEditorSubsystem<UStaticMeshEditorSubsystem>();\

	if (!StaticMeshEditorSubsystem)
	{
		return;
	}

	StaticMeshEditorSubsystem->BulkSetConvexDecompositionCollisionsWithNotification(StaticMeshes, HullCount, MaxHullVerts, HullPrecision, false);

	ModifiedObjects.Append(StaticMeshes);
}

void UDataprepOperationsLibrary::SubstituteMaterial(const TArray<UObject*>& SelectedObjects, const FString& MaterialSearch, EEditorScriptingStringMatchType StringMatch, UMaterialInterface* MaterialSubstitute)
{
	TArray<UMaterialInterface*> MaterialsUsed = DataprepOperationsLibraryUtil::GetUsedMaterials(SelectedObjects);

	SubstituteMaterial(SelectedObjects, MaterialSearch, StringMatch, MaterialsUsed, MaterialSubstitute);
}

void UDataprepOperationsLibrary::SubstituteMaterialsByTable(const TArray<UObject*>& SelectedObjects, const UDataTable* DataTable)
{
	if (DataTable == nullptr || DataTable->GetRowStruct() == nullptr || !DataTable->GetRowStruct()->IsChildOf(FMaterialSubstitutionDataTable::StaticStruct()))
	{
		return;
	}

	TArray<UMaterialInterface*> MaterialsUsed = DataprepOperationsLibraryUtil::GetUsedMaterials(SelectedObjects);

	const TMap<FName, uint8*>&  MaterialTableRowMap = DataTable->GetRowMap();
	for (auto& MaterialTableRowEntry : MaterialTableRowMap)
	{
		const FMaterialSubstitutionDataTable* MaterialRow = (const FMaterialSubstitutionDataTable*)MaterialTableRowEntry.Value;
		if (MaterialRow != nullptr && MaterialRow->MaterialReplacement != nullptr)
		{
			SubstituteMaterial(SelectedObjects, MaterialRow->SearchString, MaterialRow->StringMatch, MaterialsUsed, MaterialRow->MaterialReplacement);
		}
	}
}

void UDataprepOperationsLibrary::SubstituteMaterial(const TArray<UObject*>& SelectedObjects, const FString& MaterialSearch, EEditorScriptingStringMatchType StringMatch, const TArray<UMaterialInterface*>& MaterialList, UMaterialInterface* MaterialSubstitute)
{
	TArray<UObject*> MatchingObjects = UEditorFilterLibrary::ByIDName(TArray<UObject*>(MaterialList), MaterialSearch, StringMatch, EEditorScriptingFilterType::Include);

	TArray<UMaterialInterface*> MaterialsToReplace;
	for (UObject* Object : MatchingObjects)
	{
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
		{
			MaterialsToReplace.Add(MaterialInterface);
		}
	}

	for (UMaterialInterface* MaterialToReplace : MaterialsToReplace)
	{
		for (UObject* Object : SelectedObjects)
		{
			if (AActor* Actor = Cast< AActor >(Object))
			{
				// Find the materials by iterating over every mesh component.
				TInlineComponentArray<UMeshComponent*> MeshComponents(Actor);
				for (UMeshComponent* MeshComponent : MeshComponents)
				{
					int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

					for (int32 Index = 0; Index < MaterialCount; ++Index)
					{
						if (MeshComponent->GetMaterial(Index) == MaterialToReplace)
						{
							MeshComponent->SetMaterial(Index, MaterialSubstitute);
						}
					}
				}
			}
			else if (UMeshComponent* MeshComponent = Cast< UMeshComponent >(Object))
			{
				int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

				for (int32 Index = 0; Index < MaterialCount; ++Index)
				{
					if (MeshComponent->GetMaterial(Index) == MaterialToReplace)
					{
						MeshComponent->SetMaterial(Index, MaterialSubstitute);
					}
				}
			}
			else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
			{
				DataprepOperationsLibraryUtil::FScopedStaticMeshEdit StaticMeshEdit( StaticMesh );

				TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
				for (int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
				{
					if (StaticMesh->GetMaterial(Index) == MaterialToReplace)
					{
						DataprepOperationsLibraryUtil::SetMaterial( StaticMesh, Index, MaterialSubstitute );
					}
				}
			}
		}
	}
}

void UDataprepOperationsLibrary::SetMobility( const TArray< UObject* >& SelectedObjects, EComponentMobility::Type MobilityType )
{
	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			TInlineComponentArray<USceneComponent*> SceneComponents(Actor);
			for (USceneComponent* SceneComponent : SceneComponents)
			{
				SceneComponent->SetMobility(MobilityType);
			}

			if (ADatasmithAreaLightActor* DatasmithAreaLightActor = Cast<ADatasmithAreaLightActor>(Actor))
			{
				DatasmithAreaLightActor->Mobility = MobilityType;
			}
		}
		else if (USceneComponent* SceneComponent = Cast< USceneComponent >(Object))
		{
			SceneComponent->SetMobility(MobilityType);
		}
	}
}

void UDataprepOperationsLibrary::SetMaterial( const TArray< UObject* >& SelectedObjects, UMaterialInterface* MaterialSubstitute )
{
	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			// Find the materials by iterating over every mesh component.
			TInlineComponentArray<UMeshComponent*> MeshComponents(Actor);
			for (UMeshComponent* MeshComponent : MeshComponents)
			{
				int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

				for (int32 Index = 0; Index < MaterialCount; ++Index)
				{
					MeshComponent->SetMaterial(Index, MaterialSubstitute);
				}
			}
		}
		else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
		{
			DataprepOperationsLibraryUtil::FScopedStaticMeshEdit StaticMeshEdit( StaticMesh );

			for (int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
			{
				DataprepOperationsLibraryUtil::SetMaterial( StaticMesh, Index, MaterialSubstitute );
			}
		}
		else if (UMeshComponent* MeshComponent = Cast< UMeshComponent >(Object))
		{
			int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

			for (int32 Index = 0; Index < MaterialCount; ++Index)
			{
				MeshComponent->SetMaterial(Index, MaterialSubstitute);
			}
		}
	}
}

void UDataprepOperationsLibrary::SetLODGroup( const TArray<UObject*>& SelectedObjects, FName& LODGroupName, TArray<UObject*>& ModifiedObjects )
{
	TArray<FName> LODGroupNames;
	UStaticMesh::GetLODGroups( LODGroupNames );

	if ( LODGroupNames.Find( LODGroupName ) != INDEX_NONE )
	{
		TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(SelectedObjects);

		// Apply the new LODGroup without rebuilding the static mesh
		for (UStaticMesh* StaticMesh : SelectedMeshes)
		{
			if(StaticMesh)
			{
				StaticMesh->SetLODGroup( LODGroupName, false);
				ModifiedObjects.Add( StaticMesh );
			}
		}
	}
}

void UDataprepOperationsLibrary::SetMesh(const TArray<UObject*>& SelectedObjects, UStaticMesh* MeshSubstitute)
{
	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			// Find the meshes by iterating over every mesh component.
			TInlineComponentArray<UStaticMeshComponent*> MeshComponents(Actor);
			for (UStaticMeshComponent* MeshComponent : MeshComponents)
			{
				if(MeshComponent)
				{
					MeshComponent->SetStaticMesh( MeshSubstitute );
				}
			}
		}
		else if (UStaticMeshComponent* MeshComponent = Cast< UStaticMeshComponent >( Object ))
		{
			MeshComponent->SetStaticMesh( MeshSubstitute );
		}
	}
}

void UDataprepOperationsLibrary::SubstituteMesh(const TArray<UObject*>& SelectedObjects, const FString& MeshSearch, EEditorScriptingStringMatchType StringMatch, UStaticMesh* MeshSubstitute)
{
	TArray<UStaticMesh*> MeshesUsed = DataprepOperationsLibraryUtil::GetUsedMeshes(SelectedObjects);

	SubstituteMesh( SelectedObjects, MeshSearch, StringMatch, MeshesUsed, MeshSubstitute );
}

void UDataprepOperationsLibrary::SubstituteMeshesByTable(const TArray<UObject*>& , const UDataTable* )
{
}

void UDataprepOperationsLibrary::SubstituteMesh(const TArray<UObject*>& SelectedObjects, const FString& MeshSearch, EEditorScriptingStringMatchType StringMatch, const TArray<UStaticMesh*>& MeshList, UStaticMesh* MeshSubstitute)
{
	TArray<UObject*> MatchingObjects = UEditorFilterLibrary::ByIDName(TArray<UObject*>(MeshList), MeshSearch, StringMatch, EEditorScriptingFilterType::Include);

	TSet<UStaticMesh*> MeshesToReplace;
	for (UObject* Object : MatchingObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			MeshesToReplace.Add(StaticMesh);
		}
	}

	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			// Find the meshes by iterating over every mesh component.
			TInlineComponentArray<UStaticMeshComponent*> MeshComponents(Actor);
			for (UStaticMeshComponent* MeshComponent : MeshComponents)
			{
				if( MeshesToReplace.Contains( MeshComponent->GetStaticMesh() ) )
				{
					MeshComponent->SetStaticMesh( MeshSubstitute );
				}
			}
		}
	}
}

void UDataprepOperationsLibrary::AddTags(const TArray< UObject* >& SelectedObjects, const TArray<FName>& InTags)
{
	TFunction<void(TArray<FName>&)> AddNewTags = [&InTags](TArray<FName>& InExistingTags)
	{
		for (int TagIndex = 0; TagIndex < InTags.Num(); ++TagIndex)
		{
			if (!InTags[TagIndex].IsNone() && (INDEX_NONE == InExistingTags.Find(InTags[TagIndex])))
			{
				InExistingTags.Add(InTags[TagIndex]);
			}
		}
	};

	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			AddNewTags(Actor->Tags);
		}
		else if (UActorComponent* Comp = Cast< UActorComponent >(Object))
		{
			AddNewTags(Comp->ComponentTags);
		}
	}
}

void UDataprepOperationsLibrary::AddMetadata(const TArray<UObject*>& SelectedObjects, const TMap<FName, FString>& InMetadata)
{
	UDatasmithAssetUserData::FMetaDataContainer Metadata;

	// Add Datasmith meta data
	int32 ValueCount = InMetadata.Num();
	Metadata.Reserve(ValueCount);

	for (auto& Elem : InMetadata)
	{
		Metadata.Add(Elem.Key, *Elem.Value);
	}

	Metadata.KeySort(FNameLexicalLess());

	if (Metadata.Num() > 0)
	{
		for (UObject* Object : SelectedObjects)
		{
			if (AActor* Actor = Cast< AActor >(Object))
			{
				UActorComponent* ActorComponent = Actor->GetRootComponent();
				if (ActorComponent)
				{
					Object = ActorComponent;
				}
			}

			if (Object->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
			{
				IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Object);

				UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData< UDatasmithAssetUserData >();

				if (!DatasmithUserData)
				{
					DatasmithUserData = NewObject<UDatasmithAssetUserData>(Object, NAME_None, RF_Public | RF_Transactional);
					AssetUserData->AddAssetUserData(DatasmithUserData);
				}

				DatasmithUserData->MetaData.Append(Metadata);
			}
		}
	}
}

void UDataprepOperationsLibrary::ConsolidateObjects(const TArray< UObject* >& SelectedObjects)
{
	if (SelectedObjects.Num() < 2)
	{
		return;
	}

	// Use the first object as the consolidation object.
	UObject* ObjectToConsolidateTo = SelectedObjects[0];
	check(ObjectToConsolidateTo);

	UObject* Outer = ObjectToConsolidateTo->GetOuter();

	if (nullptr == Outer || !Outer->IsA(UPackage::StaticClass()))
	{
		UE_LOGF(LogDataprep, Warning, "Consolidate failed: the object %ls is not an asset", *ObjectToConsolidateTo->GetName());
		return;
	}

	const UClass* ComparisonClass = ObjectToConsolidateTo->GetClass();
	check(ComparisonClass);

	TArray<UObject*> OutCompatibleObjects;

	// Iterate over each proposed consolidation object, checking if each shares a common class with the consolidation objects, or at least, a common base that
	// is allowed as an exception (currently only exceptions made for textures and materials).
	for (int32 ObjectIndex = 1; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		UObject* CurProposedObj = SelectedObjects[ObjectIndex];
		check(CurProposedObj);

		// You may not consolidate object redirectors
		if (CurProposedObj->GetClass()->IsChildOf(UObjectRedirector::StaticClass()))
		{
			continue;
		}

		if (CurProposedObj->GetClass() != ComparisonClass)
		{
			const UClass* NearestCommonBase = CurProposedObj->FindNearestCommonBaseClass(ComparisonClass);

			// If the proposed object doesn't share a common class or a common base that is allowed as an exception, it is not a compatible object
			if (!(NearestCommonBase->IsChildOf(UTexture::StaticClass())) && !(NearestCommonBase->IsChildOf(UMaterialInterface::StaticClass())))
			{
				continue;
			}
		}

		// If execution has gotten this far, the current proposed object is compatible
		OutCompatibleObjects.Add(CurProposedObj);
	}

	// Sort assets according to their dependency
	// Texture first, then MaterialFunction, then ...
	auto GetAssetClassRank = [&](const UClass* AssetClass) -> int8
	{
		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			return 0;
		}
		else if (AssetClass->IsChildOf(UMaterialFunction::StaticClass()))
		{
			return 1;
		}
		else if (AssetClass->IsChildOf(UMaterialFunctionInstance::StaticClass()))
		{
			return 2;
		}
		else if (AssetClass->IsChildOf(UMaterial::StaticClass()))
		{
			return 3;
		}
		else if (AssetClass->IsChildOf(UMaterialInstance::StaticClass()))
		{
			return 4;
		}
		else if (AssetClass->IsChildOf(UStaticMesh::StaticClass()))
		{
			return 5;
		}

		return 6;
	};

	Algo::Sort(OutCompatibleObjects, [&](const UObject* A, const UObject* B)
	{
		int8 AValue = A ? GetAssetClassRank(A->GetClass()) : 7;
		int8 BValue = B ? GetAssetClassRank(B->GetClass()) : 7;
		return AValue > BValue;
	});

	// ObjectTools::ConsolidateObjects is creating undesired Redirectors
	// Collect existing redirectors to identify the newly created ones
	TSet<UObject*> ExistingRedirectors;
	for (TObjectIterator<UObjectRedirector> Itr; Itr; ++Itr)
	{
		ExistingRedirectors.Add(*Itr);
	}	

	// Perform the object consolidation
	ObjectTools::ConsolidateObjects(ObjectToConsolidateTo, OutCompatibleObjects, false);

	// Delete UObjectRedirector objects created by ObjectTools::ConsolidateObjects
	TArray<UObject*> RedirectorsToDelete;
	for (TObjectIterator<UObjectRedirector> Itr; Itr; ++Itr)
	{
		if (!ExistingRedirectors.Contains(*Itr))
		{
			FDataprepCoreUtils::MoveToTransientPackage(*Itr);
			RedirectorsToDelete.Add(*Itr);
		}
	}

	if (RedirectorsToDelete.Num() > 0)
	{
		FDataprepCoreUtils::PurgeObjects(RedirectorsToDelete);
	}
}

void UDataprepOperationsLibrary::RandomizeTransform(const TArray<UObject*>& SelectedObjects, ERandomizeTransformType TransformType, ERandomizeTransformReferenceFrame ReferenceFrame, const FVector& Min, const FVector& Max)
{
	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			if (!Actor->GetRootComponent())
			{
				continue;
			}

			// Generate random offset for X/Y/Z and apply depending on selected transform component
			const FVector Offset(FMath::RandRange(Min.X, Max.X),
								 FMath::RandRange(Min.Y, Max.Y),
								 FMath::RandRange(Min.Z, Max.Z));

			USceneComponent* RootComponent = Actor->GetRootComponent();

			switch (TransformType)
			{
				case ERandomizeTransformType::Rotation:
				{
					const FRotator OffsetRotation = FRotator::MakeFromEuler(Offset);
					if (ReferenceFrame == ERandomizeTransformReferenceFrame::World)
					{
						RootComponent->SetWorldRotation(RootComponent->GetComponentRotation() + OffsetRotation);
					}
					else
					{
						RootComponent->SetRelativeRotation(RootComponent->GetRelativeRotation() + OffsetRotation);
					}
					break;
				}
				case ERandomizeTransformType::Scale:
				{
					if (ReferenceFrame == ERandomizeTransformReferenceFrame::World)
					{
						RootComponent->SetWorldScale3D(RootComponent->GetComponentScale() + Offset);
					}
					else
					{
						RootComponent->SetRelativeScale3D(RootComponent->GetRelativeScale3D() + Offset);
					}
					break;
				}
				case ERandomizeTransformType::Location:
				{
					if (ReferenceFrame == ERandomizeTransformReferenceFrame::World)
					{
						RootComponent->SetWorldLocation(RootComponent->GetComponentLocation() + Offset);
					}
					else
					{
						RootComponent->SetRelativeLocation(RootComponent->GetRelativeLocation() + Offset);
					}
					break;
				}
			}
		}
	}
}

void UDataprepOperationsLibrary::FlipFaces(const TSet< UStaticMesh* >& StaticMeshes)
{
	for (UStaticMesh* StaticMesh : StaticMeshes)
	{
		if (nullptr == StaticMesh || !StaticMesh->IsMeshDescriptionValid(0))
		{
			continue;
		}

		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bMarkPackageDirty = false;
		Params.bUseHashAsGuid = true;

		FStaticMeshOperations::FlipPolygons(*MeshDescription);
		StaticMesh->CommitMeshDescription(0, Params);
	}
}

void UDataprepOperationsLibrary::SetSubOuputLevel(const TArray<UObject*>& SelectedObjects, const FString& SubLevelName)
{
	if(SubLevelName.IsEmpty())
	{
		return;
	}

	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				if ( RootComponent->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()) )
				{
					if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( RootComponent ) )
					{
						UDataprepConsumerUserData* DataprepContentUserData = AssetUserDataInterface->GetAssetUserData< UDataprepConsumerUserData >();

						if ( !DataprepContentUserData )
						{
							EObjectFlags Flags = RF_Public;
							DataprepContentUserData = NewObject< UDataprepConsumerUserData >( RootComponent, NAME_None, Flags );
							AssetUserDataInterface->AddAssetUserData( DataprepContentUserData );
						}

						DataprepContentUserData->AddMarker(UDataprepContentConsumer::RelativeOutput, SubLevelName);
					}
				}
			}
		}
	}
}

void UDataprepOperationsLibrary::SetSubOuputFolder(const TArray<UObject*>& SelectedObjects, const FString& SubFolderName)
{
	if(SubFolderName.IsEmpty())
	{
		return;
	}

	for (UObject* Object : SelectedObjects)
	{
		const bool bValidObject = Object->HasAnyFlags(RF_Public)
		&& IsValid(Object)
		&& Object->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass());

		if (bValidObject)
		{
			if ( IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >( Object ) )
			{
				UDataprepConsumerUserData* DataprepContentUserData = AssetUserDataInterface->GetAssetUserData< UDataprepConsumerUserData >();

				if ( !DataprepContentUserData )
				{
					EObjectFlags Flags = RF_Public;
					DataprepContentUserData = NewObject< UDataprepConsumerUserData >( Object, NAME_None, Flags );
					AssetUserDataInterface->AddAssetUserData( DataprepContentUserData );
				}

				DataprepContentUserData->AddMarker(UDataprepContentConsumer::RelativeOutput, SubFolderName);
			}
		}
	}
}

void UDataprepOperationsLibrary::AddToLayer(const TArray<UObject*>& SelectedObjects, const FName& LayerName)
{
	if (LayerName == NAME_None)
	{
		return;
	}

	for (UObject* Object : SelectedObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			if (IsValid(Actor))
			{
				Actor->Layers.AddUnique(LayerName);
			}
		}
	}
}

void UDataprepOperationsLibrary::SetCollisionComplexity(const TArray<UObject*>& InSelectedObjects, const ECollisionTraceFlag InCollisionTraceFlag, TArray<UObject*>& InModifiedObjects)
{
	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(InSelectedObjects);

	DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder( SelectedMeshes );

	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (StaticMesh)
		{
			DataprepOperationsLibraryUtil::FScopedStaticMeshEdit StaticMeshEdit( StaticMesh );

			if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
			{
				BodySetup->CollisionTraceFlag = InCollisionTraceFlag;
				InModifiedObjects.Add( StaticMesh );
			}
		}
	}
}

void UDataprepOperationsLibrary::ResizeTextures(const TArray<UTexture2D*>& InTextures, int32 InMaxSize)
{
	static const FName MaxTextureSizeName = GET_MEMBER_NAME_CHECKED(UTexture, MaxTextureSize);
	FProperty* MaxTextureSizeProperty = FindFProperty<FProperty>( UTexture::StaticClass(), MaxTextureSizeName );
	FPropertyChangedEvent PropertyChangedEvent(MaxTextureSizeProperty);

	for (UTexture2D* Texture : InTextures)
	{
		Texture->PreEditChange(MaxTextureSizeProperty);

		const int32 TextureWidth = Texture->GetSizeX();
		const int32 TextureHeight = Texture->GetSizeY();
		if (!FMath::IsPowerOfTwo(TextureWidth) || !FMath::IsPowerOfTwo(TextureHeight))
		{
			// Need to specify power of two mode for non-pot textures
			Texture->PowerOfTwoMode = ETexturePowerOfTwoSetting::PadToPowerOfTwo;
		}
		Texture->MaxTextureSize = InMaxSize;
		Texture->PostEditChangeProperty(PropertyChangedEvent);
	}
}

void UDataprepOperationsLibrary::SetNaniteSettings(const TArray<UObject*>& SelectedObjects, bool bEnabled, int32 PositionPrecision, float PercentTriangles, TArray<UObject*>& ModifiedObjects)
{
#if WITH_EDITORONLY_DATA
	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(SelectedObjects);

	FMeshNaniteSettings NewSettings;
	NewSettings.bEnabled = bEnabled;
	NewSettings.PositionPrecision = PositionPrecision;
	NewSettings.FallbackPercentTriangles = FMath::Clamp(PercentTriangles, 0.f, 1.f);

	// Apply Nanite settings but do not commit changes
	TArray<UStaticMesh*> ModifiedMeshes;
	ModifiedMeshes.Reserve(SelectedMeshes.Num());

	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (StaticMesh && StaticMesh->GetNaniteSettings() != NewSettings)
		{
			StaticMesh->SetNaniteSettings(NewSettings);
			ModifiedMeshes.Add(StaticMesh);
		}
	}

	ModifiedObjects.Append(ModifiedMeshes);
#endif // #if WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE
