// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "EditorFilterLibrary.h"
#include "Engine/DataTable.h"

#include "UObject/Package.h"
#include "DataprepOperationsLibrary.generated.h"

class UTexture2D;
enum ECollisionTraceFlag : int;
enum class EScriptCollisionShapeType : uint8;
namespace EComponentMobility { enum Type : int; }
struct FStaticMeshReductionOptions;

class AActor;
class IMeshBuilderModule;
class UMaterialInterface;
class UStaticMesh;

DECLARE_LOG_CATEGORY_EXTERN( LogDataprep, Log, All );

/*
 * Simple struct for the table row used for UDataprepOperationsLibrary::SubstituteMaterials
 */
USTRUCT(BlueprintType)
struct FMaterialSubstitutionDataTable : public FTableRowBase
{
	GENERATED_BODY()

	/** Name of the material(s) to search for. Wildcard is supported */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MaterialSubstitutionTable")
	FString SearchString;

	/** Type of matching to perform with SearchString string */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MaterialSubstitutionTable")
	EEditorScriptingStringMatchType StringMatch = EEditorScriptingStringMatchType::Contains;

	/** Material to use for the substitution */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MaterialSubstitutionTable")
	TObjectPtr<UMaterialInterface> MaterialReplacement = nullptr;
};

/*
* DEPRECATED - Simple struct for the table row used for UDataprepOperationsLibrary::SubstituteMaterials
*/
USTRUCT(meta=(Deprecated = "4.25.0"))
struct FMeshSubstitutionDataTable
{
	GENERATED_BODY()

	/** DEPRECATED - Name of the mesh(es) to search for. Wildcard is supported */
	UPROPERTY()
	FString SearchString_DEPRECATED;

	/** DEPRECATED - Type of matching to perform with SearchString string */
	UPROPERTY()
	EEditorScriptingStringMatchType StringMatch_DEPRECATED = EEditorScriptingStringMatchType::Contains;

	/** DEPRECATED - Mesh to use for the substitution */
	UPROPERTY()
	TObjectPtr<UStaticMesh> MeshReplacement_DEPRECATED = nullptr;
};

/*
 * Simple struct to set up LODGroup name on static meshes
 * This is for internal purpose only to allow users to select the name of the LODGroup
 * to apply on static meshes in UDataprepOperationsLibrary::SetLODGroup
 */
USTRUCT(BlueprintInternalUseOnly)
struct DATAPREPLIBRARIES_API FLODGroupName
{
	GENERATED_BODY()

	/** Value of the name of LODGroup not the display name */
	UPROPERTY()
	FString Value;
};

/*
 * Simple struct to set up LODGroup name on static meshes
 * This is for internal purpose only to allow users to select the name of the LODGroup
 * to apply on static meshes in UDataprepOperationsLibrary::SetLODGroup
 */
USTRUCT(BlueprintInternalUseOnly)
struct DATAPREPLIBRARIES_API FMeshReductionOptions
{
	GENERATED_BODY()

	/** Value of the name of LODGroup not the display name */
	UPROPERTY()
	float ReductionPercent = 0.0f;

	/** Value of the name of LODGroup not the display name */
	UPROPERTY()
	float ScreenSize= 0.0f;
};

/*
 * Simple struct to set up LODGroup name on static meshes
 * This is for internal purpose only to allow users to select the name of the LODGroup
 * to apply on static meshes in UDataprepOperationsLibrary::SetLODGroup
 */
USTRUCT(BlueprintInternalUseOnly)
struct DATAPREPLIBRARIES_API FMeshReductionArray
{
	GENERATED_BODY()

	/** Value of the name of LODGroup not the display name */
	UPROPERTY()
	TArray<FMeshReductionOptions> Array;
};

UENUM(BlueprintInternalUseOnly)
enum class ERandomizeTransformType : uint8
{
	Rotation,
	Location,
	Scale
};

UENUM(BlueprintInternalUseOnly)
enum class ERandomizeTransformReferenceFrame : uint8
{
	World,
	Relative
};

UCLASS()
class DATAPREPLIBRARIES_API UDataprepOperationsLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Generate LODs on the static meshes contained in the input array
	 * by the actors contained in the input array
	 * @param	SelectedObjects			Array of UObjects to process.
	 * @param	ReductionOptions		Options on how to generate LODs on the mesh.
	 * @remark: Static meshes are not re-built after the new LODs are set
	 * Generates an array of unique static meshes from the input array either by a cast if
	 * the UObject is a UStaticMesh or collecting the static meshes referred to if the UObject
	 * is a AActor
	 * Calls UEditorStaticMeshLibrary::SetLods on each static mesh of the resulting array.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetLods(const TArray<UObject*>& SelectedObjects, const FStaticMeshReductionOptions& ReductionOptions, TArray<UObject*>& ModifiedObjects);

	/**
	 * Set one simple collision of the given shape type on the static meshes contained in the
	 * input array or referred to by the actors contained in the input array
	 * @param	SelectedActors			Array of actors to process.
	 * @param	ShapeType				Options on which simple collision to add to the mesh.
	 * @remark: Static meshes are not re-built after the new collision settings are set
	 * Generates an array of unique static meshes from the input array either by a cast if
	 * the UObject is a UStaticMesh or collecting the static meshes referred to if the UObject
	 * is a AActor
	 * Calls UEditorStaticMeshLibrary::RemoveCollisions to remove any existing collisions
	 * on each static mesh of the resulting array
	 * Calls UEditorStaticMeshLibrary::AddSimpleCollisions on each static mesh of the resulting array.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetSimpleCollision(const TArray<UObject*>& SelectedObjects, const EScriptCollisionShapeType ShapeType, TArray<UObject*>& ModifiedObjects);

	/**
	 * Add complex collision on the static meshes contained in the input array
	 * by the actors contained in the input array
	 * @param	SelectedActors			Array of actors to process.
	 * @param	HullCount				Maximum number of convex pieces that will be created. Must be positive.
	 * @param	MaxHullVerts			Maximum number of vertices allowed for any generated convex hull.
	 * @param	HullPrecision			Number of voxels to use when generating collision. Must be positive.
	 * @remark: Static meshes are not re-built after the new collision settings are set
	 * Generates an array of unique static meshes from the input array either by a cast if
	 * the UObject is a UStaticMesh or collecting the static meshes referred to if the UObject
	 * is a AActor
	 * Calls UEditorStaticMeshLibrary::SetConvexDecompositionCollisions on each static mesh of the resulting array.
	 * Note that any simple collisions on each static mesh of the resulting array will be removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetConvexDecompositionCollision(const TArray<UObject*>& SelectedObjects, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, TArray<UObject*>& ModifiedObjects);

	/**
	 * Replaces designated materials in all or specific content folders with specific ones
	 * @param SelectedObjects: Objects to consider for the substitution
	 * @param MaterialSearch: Name of the material(s) to search for. Wildcard is supported
	 * @param StringMatch: Type of matching to perform with MaterialSearch string
	 * @param MaterialSubstitute: Material to use for the substitution
	 * @remark: A material override will be added to static mesh components if their attached
	 *			static mesh uses the searched material but not themselves
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SubstituteMaterial(const TArray<UObject*>& SelectedObjects, const FString& MaterialSearch, EEditorScriptingStringMatchType StringMatch, UMaterialInterface* MaterialSubstitute);

	/**
	 * Replaces designated materials in all or specific content folders with requested ones
	 * @param SelectedObjects: Objects to consider for the substitution
	 * @param DataTable: Data table to use for the substitution
	 * @remark: SubstituteMaterial is called for each entry of the input data table
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SubstituteMaterialsByTable(const TArray<UObject*>& SelectedObjects, const UDataTable* DataTable);

	/**
	 * Remove inputs content
	 * @param Objects Objects to remove
	 * @remark: Static meshes are not re-built after the new LOD groups are set
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetLODGroup( const TArray< UObject* >& SelectedObjects, FName& LODGroupName, TArray<UObject*>& ModifiedObjects );

	/**
	 * Set the material to all elements of a set of Static Meshes or Static Mesh Actors
	 * @param SelectedObjects	Objects to set the input material on
	 * @param MaterialInterface Material to use
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetMaterial( const TArray< UObject* >& SelectedObjects, UMaterialInterface* MaterialSubstitute );

	/**
	 * Set mobility on a set of static mesh actors
	 * @param SelectedObjects Objects to set mobility on
	 * @param MobilityType Type of mobility to set on selected mesh actors
	 * @remark: Only objects of class AStaticMeshActor will be considered
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetMobility( const TArray< UObject* >& SelectedObjects, EComponentMobility::Type MobilityType );

	/**
	 * Set the mesh to all elements of a set of Actors containing StaticMeshComponents
	 * @param SelectedObjects	Objects to set the input mesh on
	 * @param MeshSubstitute	Mesh to use
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetMesh( const TArray< UObject* >& SelectedObjects, UStaticMesh* MeshSubstitute );

	/**
	 * Replaces designated meshes in all or specific content folders with specific ones
	 * @param SelectedObjects:	Objects to consider for the substitution
	 * @param MeshSearch:		Name of the mesh(es) to search for. Wildcard is supported
	 * @param StringMatch:		Type of matching to perform with MeshSearch string
	 * @param MeshSubstitute:	Mesh to use for the substitution
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SubstituteMesh(const TArray<UObject*>& SelectedObjects, const FString& MeshSearch, EEditorScriptingStringMatchType StringMatch, UStaticMesh* MeshSubstitute);

	/**
	 * Replaces designated meshes in all or specific content folders with requested ones
	 * @param SelectedObjects:	Objects to consider for the substitution
	 * @param DataTable:		Data table to use for the substitution
	 * @remark: SubstituteMesh is called for each entry of the input data table
	 */
	UFUNCTION(meta=(Deprecated = "4.25.0"))
	static void SubstituteMeshesByTable(const TArray<UObject*>& SelectedObjects, const UDataTable* DataTable);

	/**
	 * Add tags to a set of actors
	 * @param SelectedObjects	Objects to add the tags to
	 * @param Tags				Array of tags to add
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void AddTags( const TArray< UObject* >& SelectedObjects, const TArray<FName>& InTags );

	/**
	 * Adds metadata to selected objects that implement the UInterface_AssetUserData interface.
	 * @param SelectedObjects:	Objects to consider
	 * @param InMetadata:		The metadata to append
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void AddMetadata(const TArray<UObject*>& SelectedObjects, const TMap<FName, FString>& InMetadata);

	/**
	 * Replace all references to the assets in the array, except the first, with the first asset of the array.
	 * @param SelectedObjects	Objects to consolidate
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void ConsolidateObjects( const TArray< UObject* >& SelectedObjects );

	/**
	 * Alters transform of selected objects by appling randomly generated offset to one of the transform components (rotation, scale or translation)
	 * @param SelectedObjects:	Objects to consider
	 * @param TransformType:	Selected transform component to randomize
	 * @param Min:				Start of randomization range
	 * @param Max:				End of randomization range
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void RandomizeTransform(const TArray<UObject*>& SelectedObjects, ERandomizeTransformType TransformType, ERandomizeTransformReferenceFrame ReferenceFrame, const FVector& Min, const FVector& Max);

	/**
	 * Flip the faces of all elements of a set of Static Meshes or Static Mesh Actors
	 * @param SelectedObjects	Objects to the flip the faces of
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void FlipFaces( const TSet< UStaticMesh* >& StaticMeshes );

	/**
	 * Add/Edit UDataprepConsumerUserData with the requested name for the sub-level
	 * @param SelectedObjects:	Objects to consider
	 * @param SubLevelName:	Name of the sub-level
	 * @note - This operation only applies on actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetSubOuputLevel(const TArray<UObject*>& SelectedObjects, const FString& SubLevelName);

	/**
	 * Add/Edit UDataprepConsumerUserData with the requested name for the sub-folder
	 * @param SelectedObjects:	Objects to consider
	 * @param SubFolderName:	Name of the sub-folder
	 * @note - This operation only applies on assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetSubOuputFolder(const TArray<UObject*>& SelectedObjects, const FString& SubFolderName);

	/**
	 * Add all Actors to a given layer.
	 * @param SelectedObjects:	Objects to consider
	 * @param LayerName:	Name of the sub-folder
	 * @note - This operation only applies on assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void AddToLayer(const TArray<UObject*>& SelectedObjects, const FName& LayerName);

	/**
	 * Set collision complexity for selected meshes
	 * @param	InSelectedObjects			Array of meshes to process.
	 * @param	InCollisionTraceFlag		The new collision complexity.
	 * @param	InModifiedObjects			List of modified meshes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetCollisionComplexity(const TArray<UObject*>& InSelectedObjects, const ECollisionTraceFlag InCollisionTraceFlag, TArray<UObject*>& InModifiedObjects);

	/**
	 * Resize textures to max width/height and optionally ensure power of two size.
	 * @param InTextures:	Textures to resize
	 * @param InMaxSize:	Max allowed width or height
	 * @note - This operation only applies on assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void ResizeTextures(const TArray<UTexture2D*>& InTextures, int32 InMaxSize);

	/**
	 * Set Nanite settings for selected meshes
	 * @param	InSelectedObjects			Array of objects to process.
	 * @param	bInEnabled					If true, Nanite data will be generated.
	 * @param	InPositionPrecision			Step size is 2^(-PositionPrecision) cm. MIN_int32 is auto.
	 * @param	InPercentTriangles			Percentage of triangles to keep from LOD0. 1.0 = no reduction, 0.0 = no triangles.
	 * @param	OutModifiedObjects			List of modified objects the processed meshes will be added to
	 */
	UFUNCTION(BlueprintCallable, Category = "Dataprep | Operation")
	static void SetNaniteSettings(const TArray<UObject*>& InSelectedObjects, bool bInEnabled, int32 InPositionPrecision, float InPercentTriangles, TArray<UObject*>& OutModifiedObjects);

private:
	static void SubstituteMaterial(const TArray<UObject*>& SelectedObjects, const FString& MaterialSearch, EEditorScriptingStringMatchType StringMatch, const TArray<UMaterialInterface*>& MaterialList, UMaterialInterface* MaterialSubstitute);
	static void SubstituteMesh(const TArray<UObject*>& SelectedObjects, const FString& MeshSearch, EEditorScriptingStringMatchType StringMatch, const TArray<UStaticMesh*>& MeshList, UStaticMesh* MeshSubstitute);
};
