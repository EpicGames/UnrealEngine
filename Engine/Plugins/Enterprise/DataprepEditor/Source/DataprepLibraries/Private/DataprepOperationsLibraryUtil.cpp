// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepOperationsLibraryUtil.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "Materials/MaterialInterface.h"
#include "EngineLogs.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"

namespace DataprepOperationsLibraryUtil
{
	TSet<UStaticMesh*> GetSelectedMeshes(const TArray<AActor*>& SelectedActors)
	{
		TSet<UStaticMesh*> SelectedMeshes;

		for (AActor* Actor : SelectedActors)
		{
			if (Actor)
			{
				TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
				for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
				{
					if(UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
					{
						SelectedMeshes.Add( StaticMesh );
					}
				}
			}
		}

		return SelectedMeshes;
	}

	TSet<UStaticMesh*> GetSelectedMeshes(const TArray<UObject*>& SelectedObjects)
	{
		TSet<UStaticMesh*> SelectedMeshes;

		for (UObject* Object : SelectedObjects)
		{
			if ( UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object) )
			{
				SelectedMeshes.Add( StaticMesh );
			}
			else if ( Object->IsA(UStaticMeshComponent::StaticClass()) )
			{
				if ((StaticMesh = Cast<UStaticMeshComponent>(Object)->GetStaticMesh()) != nullptr )
				{
					SelectedMeshes.Add(StaticMesh);
				}
			}
			else if (AActor* Actor = Cast<AActor>(Object) )
			{
				TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents( Actor );
				for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
				{
					if((StaticMesh = StaticMeshComponent->GetStaticMesh()) != nullptr)
					{
						SelectedMeshes.Add( StaticMesh );
					}
				}
			}
		}

		return SelectedMeshes;
	}

	TArray<UMaterialInterface*> GetUsedMaterials(const TArray<UObject*>& SelectedObjects)
	{
		TSet<UMaterialInterface*> MaterialSet;

		for (UObject* Object : SelectedObjects)
		{
			if (AActor* Actor = Cast< AActor >(Object))
			{
				// Find the materials by iterating over every mesh component.
				TInlineComponentArray<UMeshComponent*> MeshComponents(Actor);
				for (UMeshComponent* MeshComponent : MeshComponents)
				{
					int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

					for (int32 Index = 0; Index < MaterialCount; ++Index)
					{
						MaterialSet.Add(MeshComponent->GetMaterial(Index));
					}
				}
			}
			else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
			{
				for (int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
				{
					MaterialSet.Add(StaticMesh->GetMaterial(Index));
				}
			}
			else if (UMeshComponent* MeshComponent = Cast< UMeshComponent >(Object))
			{
				int32 MaterialCount = FMath::Max( MeshComponent->GetNumOverrideMaterials(), MeshComponent->GetNumMaterials() );

				for (int32 Index = 0; Index < MaterialCount; ++Index)
				{
					if (UMaterialInterface* Material = MeshComponent->GetMaterial(Index))
					{
						MaterialSet.Add(Material);
					}
				}
			}
		}

		return MaterialSet.Array();
	}

	TArray<UStaticMesh*> GetUsedMeshes(const TArray<UObject*>& SelectedObjects)
	{
		TSet<UStaticMesh*> MeshesSet;

		for (UObject* Object : SelectedObjects)
		{
			if (AActor* Actor = Cast< AActor >(Object))
			{
				// Find the meshes by iterating over every mesh component.
				TInlineComponentArray<UStaticMeshComponent*> MeshComponents(Actor);
				for (UStaticMeshComponent* MeshComponent : MeshComponents)
				{
					if(MeshComponent && MeshComponent->GetStaticMesh())
					{
						MeshesSet.Add( MeshComponent->GetStaticMesh() );
					}
				}
			}
		}

		return MeshesSet.Array();
	}

	FScopedStaticMeshEdit::FScopedStaticMeshEdit( UStaticMesh* InStaticMesh )
		: StaticMesh( InStaticMesh )
	{
		BuildSettingsBackup = PreventStaticMeshBuild( StaticMesh );
	}

	FScopedStaticMeshEdit::~FScopedStaticMeshEdit()
	{
		RestoreStaticMeshBuild( StaticMesh, MoveTemp( BuildSettingsBackup ) );
	}

	TArray< FMeshBuildSettings > FScopedStaticMeshEdit::PreventStaticMeshBuild( UStaticMesh* StaticMesh )
	{
		if ( !StaticMesh )
		{
			return {};
		}

		TArray< FMeshBuildSettings > BuildSettingsBackup;

		int32 NumSourceModels = StaticMesh->GetNumSourceModels();
		for (int32 LodIndex = 0; LodIndex < NumSourceModels; LodIndex++)
		{
			FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LodIndex);
			BuildSettingsBackup.Add( SourceModel.BuildSettings );

			// These were done in the PreBuild step
			SourceModel.BuildSettings.bGenerateLightmapUVs = false;
			SourceModel.BuildSettings.bRecomputeNormals = false;
			SourceModel.BuildSettings.bRecomputeTangents = false;
			SourceModel.BuildSettings.bBuildReversedIndexBuffer = false;
			SourceModel.BuildSettings.bComputeWeightedNormals = false;
		}

		return BuildSettingsBackup;
	}

	void FScopedStaticMeshEdit::RestoreStaticMeshBuild( UStaticMesh* StaticMesh, const TArray< FMeshBuildSettings >& BuildSettingsBackup )
	{
		if ( !StaticMesh )
		{
			return;
		}

		// Restore StaticMesh's build settings
		for ( int32 LODIndex = 0; LODIndex < BuildSettingsBackup.Num() ; ++LODIndex )
		{
			// Update only LODs which were cached
			if (StaticMesh->IsSourceModelValid( LODIndex ))
			{
				const FMeshBuildSettings& CachedBuildSettings = BuildSettingsBackup[ LODIndex ];
				FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;

				// Restore only the properties which were modified
				BuildSettings.bGenerateLightmapUVs = CachedBuildSettings.bGenerateLightmapUVs;
				BuildSettings.bRecomputeNormals = CachedBuildSettings.bRecomputeNormals;
				BuildSettings.bRecomputeTangents = CachedBuildSettings.bRecomputeTangents;
				BuildSettings.bBuildReversedIndexBuffer = CachedBuildSettings.bBuildReversedIndexBuffer;
				BuildSettings.bComputeWeightedNormals = CachedBuildSettings.bComputeWeightedNormals;
			}
		}
	}

	/** Customized version of UStaticMesh::SetMaterial avoiding the triggering of UStaticMesh::Build and its side-effects */
	void SetMaterial( UStaticMesh* StaticMesh, int32 MaterialIndex, UMaterialInterface* NewMaterial )
	{
		if( StaticMesh->GetStaticMaterials().IsValidIndex( MaterialIndex ) )
		{
			FStaticMaterial& StaticMaterial = StaticMesh->GetStaticMaterials()[ MaterialIndex ];
			StaticMaterial.MaterialInterface = NewMaterial;
			if( NewMaterial != nullptr )
			{
				if ( StaticMaterial.MaterialSlotName == NAME_None )
				{
					StaticMaterial.MaterialSlotName = NewMaterial->GetFName();
				}
			}
		}
	}

	FStaticMeshBuilder::FStaticMeshBuilder(const TSet<UStaticMesh *>& InStaticMeshes)
	{
		StaticMeshes = BuildStaticMeshes( InStaticMeshes );
	}

	FStaticMeshBuilder::~FStaticMeshBuilder()
	{
		// Release render data of built static meshes
		for(UStaticMesh* StaticMesh : StaticMeshes)
		{
			if(StaticMesh)
			{
				StaticMesh->SetRenderData(nullptr);
			}
		}
	}

	TArray<UStaticMesh*> BuildStaticMeshes(const TSet<UStaticMesh*>& StaticMeshes, bool bForceBuild)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DataprepOperationsLibraryUtil::BuildStaticMeshes);

		TArray<UStaticMesh*> BuiltMeshes;
		BuiltMeshes.Reserve( StaticMeshes.Num() );

		if(bForceBuild)
		{
			BuiltMeshes.Append( StaticMeshes.Array() );
		}
		else
		{
			for(UStaticMesh* StaticMesh : StaticMeshes)
			{
				if(StaticMesh && (!StaticMesh->GetRenderData() || !StaticMesh->GetRenderData()->IsInitialized()))
				{
					BuiltMeshes.Add( StaticMesh );
				}
			}
		}

		if(BuiltMeshes.Num() > 0)
		{
			// Start with the biggest mesh first to help balancing tasks on threads
			BuiltMeshes.Sort(
				[](const UStaticMesh& Lhs, const UStaticMesh& Rhs) 
			{ 
				int32 LhsVerticesNum = Lhs.IsMeshDescriptionValid(0) ? Lhs.GetMeshDescription(0)->Vertices().Num() : 0;
				int32 RhsVerticesNum = Rhs.IsMeshDescriptionValid(0) ? Rhs.GetMeshDescription(0)->Vertices().Num() : 0;

				return LhsVerticesNum > RhsVerticesNum;
			}
			);

			//Cache the BuildSettings and update them before building the meshes.
			TArray< TArray<FMeshBuildSettings> > StaticMeshesSettings;
			StaticMeshesSettings.Reserve( BuiltMeshes.Num() );

			for (UStaticMesh* StaticMesh : BuiltMeshes)
			{
				int32 NumSourceModels = StaticMesh->GetNumSourceModels();
				TArray<FMeshBuildSettings> BuildSettings;
				BuildSettings.Reserve(NumSourceModels);

				for(int32 Index = 0; Index < NumSourceModels; ++Index)
				{
					FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(Index);

					BuildSettings.Add( SourceModel.BuildSettings );

					if(FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(Index))
					{
						FStaticMeshAttributes Attributes(*MeshDescription);
						if(SourceModel.BuildSettings.DstLightmapIndex != -1)
						{
							TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
							SourceModel.BuildSettings.bGenerateLightmapUVs = VertexInstanceUVs.IsValid() && VertexInstanceUVs.GetNumChannels() > SourceModel.BuildSettings.DstLightmapIndex;
						}
						else
						{
							SourceModel.BuildSettings.bGenerateLightmapUVs = false;
						}

						SourceModel.BuildSettings.bRecomputeNormals = !(Attributes.GetVertexInstanceNormals().IsValid() && Attributes.GetVertexInstanceNormals().GetNumChannels() > 0);
						SourceModel.BuildSettings.bRecomputeTangents = false;
						//SourceModel.BuildSettings.bBuildReversedIndexBuffer = false;
					}
				}

				StaticMeshesSettings.Add(MoveTemp(BuildSettings));				
			}

			// Disable warnings from LogStaticMesh. Not useful
			ELogVerbosity::Type PrevLogStaticMeshVerbosity = LogStaticMesh.GetVerbosity();
			LogStaticMesh.SetVerbosity( ELogVerbosity::Error );

			UStaticMesh::BatchBuild(BuiltMeshes, true );

			// Restore LogStaticMesh verbosity
			LogStaticMesh.SetVerbosity( PrevLogStaticMeshVerbosity );

			for(int32 Index = 0; Index < BuiltMeshes.Num(); ++Index)
			{
				UStaticMesh* StaticMesh = BuiltMeshes[Index];
				TArray<FMeshBuildSettings>& PrevBuildSettings = StaticMeshesSettings[Index];

				int32 NumSourceModels = StaticMesh->GetNumSourceModels();
				for(int32 SourceModelIndex = 0; SourceModelIndex < NumSourceModels; ++SourceModelIndex)
				{
					StaticMesh->GetSourceModel(SourceModelIndex).BuildSettings = PrevBuildSettings[SourceModelIndex];
				}

				for ( FStaticMeshLODResources& LODResources : StaticMesh->GetRenderData()->LODResources )
				{
					LODResources.bHasColorVertexData = true;
				}
			}
		}

		return BuiltMeshes;
	}
} // ns DataprepOperationsLibraryUtil
