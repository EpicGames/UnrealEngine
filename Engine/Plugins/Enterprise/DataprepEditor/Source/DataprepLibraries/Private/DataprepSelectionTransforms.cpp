// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepSelectionTransforms.h"
#include "Components/StaticMeshComponent.h"
#include "DataprepCorePrivateUtils.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DataprepSelectionTransforms"

namespace DataprepSelectionTransformsUtils
{
	template <typename T>
	TSet<T*> GetDataprepObjects()
	{
		TSet<T*> Objects;

		for (TObjectIterator<T> It; It; ++It)
		{
			if (const UPackage* Package = It->GetPackage())
			{
				const FString PackageName = Package->GetName();

				if (PackageName.StartsWith(DataprepCorePrivateUtils::GetRootPackagePath()))
				{
					Objects.Add(*It);
				}
			}
		}

		return MoveTemp(Objects);
	}
}

void UDataprepReferenceSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TSet<UObject*> Assets;

	TFunction<void(UMaterialInterface*)> AddMaterialReferenced = [&Assets](UMaterialInterface* InMaterial)
	{
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(InMaterial))
		{
			if (MaterialInstance->Parent)
			{
				Assets.Add(MaterialInstance->Parent);
			}
		}

		// Collect textures
		TArray<UTexture*> Textures;
		InMaterial->GetUsedTextures(Textures);
		for (UTexture* Texture : Textures)
		{
			Assets.Add(Texture);
		}
	};

	TFunction<void(UStaticMesh*)> AddMeshReferenced = [&Assets, &AddMaterialReferenced, this](UStaticMesh* InMesh)
	{
		for (FStaticMaterial& StaticMaterial : InMesh->GetStaticMaterials())
		{
			if (UMaterialInterface* MaterialInterface = StaticMaterial.MaterialInterface)
			{
				Assets.Add(MaterialInterface);

				if (bAllowIndirectReferences)
				{
					AddMaterialReferenced(MaterialInterface);
				}
			}
		}
	};

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || !IsValid(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >(Object))
		{
			TArray<UActorComponent*> Components = Actor->GetComponents().Array();
			Components.Append(Actor->GetInstanceComponents());

			for (UActorComponent* Component : Components)
			{
				if (UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(Component))
				{
					if (UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
					{
						Assets.Add(StaticMesh);

						if (bAllowIndirectReferences)
						{
							AddMeshReferenced(StaticMesh);
						}
					}

					for (UMaterialInterface* MaterialInterface : MeshComponent->OverrideMaterials)
					{
						if (MaterialInterface != nullptr)
						{
							Assets.Add(MaterialInterface);

							if (bAllowIndirectReferences)
							{
								AddMaterialReferenced(MaterialInterface);
							}
						}
					}
				}
			}
		}
		else if (UStaticMeshComponent* MeshComponent = Cast< UStaticMeshComponent >(Object))
		{
			if (UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh())
			{
				Assets.Add(StaticMesh);

				if (bAllowIndirectReferences)
				{
					AddMeshReferenced(StaticMesh);
				}
			}

			for (UMaterialInterface* MaterialInterface : MeshComponent->OverrideMaterials)
			{
				if (MaterialInterface != nullptr)
				{
					Assets.Add(MaterialInterface);

					if (bAllowIndirectReferences)
					{
						AddMaterialReferenced(MaterialInterface);
					}
				}
			}
		}
		else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
		{
			AddMeshReferenced(StaticMesh);

		}
		else if (UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >(Object))
		{
			AddMaterialReferenced(MaterialInterface);
		}

		if (bOutputCanIncludeInput)
		{
			Assets.Add(Object);
		}
	}

	OutObjects.Append(Assets.Array());
}

void UDataprepReferencedSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TSet<UObject*> Assets;

	TFunction<bool(const UMaterialInterface*, const UTexture*)> DoesMaterialUseTexture = [](const UMaterialInterface* Material, const UTexture* CheckTexture) -> bool
	{
		TArray<UTexture*> Textures;
		Material->GetUsedTextures(Textures);
		for (int32 i = 0; i < Textures.Num(); i++)
		{
			if (Textures[i] == CheckTexture)
			{
				return true;
			}
		}
		return false;
	};

	TSet<AActor*> Actors = DataprepSelectionTransformsUtils::GetDataprepObjects<AActor>();
	TSet<UStaticMesh*> Meshes = DataprepSelectionTransformsUtils::GetDataprepObjects<UStaticMesh>();
	TSet<UMaterialInterface*> Materials = DataprepSelectionTransformsUtils::GetDataprepObjects<UMaterialInterface>();

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || !IsValidChecked(Object))
		{
			continue;
		}

		if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
		{
			// Collect actors referencing this mesh
			for (AActor* Actor : Actors)
			{
				TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
				for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
				{
					UStaticMesh* ActorMesh = StaticMeshComponent->GetStaticMesh();
					if (ActorMesh == StaticMesh)
					{
						Assets.Add(Actor);
					}
				}
			}
		}
		else if (UMaterialInterface* MaterialInterface = Cast< UMaterialInterface >(Object))
		{
			// Collect actor components referencing this material (material overrides)
			for (AActor* Actor : Actors)
			{
				TInlineComponentArray<UMeshComponent*> MeshComponents(Actor);
				for (UMeshComponent* MeshComponent : MeshComponents)
				{
					for (UMaterialInterface* MeshComponentMaterialInterface : MeshComponent->OverrideMaterials)
					{
						if (MeshComponentMaterialInterface == MaterialInterface)
						{
							Assets.Add(Actor);
						}
					}
				}
			}

			// Collect meshes referencing this material
			for (UStaticMesh* Mesh : Meshes)
			{
				for (FStaticMaterial& StaticMaterial : Mesh->GetStaticMaterials())
				{
					UMaterialInterface* StaticMaterialInterface = StaticMaterial.MaterialInterface;
					if (StaticMaterialInterface == MaterialInterface)
					{
						Assets.Add(Mesh);
					}
				}
			}

			// Collect material instances referencing this material
			if (UMaterial* Material = Cast<UMaterial>(MaterialInterface))
			{
				for (UMaterialInterface* MatInterface : Materials)
				{
					if (MatInterface->GetMaterial() == Material)
					{
						Assets.Add(MatInterface);
					}
				}
			}
		}
		else if (UTexture* Texture = Cast< UTexture >(Object))
		{
			// Collect materials referencing this texture
			for (UMaterialInterface* MatInterface : Materials)
			{
				if (DoesMaterialUseTexture(MatInterface, Texture))
				{
					if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MatInterface))
					{
						Assets.Add(MaterialInstance);
						if (MaterialInstance->Parent)
						{
							Assets.Add(MaterialInstance->Parent);
						}
					}
					else if (UMaterial* Material = Cast<UMaterial>(MatInterface))
					{
						Assets.Add(Material);
					}
				}
			}
		}
	}

	OutObjects.Append(Assets.Array());
}

void UDataprepHierarchySelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TArray<UObject*> ObjectsToVisit;

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || !IsValid(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >(Object))
		{
			TArray<AActor*> Children;
			Actor->GetAttachedActors( Children );

			ObjectsToVisit.Append( Children );
		}
		else if (USceneComponent* SceneComponent = Cast< USceneComponent >(Object))
		{
			ObjectsToVisit.Append( SceneComponent->GetAttachChildren() );
		}
	}

	TSet<UObject*> NewSelection;

	while ( ObjectsToVisit.Num() > 0)
	{
		UObject* VisitedObject = ObjectsToVisit.Pop();

		if (VisitedObject == nullptr)
		{
			continue;
		}

		NewSelection.Add(VisitedObject);

		if(SelectionPolicy == EDataprepHierarchySelectionPolicy::AllDescendants)
		{
			// Continue with children
			if (AActor* Actor = Cast< AActor >(VisitedObject))
			{
				TArray<AActor*> Children;
				Actor->GetAttachedActors( Children );
				ObjectsToVisit.Append( Children );
			}
			else if (USceneComponent* SceneComponent = Cast< USceneComponent >(VisitedObject))
			{
				ObjectsToVisit.Append( SceneComponent->GetAttachChildren() );
			}
		}
	}

	OutObjects.Append(NewSelection.Array());

	if (bOutputCanIncludeInput)
	{
		OutObjects.Reserve( OutObjects.Num() + InObjects.Num());

		for (UObject* Object : InObjects)
		{
			if (!ensure(Object) || !IsValid(Object))
			{
				continue;
			}

			if (AActor* Actor = Cast< AActor >(Object))
			{
				OutObjects.Add(Object);
			}
			else if (USceneComponent* SceneComponent = Cast< USceneComponent >(Object))
			{
				OutObjects.Add(Object);
			}
		}
	}
}

void UDataprepActorComponentsSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TSet<UActorComponent*> NewSelection;

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || ! IsValidChecked(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >(Object))
		{
			const TSet< UActorComponent* >& Components = Actor->GetComponents();
			for (UActorComponent* Comp : Components)
			{
				NewSelection.Add( Comp );
			}
		}
		else if (UActorComponent* Component = Cast< UActorComponent >(Object))
		{
			if (bOutputCanIncludeInput)
			{
				NewSelection.Add( Component );
			}
		}
	}

	OutObjects.Append( NewSelection.Array() );
}

void UDataprepOwningActorSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TSet<AActor*> NewSelection;

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || !IsValidChecked(Object))
		{
			continue;
		}

		if (UActorComponent* Component = Cast< UActorComponent >(Object))
		{
			NewSelection.Add( Component->GetOwner() );
		}
	}

	OutObjects.Append( NewSelection.Array() );
}

#undef LOCTEXT_NAMESPACE
