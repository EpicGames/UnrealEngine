// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepSelectionTransform.h"


#include "DataprepSelectionTransforms.generated.h"

UENUM()
enum class EDataprepHierarchySelectionPolicy : uint8
{
	/** Select immediate children of the selected objects */
	ImmediateChildren,

	/** Select all descendants of the selected objects */
	AllDescendants,
};

UCLASS(Category = SelectionTransform, Meta = (DisplayName="Select Referenced", ToolTip = "Return assets directly used/referenced by the selected objects.") )
class UDataprepReferenceSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

	UDataprepReferenceSelectionTransform()
		: bAllowIndirectReferences(false)
	{}

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;

	UPROPERTY(EditAnywhere, Category = "HierarchySelectionOptions", meta = (ToolTip = "Include assets referenced/used by assets directly referenced/used by selected objects"))
	bool bAllowIndirectReferences;
};

UCLASS(Category = SelectionTransform, Meta = (DisplayName = "Select Referencers", ToolTip = "Return assets directly using/referencing the objects from previous filtering"))
class UDataprepReferencedSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;
};

UCLASS(Category = SelectionTransform, Meta = (DisplayName="Select Hierarchy", ToolTip = "Return immediate children or all the descendants of the selected objects") )
class UDataprepHierarchySelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

	UDataprepHierarchySelectionTransform()
		: SelectionPolicy(EDataprepHierarchySelectionPolicy::ImmediateChildren)
	{}

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;

	UPROPERTY(EditAnywhere, Category = "HierarchySelectionOptions", meta = (DisplayName = "Select", ToolTip = "Specify policy of hierarchical parsing of selected objects"))
	EDataprepHierarchySelectionPolicy SelectionPolicy;
};

UCLASS(Category = SelectionTransform, Meta = (DisplayName="Select Actor Components", ToolTip = "Return components of the selected actors") )
class UDataprepActorComponentsSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;
};

UCLASS(Category = SelectionTransform, Meta = (DisplayName="Select Owning Actor", ToolTip = "Return the owning actors of selected components") )
class UDataprepOwningActorSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;
};
