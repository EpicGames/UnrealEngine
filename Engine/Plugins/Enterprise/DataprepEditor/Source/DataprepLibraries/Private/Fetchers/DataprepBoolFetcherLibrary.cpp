// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fetchers/DataprepBoolFetcherLibrary.h"


#define LOCTEXT_NAMESPACE "DataprepBoolFetcherLibrary"

/* UDataprepIsOfClassFetcher methods
 *****************************************************************************/
FText UDataprepIsClassOfFetcher::AdditionalKeyword = NSLOCTEXT("DataprepIsClassOfFetcher","AdditionalKeyword","Is Child Of");

bool UDataprepIsClassOfFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if ( Object )
	{
		bOutFetchSucceded = true;
		if ( bShouldIncludeChildClass )
		{
			return Object->GetClass()->IsChildOf( Class );
		}
		else
		{
			return Object->GetClass() == Class.Get();
		}
	}

	bOutFetchSucceded = false;
	return false;
}

bool UDataprepIsClassOfFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepIsClassOfFetcher::GetAdditionalKeyword_Implementation() const
{
	return AdditionalKeyword;
}

FText UDataprepIsClassOfFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("ClassFilterTitle", "Class");
}

#undef LOCTEXT_NAMESPACE
