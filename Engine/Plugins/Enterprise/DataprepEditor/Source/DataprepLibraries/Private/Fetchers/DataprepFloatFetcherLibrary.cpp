// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fetchers/DataprepFloatFetcherLibrary.h"

#include "Components/PrimitiveComponent.h"
#include "Engine/StaticMesh.h"

#define LOCTEXT_NAMESPACE "DataprepFloatFetcherLibrary"

/* UDataprepFloatVolumeFetcher methods+
 *****************************************************************************/
float UDataprepFloatBoundingVolumeFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if ( IsValid(Object) )
	{
		TOptional<float> Volume;
		if ( const AActor* Actor = Cast<const AActor>( Object ) )
		{
			const FBox ActorBox = Actor->GetComponentsBoundingBox();
			if ( ActorBox.IsValid )
			{
				Volume = ActorBox.GetVolume();
			}
		}
		else if ( const UStaticMesh* StaticMesh = Cast<const UStaticMesh>( Object ) )
		{
			const FBox MeshBox = StaticMesh->GetBoundingBox();

			if (MeshBox.IsValid)
			{
				Volume = MeshBox.GetVolume();
			}
		}
		else if ( const UPrimitiveComponent* PrimComp = Cast<const UPrimitiveComponent>( Object ) )
		{
			const FBox CompBounds = PrimComp->CalcBounds(PrimComp->GetComponentToWorld()).GetBox();

			if (CompBounds.IsValid)
			{
				Volume = CompBounds.GetVolume();
			}
		}

		if ( Volume.IsSet() )
		{
			bOutFetchSucceded = true;
			return Volume.GetValue();
		}
	}

	bOutFetchSucceded = false;
	return {};
}

bool UDataprepFloatBoundingVolumeFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepFloatBoundingVolumeFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("BoundingVolumeFilterTitle", "Bounding Volume");
}

#undef LOCTEXT_NAMESPACE
