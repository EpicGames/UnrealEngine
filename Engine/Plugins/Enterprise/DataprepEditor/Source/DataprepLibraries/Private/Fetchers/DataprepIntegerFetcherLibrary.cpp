// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fetchers/DataprepIntegerFetcherLibrary.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshResources.h"

#define LOCTEXT_NAMESPACE "DataprepIntegerFetcherLibrary"

/* UDataprepTriangleCountFetcher methods+
 *****************************************************************************/
int32 UDataprepTriangleCountFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	bOutFetchSucceded = false;

	if ( IsValid(Object) )
	{
		auto GetStaticMeshTriangleCount = [](const UStaticMesh* StaticMesh) -> int
		{
			if ( !StaticMesh )
			{
				return 0;
			}

			check( StaticMesh->GetRenderData() );

			return StaticMesh->GetRenderData()->LODResources[0].GetNumTriangles();
		};

		int TriangleCount = 0;

		if ( const AActor* Actor = Cast<const AActor>( Object ) )
		{
			for ( const UActorComponent* ActorComponent : Actor->GetComponents() )
			{
				if ( const UPrimitiveComponent* PrimComp = Cast<const UPrimitiveComponent>( ActorComponent ) )
				{
					if ( const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>( ActorComponent ) )
					{
						TriangleCount += GetStaticMeshTriangleCount( StaticMeshComponent->GetStaticMesh() );
						bOutFetchSucceded = true;
					}
				}
			}
		}
		else if ( const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>( Object ) )
		{
			if ( const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh() )
			{
				TriangleCount = GetStaticMeshTriangleCount( StaticMesh );
				bOutFetchSucceded = true;
			}
		}
		else if ( const UStaticMesh* StaticMesh = Cast<const UStaticMesh>( Object ) )
		{
			TriangleCount = GetStaticMeshTriangleCount( StaticMesh );
			bOutFetchSucceded = true;
		}
		
		return TriangleCount;
	}

	return {};
}

bool UDataprepTriangleCountFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepTriangleCountFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("TriangleCountFilterTitle", "Triangle Count");
}

/* UDataprepVertexCountFetcher methods+
 *****************************************************************************/
int32 UDataprepVertexCountFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	bOutFetchSucceded = false;

	if ( IsValid(Object) )
	{
		auto GetStaticMeshVertexCount = []( const UStaticMesh* StaticMesh ) -> int
		{
			if ( !StaticMesh )
			{
				return 0;
			}

			check( StaticMesh->GetRenderData());

			return StaticMesh->GetRenderData()->LODResources[0].GetNumVertices();
		};

		int VertexCount = 0;

		if ( const AActor* Actor = Cast<const AActor>(Object) )
		{
			for ( const UActorComponent* ActorComponent : Actor->GetComponents() )
			{
				if ( const UPrimitiveComponent* PrimComp = Cast<const UPrimitiveComponent>(ActorComponent) )
				{
					if ( const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(ActorComponent) )
					{
						VertexCount += GetStaticMeshVertexCount( StaticMeshComponent->GetStaticMesh() );
						bOutFetchSucceded = true;
					}
				}
			}
		}
		else if ( const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>( Object ) )
		{
			if ( const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh() )
			{
				VertexCount = GetStaticMeshVertexCount( StaticMesh );
				bOutFetchSucceded = true;
			}
		}
		else if ( const UStaticMesh* StaticMesh = Cast<const UStaticMesh>(Object) )
		{
			VertexCount = GetStaticMeshVertexCount(StaticMesh);
			bOutFetchSucceded = true;
		}

		return VertexCount;
	}

	return {};
}

bool UDataprepVertexCountFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepVertexCountFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("VertexCountFilterTitle", "Vertex Count");
}

#undef LOCTEXT_NAMESPACE
