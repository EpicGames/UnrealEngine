// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fetchers/DataprepStringFetcherLibrary.h"

#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "DataprepStringFetcherLibrary"

/* UDataprepStringObjectNameFetcher methods
 *****************************************************************************/
FString UDataprepStringObjectNameFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if ( Object )
	{
		bOutFetchSucceded = true;
		return Object->GetName();
	}

	bOutFetchSucceded = false;
	return {};
}

bool UDataprepStringObjectNameFetcher::IsThreadSafe() const 
{
	return true;
}

FText UDataprepStringObjectNameFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("ObjectNameFilterTitle", "Object Name");
}

/* UDataprepStringActorLabelFetcher methods
 *****************************************************************************/
FString UDataprepStringActorLabelFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if ( const AActor* Actor = Cast<const AActor>( Object ) )
	{
		bOutFetchSucceded = true;
		return Actor->GetActorLabel();
	}

	bOutFetchSucceded = false;
	return {};
}

bool UDataprepStringActorLabelFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepStringActorLabelFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("ActorLabelFilterTitle", "Actor Label");
}

#undef LOCTEXT_NAMESPACE
