// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fetchers/DataprepStringsArrayFetcherLibrary.h"

#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "DataprepStringsArrayFetcherLibrary"

/* UDataprepStringActorTagsFetcher methods
 *****************************************************************************/
TArray<FString> UDataprepStringActorTagsFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	if (const AActor* Actor = Cast<const AActor>(Object))
	{
		bOutFetchSucceded = true;
		TArray<FString> Tags;
		Tags.Reserve(Actor->Tags.Num());
		for (int TagIndex = 0; TagIndex < Actor->Tags.Num(); ++TagIndex)
		{
			Tags.Add(Actor->Tags[TagIndex].ToString());
		}
		return MoveTemp(Tags);
	}
	else if (const UActorComponent* Component = Cast<const UActorComponent>(Object))
	{
		bOutFetchSucceded = true;
		TArray<FString> Tags;
		Tags.Reserve(Component->ComponentTags.Num());
		for (int TagIndex = 0; TagIndex < Component->ComponentTags.Num(); ++TagIndex)
		{
			Tags.Add(Component->ComponentTags[TagIndex].ToString());
		}
		return MoveTemp(Tags);
	}

	bOutFetchSucceded = false;
	return {};
}

bool UDataprepStringActorTagsFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepStringActorTagsFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("ActorTagsFilterTitle", "Tag");
}

/* UDataprepStringActorLayersFetcher methods
 *****************************************************************************/
TArray<FString> UDataprepStringActorLayersFetcher::Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const
{
	const AActor* Actor = Cast<const AActor>(Object);
	if (Actor)
	{
		bOutFetchSucceded = true;
		TArray<FString> Layers;
		Layers.Reserve(Actor->Layers.Num());
		for (int LayerIndex = 0; LayerIndex < Actor->Layers.Num(); ++LayerIndex)
		{
			Layers.Add(Actor->Layers[LayerIndex].ToString());
		}
		return MoveTemp(Layers);
	}

	bOutFetchSucceded = false;
	return {};
}

bool UDataprepStringActorLayersFetcher::IsThreadSafe() const
{
	return true;
}

FText UDataprepStringActorLayersFetcher::GetNodeDisplayFetcherName_Implementation() const
{
	return LOCTEXT("ActorLayersFilterTitle", "Layer");
}

#undef LOCTEXT_NAMESPACE
