// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class UStaticMesh;
class UMaterialInterface;

namespace DataprepOperationsLibraryUtil
{
	class DATAPREPLIBRARIES_API FStaticMeshBuilder
	{
	public:
		FStaticMeshBuilder(const TSet<UStaticMesh*>& InStaticMeshes);
		~FStaticMeshBuilder();
	private:
		TArray<UStaticMesh*> StaticMeshes;
	};

	class DATAPREPLIBRARIES_API FScopedStaticMeshEdit final
	{
	public:
		FScopedStaticMeshEdit( UStaticMesh* InStaticMesh );

		FScopedStaticMeshEdit( const FScopedStaticMeshEdit& Other ) = default;
		FScopedStaticMeshEdit( FScopedStaticMeshEdit&& Other ) = default;

		FScopedStaticMeshEdit& operator=( const FScopedStaticMeshEdit& Other ) = default;
		FScopedStaticMeshEdit& operator=( FScopedStaticMeshEdit&& Other ) = default;

		~FScopedStaticMeshEdit();

	public:
		static TArray< FMeshBuildSettings > PreventStaticMeshBuild( UStaticMesh* StaticMesh );

		static void RestoreStaticMeshBuild( UStaticMesh* StaticMesh, const TArray< FMeshBuildSettings >& BuildSettingsBackup );

	private:
		TArray< FMeshBuildSettings > BuildSettingsBackup;
		UStaticMesh* StaticMesh;
	};

	/*
	 * Builds render data of a set of static meshes.
	 * @param StaticMeshes	Set of static meshes to build if render data is missing or a forced build is required
	 * @param bForceBuild	Indicates if all static meshes should be built or only the incomplete ones
	 * @returns the array of static meshes which have actually been built
	 */
	TArray<UStaticMesh*> DATAPREPLIBRARIES_API BuildStaticMeshes( const TSet<UStaticMesh*>& StaticMeshes, bool bForceBuild = false );

	/*
	 * Find the set of static meshes in or referenced by a given array of objects.
	 * @param SelectedObjects	Array of UObjects to go through
	 * @returns a set of static meshes
	 */
	TSet<UStaticMesh*> DATAPREPLIBRARIES_API GetSelectedMeshes(const TArray<UObject*>& SelectedObjects);

	/*
	 * Find the materials used by a set of objects.
	 * @param SelectedObjects	Array of UObjects to go through
	 * @returns array of materials
	 */
	TArray<UMaterialInterface*> DATAPREPLIBRARIES_API GetUsedMaterials(const TArray<UObject*>& SelectedObjects);

	/*
	 * Find the meshes referenced by a set of actors.
	 * @param SelectedObjects	Array of UObjects to go through
	 * @returns array of static meshes used by the actors. Meshes in the result array are unique
	 */
	TArray<UStaticMesh*> DATAPREPLIBRARIES_API GetUsedMeshes(const TArray<UObject*>& SelectedObjects);

	/* 
	 * Customized version of UStaticMesh::SetMaterial avoiding the triggering of UStaticMesh::Build and its side-effects 
	 */
	void DATAPREPLIBRARIES_API SetMaterial(UStaticMesh* StaticMesh, int32 MaterialIndex, UMaterialInterface* NewMaterial);
}
