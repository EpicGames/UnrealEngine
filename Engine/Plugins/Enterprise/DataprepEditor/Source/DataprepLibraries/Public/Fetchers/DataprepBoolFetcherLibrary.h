// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepBoolFetcher.h"

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"

#include "DataprepBoolFetcherLibrary.generated.h"

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Is Class Of", ToolTip = "Filter objects based of their selected class."))
class UDataprepIsClassOfFetcher final : public UDataprepBoolFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepBoolFetcher interface
	virtual bool Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const final;
	//~ End of UDataprepFloatFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetAdditionalKeyword_Implementation() const;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface

	// The key for the for the string
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Settings")
	TSubclassOf<UObject> Class;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Settings")
	bool bShouldIncludeChildClass = true;

	static FText AdditionalKeyword;
};
