// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepFloatFetcher.h"

#include "CoreMinimal.h"

#include "DataprepFloatFetcherLibrary.generated.h"

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Bounding Volume", ToolTip = "Filter objects based on their bounding box volume.\n For actor's bounding box only the components with a collision enabled will be used."))
class UDataprepFloatBoundingVolumeFetcher final : public UDataprepFloatFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepFloatFetcher interface
	virtual float Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const final;
	//~ End of UDataprepFloatFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};
