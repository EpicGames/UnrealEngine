// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepIntegerFetcher.h"

#include "CoreMinimal.h"

#include "DataprepIntegerFetcherLibrary.generated.h"

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Triangle Count", ToolTip = "Filter objects based on their triangle count."))
class UDataprepTriangleCountFetcher final : public UDataprepIntegerFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepIntegerFetcher interface
	virtual int32 Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const final;
	//~ End of UDataprepIntegerFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName = "Vertex Count", ToolTip = "Filter objects based on their vertex count."))
class UDataprepVertexCountFetcher final : public UDataprepIntegerFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepIntegerFetcher interface
	virtual int32 Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const final;
	//~ End of UDataprepIntegerFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};
