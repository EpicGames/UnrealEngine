// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepStringFetcher.h"

#include "CoreMinimal.h"

#include "DataprepStringFetcherLibrary.generated.h"

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Object Name", ToolTip="Filter objects based on their names."))
class UDataprepStringObjectNameFetcher final : public UDataprepStringFetcher
{
	GENERATED_BODY()
public:
	//~ UDataprepStringFetcher interface
	virtual FString Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const final;
	//~ End of UDataprepStringFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName="Actor Label", ToolTip="Filter actors based on their label."))
class UDataprepStringActorLabelFetcher final : public UDataprepStringFetcher
{
	GENERATED_BODY()

public:
	//~ UDataprepStringFetcher interface
	virtual FString Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const override;
	//~ End of UDataprepStringFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};
