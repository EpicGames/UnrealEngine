// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepStringsArrayFetcher.h"

#include "CoreMinimal.h"

#include "DataprepStringsArrayFetcherLibrary.generated.h"

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName = "Tag Value", ToolTip = "Filter actors based on the key values of their tags."))
class UDataprepStringActorTagsFetcher final : public UDataprepStringsArrayFetcher
{
	GENERATED_BODY()

public:
	//~ UDataprepStringsArrayFetcher interface
	virtual TArray<FString> Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const override;
	//~ End of UDataprepStringsArrayFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};

UCLASS(BlueprintType, NotBlueprintable, Meta = (DisplayName = "Actor Layer", ToolTip = "Filter actors based on their layers."))
class UDataprepStringActorLayersFetcher final : public UDataprepStringsArrayFetcher
{
	GENERATED_BODY()

public:
	//~ UDataprepStringsArrayFetcher interface
	virtual TArray<FString> Fetch_Implementation(const UObject* Object, bool& bOutFetchSucceded) const override;
	//~ End of UDataprepStringsArrayFetcher interface

	//~ UDataprepFetcher interface
	virtual bool IsThreadSafe() const final;
	virtual FText GetNodeDisplayFetcherName_Implementation() const;
	//~ End of UDataprepFetcher interface
};
